[CoreRedirects]
; Redirects from MLDeformer to MLDeformerFramework.
+ClassRedirects=(OldName="/Script/MLDeformer.MLDeformerAsset",NewName="/Script/MLDeformerFramework.MLDeformerAsset")

; Property redirects.
+PropertyRedirects=(OldName="/Script/MLDeformerFramework.MLDeformerInputInfo.BoneNameStrings",NewName="BoneNames")
+PropertyRedirects=(OldName="/Script/MLDeformerFramework.MLDeformerInputInfo.CurveNameStrings",NewName="CurveNames")
+PropertyRedirects=(OldName="/Script/MLDeformerFramework.MLDeformerMorphModel.MorphTargetErrorTolerance",NewName="MorphCompressionLevel")
+PropertyRedirects=(OldName="/Script/MLDeformerFramework.MLDeformerMorphModel.MorphTargetDeltaThreshold",NewName="MorphDeltaZeroThreshold")

