{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "ML Deformer Framework",
	"Description": "Machine Learning Mesh Deformer Framework",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/5.0/en-US/using-the-machine-learning-deformer-in-unreal-engine/",
	"MarketplaceURL": "",
	"SupportURL": "https://forums.unrealengine.com/",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "MLDeformerFramework",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MLDeformerFrameworkEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "DeformerGraph",
			"Enabled": true
		},
		{
			"Name": "PythonMLPackages",
			"Enabled": true
		},
		{
			"Name": "GeometryCache",
			"Enabled": true
		},
		{
			"Name": "GameplayInsights",
			"Enabled": true
		},
		{
			"Name": "ModelingToolsEditorMode",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "SkeletalMeshModelingTools",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		}
	]
}