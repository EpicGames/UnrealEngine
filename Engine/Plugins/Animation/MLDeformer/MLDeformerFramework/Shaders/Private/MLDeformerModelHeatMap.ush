// Copyright Epic Games, Inc. All Rights Reserved.

uint {DataInterfaceName}_NumVertices;
uint {DataInterfaceName}_HeatMapMode;
float {DataInterfaceName}_HeatMapMax;
float {DataInterfaceName}_GroundTruthLerp;
uint {DataInterfaceName}_GroundTruthBufferSize;
StructuredBuffer<float> {DataInterfaceName}_PositionGroundTruthBuffer;
Buffer<uint> {DataInterfaceName}_VertexMapBuffer;

uint ReadNumVertices_{DataInterfaceName}()
{
	return {DataInterfaceName}_NumVertices;
}

float3 ReadPositionGroundTruth_{DataInterfaceName}(uint VertexIndex)
{
	uint BufferIndex = {DataInterfaceName}_VertexMapBuffer[VertexIndex];
	if (BufferIndex >= {DataInterfaceName}_GroundTruthBufferSize)
	{
		return float3(0, 0, 0);
	}
	BufferIndex *= 3;
	return float3({DataInterfaceName}_PositionGroundTruthBuffer[BufferIndex], {DataInterfaceName}_PositionGroundTruthBuffer[BufferIndex + 1], {DataInterfaceName}_PositionGroundTruthBuffer[BufferIndex + 2]);
}

uint ReadHeatMapMode_{DataInterfaceName}()
{
	return {DataInterfaceName}_HeatMapMode;
}

float ReadHeatMapMax_{DataInterfaceName}()
{
	return {DataInterfaceName}_HeatMapMax;
}

float ReadGroundTruthLerp_{DataInterfaceName}()
{
	return {DataInterfaceName}_GroundTruthLerp;
}
