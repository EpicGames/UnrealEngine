// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "PrimitiveComponentId.h"
#include "RenderGraphFwd.h"
#include "DeformerGraphDataInterfaceSkeletonWithQuats.generated.h"

#define UE_API MLDEFORMERFRAMEWORK_API

class FSceneInterface;
class FSkeletalMeshObject;
class FSkeletonWithQuatsDataInterfaceParameters;
class USkinnedMeshComponent;

/** Compute Framework Data Interface for skeletal data. */
UCLASS(MinimalAPI, Category = ComputeFramework)
class UDeformerGraphSkeletonWithWeightedQuatsDataInterface
	: public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:
	//~ Begin UOptimusComputeDataInterface Interface
	UE_API FString GetDisplayName() const override;
	UE_API TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	UE_API TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	//~ End UOptimusComputeDataInterface Interface
	
	//~ Begin UComputeDataInterface Interface
	TCHAR const* GetClassName() const override { return TEXT("MLDeformer_SkeletonWithQuats"); }
	UE_API void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	UE_API void GetPermutations(FComputeKernelPermutationVector& OutPermutationVector) const override;
	UE_API TCHAR const* GetShaderVirtualPath() const override;
	UE_API void GetShaderHash(FString& InOutKey) const override;
	UE_API void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UE_API UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	//~ End UComputeDataInterface Interface

private:
	static UE_API TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for reading skeletal mesh. */
UCLASS(MinimalAPI, BlueprintType, EditInlineNew, Category = ComputeFramework)
class UDeformerGraphSkeletonWithWeightedQuatsDataProvider
	: public UComputeDataProvider
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Binding)
	TObjectPtr<USkinnedMeshComponent> SkinnedMesh = nullptr;

	//~ Begin UComputeDataProvider Interface
	UE_API FComputeDataProviderRenderProxy* GetRenderProxy() override;
	//~ End UComputeDataProvider Interface
};

namespace UE::MLDeformer
{
	class FDeformerGraphSkeletonWithWeightedQuatsDataProviderProxy
		: public FComputeDataProviderRenderProxy
	{
	public:
		UE_API FDeformerGraphSkeletonWithWeightedQuatsDataProviderProxy(USkinnedMeshComponent* SkinnedMeshComponent);

		//~ Begin FComputeDataProviderRenderProxy Interface
		UE_API bool IsValid(FValidationData const& InValidationData) const override;
		UE_API void GatherPermutations(FPermutationData& InOutPermutationData) const override;
		UE_API void GatherDispatchData(FDispatchData const& InDispatchData) override;
		UE_API void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
		//~ End FComputeDataProviderRenderProxy Interface

	private:
		using FParameters = FSkeletonWithQuatsDataInterfaceParameters;
		void CacheRefToLocalQuats(TArray<FQuat4f>& OutRefToLocalQuats) const;

		FSceneInterface* SceneInterface = nullptr;
		FPrimitiveComponentId ComponentId;
		uint32 BoneRevisionNumber = 0;
		TArray<TArray<FQuat4f> > PerSectionRefToLocalQuats;
		TArray<FRDGBufferSRVRef> PerSectionRefToLocalQuatsSRVs;
	};
}	// namespace UE::MLDeformer

#undef UE_API
