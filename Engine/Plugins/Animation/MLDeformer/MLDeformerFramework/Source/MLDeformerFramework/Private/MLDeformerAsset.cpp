// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerAsset.h"
#include "MLDeformerObjectVersion.h"
#include "MLDeformerModel.h"
#include "MLDeformerInputInfo.h"
#include "Engine/SkeletalMesh.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MLDeformerAsset)

void UMLDeformerAsset::Serialize(FArchive& Archive)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMLDeformerAsset::Serialize)

	Archive.UsingCustomVersion(UE::MLDeformer::FMLDeformerObjectVersion::GUID);
	Super::Serialize(Archive);
}

void UMLDeformerAsset::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);
	if (Model)
	{
		Model->GetAssetRegistryTags(Context);
	}
}
