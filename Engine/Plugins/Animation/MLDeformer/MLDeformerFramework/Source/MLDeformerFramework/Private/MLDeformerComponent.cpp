// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerComponent.h"
#include "MLDeformerAsset.h"
#include "MLDeformerModelInstance.h"
#include "MLDeformerModel.h"
#include "MLDeformerInputInfo.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/MeshDeformer.h"
#include "Animation/MeshDeformerInstance.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MLDeformerComponent)

static float GMLDeformerOverrideWeight = -1;
static FAutoConsoleVariableRef CVarMLDeformerOverrideWeight(
	TEXT("MLDeformer.ForceWeight"),
	GMLDeformerOverrideWeight,
	TEXT("Force the Weight for MLDeformer components.")
	TEXT("1 will force completely on, 0 will force completely off.")
	TEXT("Negative values (the default) will use default weights."),
	ECVF_Default
);

UMLDeformerComponent::UMLDeformerComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bTickInEditor = true;
	bAutoActivate = true;
	PrimaryComponentTick.TickGroup = TG_PostPhysics;
	PrimaryComponentTick.bCanEverTick = true;
}

void UMLDeformerComponent::Init()
{
	UnbindDelegates();

	// If there is no deformer asset linked, release what we currently have.
	if (!DeformerAsset)
	{
		ReleaseModelInstance();
		return;
	}

	// Try to initialize the deformer model.
	UMLDeformerModel* Model = DeformerAsset->GetModel();
	if (Model)
	{
		ReleaseModelInstance();
		ModelInstance = Model->CreateModelInstance(this);
		ModelInstance->SetModel(Model);
		ModelInstance->Init(SkelMeshComponent);
		ModelInstance->PostMLDeformerComponentInit();
		BindDelegates();
	}
	else
	{
		ReleaseModelInstance();
		UE_LOGF(LogMLDeformer, Warning, "ML Deformer component on '%ls' has a deformer asset that has no ML model setup.", *GetOuter()->GetName());
	}
}

void UMLDeformerComponent::SetupComponent(UMLDeformerAsset* InDeformerAsset, USkeletalMeshComponent* InSkelMeshComponent)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMLDeformerComponent::SetupComponent)

	RenderCommandFence.BeginFence();
	RenderCommandFence.Wait();

	// Remove the existing skeletal mesh component as tick prerequisite.
	if (SkelMeshComponent)
	{
		RemoveTickPrerequisiteComponent(SkelMeshComponent);
	}

	// Add the new one.
	if (InSkelMeshComponent)
	{		
		AddTickPrerequisiteComponent(InSkelMeshComponent);
	}

	UnbindDelegates();

	DeformerAsset = InDeformerAsset;
	SkelMeshComponent = InSkelMeshComponent;

	// Initialize and make sure we have a model instance.
	Init();

	// Verify that a mesh deformer has been setup when the used ML model requires one.
	if (!bSuppressMeshDeformerLogWarnings && DeformerAsset && ModelInstance && ModelInstance->GetModel() && SkelMeshComponent)
	{
		const FString DefaultGraph = ModelInstance->GetModel()->GetDefaultDeformerGraphAssetPath();
		if (!DefaultGraph.IsEmpty() && !SkelMeshComponent->HasMeshDeformer())
		{
			UE_LOG(LogMLDeformer,
				Warning, 
				TEXT("ML Deformer Asset '%s' used with skel mesh component '%s' on actor '%s' uses model type '%s', which requires a deformer graph (default='%s'). " \
					"No Mesh Deformer has been set on the mesh or skel mesh component, so linear skinning will be used."),
					*DeformerAsset->GetName(),
					*SkelMeshComponent->GetName(),
					*SkelMeshComponent->GetOuter()->GetName(),
					*ModelInstance->GetModel()->GetDisplayName(),
					*ModelInstance->GetModel()->GetDefaultDeformerGraphAssetPath()
			);
		}
	}

	#if WITH_EDITOR
		TickPerfCounter.Reset();
	#endif
}

void UMLDeformerComponent::BindDelegates()
{
	UMLDeformerModel* Model = DeformerAsset ? DeformerAsset->GetModel() : nullptr;
	if (Model && !ReinitModelInstanceDelegateHandle.IsValid())
	{
		ReinitModelInstanceDelegateHandle = Model->GetReinitModelInstanceDelegate().AddLambda( 
			[this]()
			{
				Init();
			});
	}
	else
	{
		ReinitModelInstanceDelegateHandle = FDelegateHandle();
	}
}

void UMLDeformerComponent::UnbindDelegates()
{
	UMLDeformerModel* Model = DeformerAsset ? DeformerAsset->GetModel() : nullptr;
	if (Model && ReinitModelInstanceDelegateHandle.IsValid())
	{
		Model->GetReinitModelInstanceDelegate().Remove(ReinitModelInstanceDelegateHandle);	
		ReinitModelInstanceDelegateHandle = FDelegateHandle();
	}
}

void UMLDeformerComponent::BeginDestroy()
{
	UnbindDelegates();
	ReleaseModelInstance();
	Super::BeginDestroy();
}

void UMLDeformerComponent::ReleaseModelInstance()
{
	if (ModelInstance && IsValid(ModelInstance))
	{
		ModelInstance->ConditionalBeginDestroy(); // Force destruction immediately instead of waiting for the next GC.
		ModelInstance = nullptr;
	}
}

USkeletalMeshComponent* UMLDeformerComponent::FindSkeletalMeshComponent(const UMLDeformerAsset* const Asset) const
{
	USkeletalMeshComponent* ResultingComponent = nullptr;
	if (Asset)
	{
		// First search for a skeletal mesh component that uses the same skeletal mesh as the ML Deformer asset was trained on.
		const UMLDeformerModel* Model = Asset ? Asset->GetModel() : nullptr;
		if (Model && Model->GetInputInfo() && Model->GetInputInfo()->GetSkeletalMesh().IsValid())
		{
			const FSoftObjectPath& ModelSkeletalMesh = Model->GetInputInfo()->GetSkeletalMesh();

			// Get a list of all skeletal mesh components on the actor.
			TArray<USkeletalMeshComponent*> Components;
			const AActor* Actor = Cast<AActor>(GetOuter());
			if (Actor)
			{
				Actor->GetComponents<USkeletalMeshComponent>(Components);
		
				// Find a component that uses a mesh with the same vertex count.
				for (USkeletalMeshComponent* Component : Components)
				{
					const USkeletalMesh* ComponentSkeletalMesh = Component->GetSkeletalMeshAsset();
					if (FSoftObjectPath(ComponentSkeletalMesh) == ModelSkeletalMesh)
					{
						ResultingComponent = Component;
						break;
					}
				}
			}
		}
	}

	return ResultingComponent;
}

void UMLDeformerComponent::UpdateSkeletalMeshComponent()
{
	USkeletalMeshComponent* Component = FindSkeletalMeshComponent(DeformerAsset.Get());
	SetupComponent(DeformerAsset, Component);
}

void UMLDeformerComponent::Activate(bool bReset)
{
	UpdateSkeletalMeshComponent();
	Super::Activate(bReset);
}

void UMLDeformerComponent::Deactivate()
{
	#if WITH_EDITOR
		TickPerfCounter.Reset();
	#endif

	UnbindDelegates();
	ReleaseModelInstance();
	Super::Deactivate();
}


float UMLDeformerComponent::GetFinalMLDeformerWeight() const
{
	float FinalWeight = Weight;
	if (GMLDeformerOverrideWeight >= 0.0f)
	{
		FinalWeight = FMath::Min(GMLDeformerOverrideWeight, 1.0f);
	}
	return FinalWeight;
}

void UMLDeformerComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	#if WITH_EDITOR
		TickPerfCounter.BeginSample();
	#endif
	SCOPE_CYCLE_COUNTER(STAT_MLDeformerInference);
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	if (TickType != ELevelTick::LEVELTICK_PauseTick)
	{
		if (ModelInstance &&
			ModelInstance->GetModel() &&
			SkelMeshComponent && 
			SkelMeshComponent->GetPredictedLODLevel() < ModelInstance->GetModel()->GetMaxNumLODs())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UMLDeformerComponent::TickComponent)
			
			const float ApplyWeight = GetFinalMLDeformerWeight();
			ModelInstance->Tick(DeltaTime, ApplyWeight);

			#if WITH_EDITOR
				// Update our memory usage if desired.
				// This can be pretty slow.
				UMLDeformerModel* Model = ModelInstance->GetModel();
				if (Model->IsMemUsageInvalidated())
				{
					Model->UpdateMemoryUsage();
				}
			#endif
		}
	}

	#if WITH_EDITOR
		TickPerfCounter.EndSample();
	#endif
}

void UMLDeformerComponent::SetWeightInternal(const float NormalizedWeightValue)
{ 
	Weight = FMath::Clamp<float>(NormalizedWeightValue, 0.0f, 1.0f);
}

void UMLDeformerComponent::SetDeformerAssetInternal(UMLDeformerAsset* const InDeformerAsset)
{ 
	USkeletalMeshComponent* SkelMeshComp = FindSkeletalMeshComponent(InDeformerAsset);
	SetupComponent(InDeformerAsset, SkelMeshComp);
}

#if WITH_EDITOR
	void UMLDeformerComponent::PreEditChange(FProperty* Property)
	{
		if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMLDeformerComponent, DeformerAsset))
		{
			UnbindDelegates();
		}
	}

	void UMLDeformerComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
	{
		const FProperty* Property = PropertyChangedEvent.Property;
		if (Property == nullptr)
		{
			return;
		}

		if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMLDeformerComponent, DeformerAsset))
		{
			SetDeformerAsset(DeformerAsset);
		}
	}
#endif
