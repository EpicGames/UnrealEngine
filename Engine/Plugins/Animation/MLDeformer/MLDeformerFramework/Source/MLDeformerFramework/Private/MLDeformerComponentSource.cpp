// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerComponentSource.h"

#include "MLDeformerComponent.h"
#include "MLDeformerModelInstance.h"

#include "Components/SkeletalMeshComponent.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "SkeletalRenderPublic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MLDeformerComponentSource)


#define LOCTEXT_NAMESPACE "MLDeformerComponentSource"


FName UMLDeformerComponentSource::Domains::Vertex("Vertex");


FText UMLDeformerComponentSource::GetDisplayName() const
{
	return LOCTEXT("MLDeformerComponent", "ML Deformer Component");
}


TSubclassOf<UActorComponent> UMLDeformerComponentSource::GetComponentClass() const
{
	return UMLDeformerComponent::StaticClass();
}


TArray<FName> UMLDeformerComponentSource::GetExecutionDomains() const
{
	return {Domains::Vertex};
}


int32 UMLDeformerComponentSource::GetLodIndex(const UActorComponent* InComponent) const
{
	const UMLDeformerComponent* DeformerComponent = Cast<UMLDeformerComponent>(InComponent);
	const UMLDeformerModelInstance* ModelInstance = DeformerComponent ? DeformerComponent->GetModelInstance() : nullptr;
	const USkeletalMeshComponent* SkeletalMeshComponent = ModelInstance ? ModelInstance->GetSkeletalMeshComponent() : nullptr;
	const FSkeletalMeshObject* SkeletalMeshObject = SkeletalMeshComponent ? SkeletalMeshComponent->MeshObject : nullptr;
	return SkeletalMeshObject ? SkeletalMeshObject->GetLOD() : 0;
}


uint32 UMLDeformerComponentSource::GetDefaultNumInvocations(
	const UActorComponent* InComponent,
    int32 InLod
    ) const
{
	const USkinnedMeshComponent* SkinnedMeshComponent = Cast<USkinnedMeshComponent>(InComponent);
	if (!SkinnedMeshComponent)
	{
		return 0;
	}

	const FSkeletalMeshObject* SkeletalMeshObject = SkinnedMeshComponent->MeshObject;
	if (!SkeletalMeshObject)
	{
		return 0;
	}

	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[InLod];

	return LodRenderData->RenderSections.Num();
}


bool UMLDeformerComponentSource::GetComponentElementCountsForExecutionDomain(
	FName InDomainName,
	const UActorComponent* InComponent,
	int32 InLodIndex,
	TArray<int32>& OutElementCounts
	) const
{
	if (InDomainName != Domains::Vertex)
	{
		return false;
	}

	const UMLDeformerComponent* DeformerComponent = Cast<UMLDeformerComponent>(InComponent);
	if (!DeformerComponent)
	{
		return false;
	}
	
	const UMLDeformerModelInstance* ModelInstance = DeformerComponent->GetModelInstance();
	if (!ModelInstance)
	{
		return false;
	}
		
	const FSkeletalMeshObject* SkeletalMeshObject = ModelInstance->GetSkeletalMeshComponent()->MeshObject;
	if (!SkeletalMeshObject)
	{
		return false;
	}
	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = &SkeletalMeshRenderData.LODRenderData[InLodIndex];
		
	OutElementCounts.Reset(LodRenderData->RenderSections.Num());
	for (FSkelMeshRenderSection const& RenderSection: LodRenderData->RenderSections)
	{
		OutElementCounts.Add(RenderSection.NumVertices);
	}
	return true;
}

#undef LOCTEXT_NAMESPACE
