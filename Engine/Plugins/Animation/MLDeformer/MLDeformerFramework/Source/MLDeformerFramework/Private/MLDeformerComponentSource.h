// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusComponentSource.h"

#include "MLDeformerComponentSource.generated.h"


UCLASS(meta=(DisplayName="ML Deformer Component"))
class UMLDeformerComponentSource :
	public UOptimusComponentSource
{
	GENERATED_BODY()
public:
	struct Domains
	{
		static FName Vertex;
	};
	
	// UOptimusComponentSource implementations
	FText GetDisplayName() const override;
	FName GetBindingName() const override { return FName("MLDeformer"); }
	TSubclassOf<UActorComponent> GetComponentClass() const override;
	TArray<FName> GetExecutionDomains() const override;
	int32 GetLodIndex(const UActorComponent* InComponent) const override;
	uint32 GetDefaultNumInvocations(const UActorComponent* InComponent, int32 InLod) const override;
	bool GetComponentElementCountsForExecutionDomain(FName InDomainName, const UActorComponent* InComponent, int32 InLodIndex, TArray<int32>& OutElementCounts) const override;
};
