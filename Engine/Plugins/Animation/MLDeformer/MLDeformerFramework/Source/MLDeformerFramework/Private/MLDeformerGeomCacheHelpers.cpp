// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerGeomCacheHelpers.h"
#include "MLDeformerModel.h"
#include "MLDeformerModule.h"
#include "GeometryCache.h"
#include "GeometryCacheMeshData.h"
#include "GeometryCacheTrack.h"
#include "SkeletalMeshAttributes.h"
#include "Animation/AnimSequence.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Async/ParallelFor.h"

#define LOCTEXT_NAMESPACE "MLDeformerGeomCacheHelpers"

namespace UE::MLDeformer
{
#if WITH_EDITORONLY_DATA
	FText GetGeomCacheErrorText(USkeletalMesh* InSkeletalMesh, UGeometryCache* InGeomCache)
	{
		FText Result;
		if (InGeomCache)
		{
			FString ErrorString;

			// Verify that we have imported vertex numbers enabled.
			TArray<FGeometryCacheMeshData> MeshData;
			InGeomCache->GetMeshDataAtTime(0.0f, MeshData);
			if (MeshData.Num() == 0)
			{
				ErrorString = FText(LOCTEXT("TargetMeshNoMeshData", "No geometry data is present.")).ToString();
			}
			else
			{
				if (MeshData[0].ImportedVertexNumbers.Num() == 0)
				{
					ErrorString = FText(LOCTEXT("TargetMeshNoImportedVertexNumbers", "Please import Geometry Cache with option 'Store Imported Vertex Numbers' enabled!")).ToString();
				}
			}

			// Check if we flattened the tracks.
			if (InGeomCache->Tracks.Num() == 1 && InGeomCache->Tracks[0]->GetName() == TEXT("Flattened_Track"))
			{
				int32 NumSkelMeshes = 0;
				if (InSkeletalMesh)
				{
					if (const FMeshDescription* MeshDescription = InSkeletalMesh->GetMeshDescription(0))
					{
						const FSkeletalMeshConstAttributes Attributes(*MeshDescription);
						NumSkelMeshes = Attributes.GetNumSourceGeometryParts();
					}
				}

				if (NumSkelMeshes > 1)
				{
					if (!ErrorString.IsEmpty())
					{
						ErrorString += TEXT("\n\n");
					}
					ErrorString += FText(LOCTEXT("TargetMeshFlattened", "Please import Geometry Cache with option 'Flatten Tracks' disabled!")).ToString();
				}
			}

			Result = FText::FromString(ErrorString);
		}

		return Result;
	}

	FText GetGeomCacheMeshMappingErrorText(USkeletalMesh* InSkelMesh, UGeometryCache* InGeomCache)
	{
		FText Result;
		if (InGeomCache && InSkelMesh)
		{
			// Check for failed mesh mappings.
			TArray<FMLDeformerGeomCacheMeshMapping> MeshMappings;
			TArray<FString> FailedNames;
			TArray<FString> VertexMisMatchNames;
			const bool bSuppressLogMessages = true;
			GenerateGeomCacheMeshMappings(InSkelMesh, InGeomCache, MeshMappings, FailedNames, VertexMisMatchNames, bSuppressLogMessages);

			// List all mesh names that have issues.
			FString ErrorString;
			if (!FailedNames.IsEmpty())
			{
				const FText ErrorText = LOCTEXT("FailedMappingNames", "No SkelMesh meshes found for GeomCache tracks:\n\n");
				ErrorString += ErrorText.ToString();
				for (int32 Index = 0; Index < FailedNames.Num(); ++Index)
				{
					ErrorString += "\t\t" + FailedNames[Index];
					if (Index < FailedNames.Num() - 1)
					{
						ErrorString += TEXT("\n");
					}
				}
			}

			if (VertexMisMatchNames.Num() > 0)
			{
				if (!FailedNames.IsEmpty())
				{
					ErrorString += "\n\n";
				}
				const FText ErrorText = LOCTEXT("FailedMappingVertexMisMatch", "Mismatching vertex counts for:\n\n");
				ErrorString += ErrorText.ToString();
				for (int32 Index = 0; Index < VertexMisMatchNames.Num(); ++Index)
				{
					ErrorString += "\t\t" + VertexMisMatchNames[Index];
					if (Index < VertexMisMatchNames.Num() - 1)
					{
						ErrorString += TEXT("\n");
					}
				}
			}

			Result = FText::FromString(ErrorString);
		}

		return Result;
	}

	// A fuzzy name match.
	// There is a match when the track name starts with the mesh name.
	bool IsPotentialMatch(const FString& TrackName, const FString& MeshName)
	{
		return (TrackName.Find(MeshName) == 0);
	}

	void GenerateGeomCacheMeshMappings(USkeletalMesh* SkelMesh, UGeometryCache* GeomCache, TArray<FMLDeformerGeomCacheMeshMapping>& OutMeshMappings, TArray<FString>& OutFailedImportedMeshNames, TArray<FString>& OutVertexMisMatchNames, bool bSuppressLog)
	{
		OutMeshMappings.Empty();
		OutFailedImportedMeshNames.Empty();
		OutVertexMisMatchNames.Empty();
		if (SkelMesh == nullptr || GeomCache == nullptr)
		{
			return;
		}

		if (!SkelMesh->GetImportedModel()->LODModels.IsValidIndex(0))
		{
			return;
		}

		const FSkeletalMeshLODModel& LODModel = SkelMesh->GetImportedModel()->LODModels[0];

		// If we haven't got any imported mesh infos then the asset needs to be reimported first.
		// We show an error for this in the editor UI already.
		const FMeshDescription* MeshDescription = SkelMesh->GetMeshDescription(0);
		if (!MeshDescription || MeshDescription->IsEmpty())
		{
			return;
		}

		FSkeletalMeshConstAttributes MeshAttributes(*MeshDescription);
		if (!MeshAttributes.HasSourceGeometryParts())
		{
			return;
		}

		// Add all the names to the OutVertexMisMatchNames ahead of time.  They'll be removed as matches are found.
		for (int32 TrackIndex = 0; TrackIndex < GeomCache->Tracks.Num(); ++TrackIndex)
		{
			UGeometryCacheTrack* Track = GeomCache->Tracks[TrackIndex];
			OutVertexMisMatchNames.Add(Track->GetName());
		}

		// For all meshes in the skeletal mesh.
		const float SampleTime = 0.0f;

		const int32 NumSourceGeoParts = MeshAttributes.GetNumSourceGeometryParts();
		const FSkeletalMeshAttributes::FSourceGeometryPartNameConstRef GeoPartNames = MeshAttributes.GetSourceGeometryPartNames();
		const FSkeletalMeshAttributes::FSourceGeometryPartVertexOffsetAndCountConstRef GeoPartOffsetAndCounts = MeshAttributes.GetSourceGeometryPartVertexOffsetAndCounts();	
	
		const bool bIsSoloMesh = (GeomCache->Tracks.Num() == 1 && NumSourceGeoParts == 1);	// Do we just have one mesh and one track?
		for (int32 TrackIndex = 0; TrackIndex < GeomCache->Tracks.Num(); ++TrackIndex)
		{
			// Check if this is a candidate based on the mesh and track name.
			UGeometryCacheTrack* Track = GeomCache->Tracks[TrackIndex];

			bool bFoundMatch = false;
			for (int32 GeoPartIndex = 0; GeoPartIndex < NumSourceGeoParts; ++GeoPartIndex)
			{
				const FSourceGeometryPartID GeoPartID(GeoPartIndex);
				FString SkelMeshName = GeoPartNames[GeoPartID].ToString();

				if (Track &&
					(IsPotentialMatch(Track->GetName(), SkelMeshName) || bIsSoloMesh))
				{
					// Extract the geom cache mesh data.
					FGeometryCacheMeshData GeomCacheMeshData;
					if (!Track->GetMeshDataAtTime(SampleTime, GeomCacheMeshData))
					{
						continue;
					}

					// Verify that we have imported vertex numbers.
					if (GeomCacheMeshData.ImportedVertexNumbers.IsEmpty())
					{
						continue;
					}

					// Get the number of geometry cache mesh imported verts.
					int32 NumGeomMeshVerts = 0;
					for (int32 GeomVertIndex = 0; GeomVertIndex < GeomCacheMeshData.ImportedVertexNumbers.Num(); ++GeomVertIndex)
					{
						NumGeomMeshVerts = FMath::Max(NumGeomMeshVerts, (int32)GeomCacheMeshData.ImportedVertexNumbers[GeomVertIndex]);
					}
					NumGeomMeshVerts += 1;	// +1 Because we use indices, so a cube's max index is 7, while there are 8 vertices.

					// Make sure the vertex counts match.
					TArrayView<const int32> GeoPartInfo = GeoPartOffsetAndCounts.Get(GeoPartIndex);
					const int32 StartImportedVertex = GeoPartInfo[0]; 
					const int32 NumSkelMeshVerts = GeoPartInfo[1];
					if (NumSkelMeshVerts != NumGeomMeshVerts)
					{
						continue;
					}

					// Since it is a match, remove it from the list.
					OutVertexMisMatchNames.Remove(Track->GetName());

					// Create a new mesh mapping entry.
					OutMeshMappings.AddDefaulted();
					UE::MLDeformer::FMLDeformerGeomCacheMeshMapping& Mapping = OutMeshMappings.Last();
					Mapping.MeshIndex = GeoPartIndex;
					Mapping.TrackIndex = TrackIndex;
					Mapping.SkelMeshToTrackVertexMap.AddUninitialized(NumSkelMeshVerts);
					Mapping.ImportedVertexToRenderVertexMap.AddUninitialized(NumSkelMeshVerts);

					// For all vertices (both skel mesh and geom cache mesh have the same number of verts here).
					const int32 BatchSize = 100;
					const int32 NumBatches = (NumSkelMeshVerts / BatchSize) + 1;
					ParallelFor(NumBatches, [&](int32 BatchIndex)
					{
						const int32 StartVertex = BatchIndex * BatchSize;
						if (StartVertex >= NumSkelMeshVerts)
						{
							return;
						}

						const int32 NumVertsInBatch = (StartVertex + BatchSize) < NumSkelMeshVerts ? BatchSize : FMath::Max(NumSkelMeshVerts - StartVertex, 0);
						for (int32 VertexIndex = StartVertex; VertexIndex < StartVertex + NumVertsInBatch; ++VertexIndex)
						{
							// Find the first vertex with the same dcc vertex in the geom cache mesh.
							// When there are multiple vertices with the same vertex number here, they are duplicates with different normals or uvs etc.
							// However they all share the same vertex position, so we can just find the first hit, as we only need the position later on.
							const int32 GeomCacheVertexIndex = GeomCacheMeshData.ImportedVertexNumbers.Find(VertexIndex);
							Mapping.SkelMeshToTrackVertexMap[VertexIndex] = GeomCacheVertexIndex;

							// Map the source asset vertex number to a render vertex. This is the first duplicate of that vertex.
							const int32 RenderVertexIndex = LODModel.MeshToImportVertexMap.Find(StartImportedVertex + VertexIndex);
							Mapping.ImportedVertexToRenderVertexMap[VertexIndex] = RenderVertexIndex;
						}
					});

					// Figure out what materials we use.
					for (int32 VertexIndex = 0; VertexIndex < NumSkelMeshVerts; ++VertexIndex)
					{
						const int32 RenderVertexIndex = Mapping.ImportedVertexToRenderVertexMap[VertexIndex];

						int32 SectionIndex = INDEX_NONE;
						int32 SectionVertexIndex = INDEX_NONE;
						LODModel.GetSectionFromVertexIndex(RenderVertexIndex, SectionIndex, SectionVertexIndex);
						if (SectionIndex != INDEX_NONE && SectionVertexIndex != INDEX_NONE)
						{
							const int32 MaterialIndex = LODModel.Sections[SectionIndex].MaterialIndex;
							Mapping.MaterialIndices.AddUnique(MaterialIndex);
						}
					}

					// We found a match, no need to iterate over more Tracks.
					bFoundMatch = true;
					if (!bSuppressLog)
					{
						UE_LOGF(LogMLDeformer, Verbose, "Mapped geom cache track '%ls' to SkelMesh submesh '%ls'", *Track->GetName(), *SkelMeshName);
					}
					break;
				} // If the track name matches the skeletal meshes internal mesh name.
			} // For all meshes in the Skeletal Mesh.

			if (Track && !bFoundMatch)
			{
				OutFailedImportedMeshNames.Add(Track->GetName());
				if (!bSuppressLog)
				{
					UE_LOGF(LogMLDeformer, Warning, "Geometry cache '%ls' cannot be matched with a mesh inside the Skeletal Mesh.", *Track->GetName());
				}
			}
		} // For all tracks.
	}

	void SampleGeomCachePositionsAtFrame(
		int32 InLODIndex,
		int32 FrameIndex,
		const TArray<UE::MLDeformer::FMLDeformerGeomCacheMeshMapping>& InMeshMappings,
		const USkeletalMesh* InSkelMesh,
		const UGeometryCache* InGeometryCache,
		const FTransform& InAlignmentTransform,
		TArray<FVector3f>& OutPositions)
	{
		if (InGeometryCache == nullptr)
		{
			return;
		}

		if (!ensure(InSkelMesh != nullptr))
		{
			return;
		}

		const FMeshDescription* MeshDescription = InSkelMesh->GetMeshDescription(InLODIndex);
		if (!ensure(MeshDescription))
		{
			return;
		}

		const FSkeletalMeshConstAttributes MeshAttributes(*MeshDescription);
		const FSkeletalMeshAttributes::FSourceGeometryPartVertexOffsetAndCountConstRef GeoPartOffsetAndCounts = MeshAttributes.GetSourceGeometryPartVertexOffsetAndCounts();

		const uint32 NumVertices = MeshDescription->Vertices().Num();
		OutPositions.Reset(NumVertices);
		OutPositions.AddZeroed(NumVertices);

		// For all mesh mappings we found.
		for (int32 MeshMappingIndex = 0; MeshMappingIndex < InMeshMappings.Num(); ++MeshMappingIndex)
		{
			const UE::MLDeformer::FMLDeformerGeomCacheMeshMapping& MeshMapping = InMeshMappings[MeshMappingIndex];
			TArrayView<const int32> GeoPartInfo = GeoPartOffsetAndCounts.Get(MeshMapping.MeshIndex);
			const int32 StartImportedVertex = GeoPartInfo[0];
			const int32 NumImportedVertices = GeoPartInfo[1];

			UGeometryCacheTrack* Track = InGeometryCache->Tracks[MeshMapping.TrackIndex];

			FGeometryCacheMeshData GeomCacheMeshData;
			if (!Track->GetMeshDataAtSampleIndex(FrameIndex, GeomCacheMeshData))
			{
				continue;
			}

			for (int32 VertexIndex = 0; VertexIndex < NumImportedVertices; ++VertexIndex)
			{
				const int32 SkinnedVertexIndex = StartImportedVertex + VertexIndex;
				const int32 GeomCacheVertexIndex = MeshMapping.SkelMeshToTrackVertexMap[VertexIndex];
				if (GeomCacheVertexIndex != INDEX_NONE && GeomCacheMeshData.Positions.IsValidIndex(GeomCacheVertexIndex))
				{
					const FVector3f GeomCacheVertexPos = (FVector3f)InAlignmentTransform.TransformPosition((FVector)GeomCacheMeshData.Positions[GeomCacheVertexIndex]);
					OutPositions[SkinnedVertexIndex] = GeomCacheVertexPos;
				}
			}
		}
	}

	int32 ExtractNumImportedGeomCacheVertices(UGeometryCache* GeometryCache)
	{
		if (GeometryCache == nullptr)
		{
			return 0;
		}

		int32 NumGeomCacheImportedVerts = 0;

		// Extract the geom cache number of imported vertices.
		TArray<FGeometryCacheMeshData> MeshDatas;
		GeometryCache->GetMeshDataAtTime(0.0f, MeshDatas);
		for (const FGeometryCacheMeshData& MeshData : MeshDatas)
		{
			const TArray<uint32>& ImportedVertexNumbers = MeshData.ImportedVertexNumbers;
			if (ImportedVertexNumbers.Num() > 0)
			{
				// Find the maximum value.
				int32 MaxIndex = -1;
				for (int32 Index = 0; Index < ImportedVertexNumbers.Num(); ++Index)
				{
					MaxIndex = FMath::Max(static_cast<int32>(ImportedVertexNumbers[Index]), MaxIndex);
				}
				check(MaxIndex > -1);

				NumGeomCacheImportedVerts += MaxIndex + 1;
			}
		}

		return NumGeomCacheImportedVerts;
	}

	FText GetGeomCacheAnimSequenceErrorText(const UGeometryCache* InGeomCache, const UAnimSequence* InAnimSequence)
	{
		FText Result;

		if (InAnimSequence && InGeomCache)
		{
			const int32 NumGeomCacheFrames = InGeomCache->GetEndFrame() - InGeomCache->GetStartFrame() + 1;
			const int32 NumAnimSeqFrames = InAnimSequence->GetNumberOfSampledKeys();
			const float AnimSeqDuration = InAnimSequence->GetPlayLength();
			const float GeomCacheDuration = InGeomCache->CalculateDuration();
			
			if (NumGeomCacheFrames != NumAnimSeqFrames)
			{
				Result = FText::Format(
					LOCTEXT("AnimSeqNumFramesMismatch", "Anim sequence has {0} frames, while the geometry cache has {1} frames."),
					FText::AsNumber(NumAnimSeqFrames),
					FText::AsNumber(NumGeomCacheFrames));
			}
			
			if (FMath::Abs(AnimSeqDuration - GeomCacheDuration) > 0.001f)
			{
				FNumberFormattingOptions Options;
				Options.SetUseGrouping(false);
				Options.SetMaximumFractionalDigits(3);

				Result = FText::Format(
					LOCTEXT("AnimSeqDurationMismatch", "{0}{1}Anim sequence duration ({2} secs) doesn't match the geometry cache's duration ({3} secs)."),
					Result,
					!Result.IsEmpty() ? FText::FromString("\n\n") : FText::GetEmpty(),
					FText::AsNumber(AnimSeqDuration, &Options),
					FText::AsNumber(GeomCacheDuration, &Options));
			}

			const double AnimSequenceFrameRate = InAnimSequence->GetSamplingFrameRate().AsDecimal();
			const double GeomCacheFrameRate = static_cast<double>(NumGeomCacheFrames - 1) / GeomCacheDuration;
			if (FMath::Abs(GeomCacheFrameRate - AnimSequenceFrameRate) > KINDA_SMALL_NUMBER)
			{
				FNumberFormattingOptions Options;
				Options.SetUseGrouping(false);
				Options.SetMaximumFractionalDigits(2);

				Result = FText::Format(
					LOCTEXT("FPSMismatch", "{0}{1}Anim sequence frame rate of {2} doesn't match the geometry cache's frame rate of {3}."),
					Result,
					!Result.IsEmpty() ? FText::FromString("\n\n") : FText::GetEmpty(),
					FText::AsNumber(AnimSequenceFrameRate, &Options),
					FText::AsNumber(GeomCacheFrameRate, &Options));
			}
		}

		return Result;
	}
#endif	// #if WITH_EDITORONLY_DATA
}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
