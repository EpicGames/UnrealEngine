// Copyright Epic Games, Inc. All Rights Reserved.
#include "MLDeformerGeomCacheModel.h"
#include "MLDeformerGeomCacheVizSettings.h"
#include "MLDeformerGeomCacheHelpers.h"
#include "MLDeformerComponent.h"
#include "MLDeformerAsset.h"
#include "MLDeformerObjectVersion.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/ICookInfo.h"
#include "GeometryCache.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MLDeformerGeomCacheModel)

#define LOCTEXT_NAMESPACE "MLDeformerGeomCacheModel"


void UMLDeformerGeomCacheModel::PostLoad()
{
	Super::PostLoad();

	#if WITH_EDITORONLY_DATA
		// Handle backward compatibility by converting the asset that just has a single asset, into an entry
		// inside the multiple training input animation array.
		if (!AnimSequence_DEPRECATED.IsNull() || !GeometryCache_DEPRECATED.IsNull())
		{
			check(TrainingInputAnims.IsEmpty());	// This should only happen when we don't use the training input anims array yet.

			// Add an item to the training input anims list.
			TrainingInputAnims.AddDefaulted();
			FMLDeformerGeomCacheTrainingInputAnim& AnimEntry = TrainingInputAnims.Last();
			AnimEntry.SetGeometryCache(GeometryCache_DEPRECATED);
			AnimEntry.SetAnimSequence(AnimSequence_DEPRECATED);

			// Set the old unused properties to nullptr, as we don't want to use those anymore.
			GeometryCache_DEPRECATED = nullptr;
			AnimSequence_DEPRECATED = nullptr;
		}
	#endif
}

void UMLDeformerGeomCacheModel::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	#if WITH_EDITORONLY_DATA
		FString AnimInputString;
		for (const FMLDeformerGeomCacheTrainingInputAnim& Anim : TrainingInputAnims)
		{
			AnimInputString += Anim.GetAnimSequenceSoftObjectPtr().ToSoftObjectPath().ToString();
			AnimInputString += TEXT("\n");
			AnimInputString += Anim.GetGeometryCacheSoftObjectPtr().ToSoftObjectPath().ToString();
			AnimInputString += TEXT("\n");
		}
		if (AnimInputString.IsEmpty())
		{
			AnimInputString = TEXT("None");
		}
		Context.AddTag(FAssetRegistryTag("MLDeformer.TrainingAnims", AnimInputString, FAssetRegistryTag::TT_Alphabetical));
		Context.AddTag(FAssetRegistryTag("MLDeformer.NumTrainingAnims", FString::FromInt(TrainingInputAnims.Num()), FAssetRegistryTag::TT_Numerical));
	#endif
}

#if WITH_EDITOR
void UMLDeformerGeomCacheModel::UpdateNumTargetMeshVertices()
{
	FCookLoadScope CookLoadScope(ECookLoadType::EditorOnly);
	for (FMLDeformerGeomCacheTrainingInputAnim& Anim : TrainingInputAnims)
	{
		UGeometryCache* GeomCache = Anim.GetGeometryCache();
		if (GeomCache && Anim.IsEnabled())
		{
			SetNumTargetMeshVerts(UE::MLDeformer::ExtractNumImportedGeomCacheVertices(GeomCache));
			return;
		}
	}

	SetNumTargetMeshVerts(0);
}

UMLDeformerGeomCacheVizSettings* UMLDeformerGeomCacheModel::GetGeomCacheVizSettings() const
{
	return Cast<UMLDeformerGeomCacheVizSettings>(GetVizSettings());
}
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
bool UMLDeformerGeomCacheModel::HasTrainingGroundTruth() const
{
	for (const FMLDeformerGeomCacheTrainingInputAnim& Anim : TrainingInputAnims)
	{
		if (Anim.GetGeometryCache())
		{
			return true;
		}
	}
	return false;
}

void UMLDeformerGeomCacheModel::SampleGroundTruthPositionsAtFrame(int32 FrameIndex, TArray<FVector3f>& OutPositions)
{
	const UMLDeformerGeomCacheVizSettings* GeomCacheVizSettings = GetGeomCacheVizSettings();
	check(GeomCacheVizSettings);

	UGeometryCache* GeomCache = GeomCacheVizSettings->GetTestGroundTruth();
	if (GeomCache == nullptr)
	{
		OutPositions.Reset();
		return;
	}

	if (MeshMappings.IsEmpty())
	{
		TArray<FString> FailedImportedMeshNames;
		TArray<FString> VertexMisMatchNames;
		UE::MLDeformer::GenerateGeomCacheMeshMappings(GetSkeletalMesh(), GeomCache, MeshMappings, FailedImportedMeshNames, VertexMisMatchNames, /**bSuppressLog*/true);
	}

	UE::MLDeformer::SampleGeomCachePositionsAtFrame(
		0,
		FrameIndex,
		MeshMappings,
		GetSkeletalMesh(),
		GeomCache,
		GetAlignmentTransform(),
		OutPositions);
}

void UMLDeformerGeomCacheModel::SampleGroundTruthPositions(float SampleTime, TArray<FVector3f>& OutPositions)
{
	OutPositions.Reset();
}

#endif

#undef LOCTEXT_NAMESPACE
