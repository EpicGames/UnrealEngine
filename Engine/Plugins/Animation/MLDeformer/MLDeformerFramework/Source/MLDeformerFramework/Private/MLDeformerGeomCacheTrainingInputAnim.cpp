// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerGeomCacheTrainingInputAnim.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MLDeformerGeomCacheTrainingInputAnim)

#if WITH_EDITORONLY_DATA
#include "GeometryCache.h"
#include "Animation/AnimSequence.h"

int32 FMLDeformerGeomCacheTrainingInputAnim::GetNumFramesToSample() const
{
	const UGeometryCache* GeomCache = GetGeometryCache();
	const UAnimSequence* AnimSeq = GetAnimSequence();
	if (!IsEnabled() || !AnimSeq || !GeomCache)
	{
		return 0;
	}
	
	const int32 NumAnimFrames = GeomCache->GetEndFrame() - GeomCache->GetStartFrame() + 1;
	int32 NumFrames = 0;
	if (!GetUseCustomRange())
	{
		NumFrames = NumAnimFrames;
	}
	else
	{
		int32 SafeStartFrame = FMath::Min<int32>(GetStartFrame(), GetEndFrame());
		int32 SafeEndFrame = FMath::Max<int32>(GetStartFrame(), GetEndFrame());

		if (SafeStartFrame >= GeomCache->GetEndFrame() - GeomCache->GetStartFrame())
		{
			return 0;
		}

		if (SafeEndFrame >= GeomCache->GetEndFrame() - GeomCache->GetStartFrame())
		{
			SafeEndFrame = GeomCache->GetEndFrame() - GeomCache->GetStartFrame();
		}

		const int32 NumManualFrames = SafeEndFrame - SafeStartFrame + 1;
		NumFrames = FMath::Min<int32>(NumAnimFrames, NumManualFrames);
	}

	return NumFrames;
}

bool FMLDeformerGeomCacheTrainingInputAnim::IsValid() const
{
	return (FMLDeformerTrainingInputAnim::IsValid() && GetGeometryCache() != nullptr);
}

int32 FMLDeformerGeomCacheTrainingInputAnim::ExtractNumAnimFrames() const
{ 
	const UAnimSequence* Anim = GetAnimSequence();
	const UGeometryCache* GeomCache = GetGeometryCache();
	if (!Anim || !GeomCache)
	{
		return 0;
	}

	return GeomCache->GetEndFrame() - GeomCache->GetStartFrame() + 1;
}

#endif // #if WITH_EDITORONLY_DATA
