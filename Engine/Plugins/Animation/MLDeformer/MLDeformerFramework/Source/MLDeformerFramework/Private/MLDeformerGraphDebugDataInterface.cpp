// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerGraphDebugDataInterface.h"
#include "MLDeformerModelInstance.h"
#include "SkeletalMeshDeformerHelpers.h"
#include "MLDeformerAsset.h"
#include "MLDeformerComponent.h"
#include "MLDeformerVizSettings.h"
#include "Components/SkeletalMeshComponent.h"
#include "ComputeFramework/ComputeKernelPermutationSet.h"
#include "ComputeFramework/ShaderParamTypeDefinition.h"
#include "OptimusDataDomain.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "RenderGraphResources.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMetadataBuilder.h"
#include "SkeletalRenderPublic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MLDeformerGraphDebugDataInterface)

TArray<FOptimusCDIPinDefinition> UMLDeformerGraphDebugDataInterface::GetPinDefinitions() const
{
	TArray<FOptimusCDIPinDefinition> Defs;
	Defs.Add({ "HeatMapMode", "ReadHeatMapMode" });
	Defs.Add({ "HeatMapMax", "ReadHeatMapMax" });
	Defs.Add({ "GroundTruthLerp", "ReadGroundTruthLerp" });
	Defs.Add({ "PositionGroundTruth", "ReadPositionGroundTruth", Optimus::DomainName::Vertex, "ReadNumVertices" });
	return Defs;
}

TSubclassOf<UActorComponent> UMLDeformerGraphDebugDataInterface::GetRequiredComponentClass() const
{
	return UMLDeformerComponent::StaticClass();
}

void UMLDeformerGraphDebugDataInterface::GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const
{
	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadNumVertices"))
		.AddReturnType(EShaderFundamentalType::Uint);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadHeatMapMode"))
		.AddReturnType(EShaderFundamentalType::Int);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadHeatMapMax"))
		.AddReturnType(EShaderFundamentalType::Float);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadGroundTruthLerp"))
		.AddReturnType(EShaderFundamentalType::Float);

	OutFunctions.AddDefaulted_GetRef()
		.SetName(TEXT("ReadPositionGroundTruth"))
		.AddReturnType(EShaderFundamentalType::Float, 3)
		.AddParam(EShaderFundamentalType::Uint);
}

BEGIN_SHADER_PARAMETER_STRUCT(FMLDeformerGraphDebugDataInterfaceParameters, )
	SHADER_PARAMETER(uint32, NumVertices)
	SHADER_PARAMETER(int32, HeatMapMode)
	SHADER_PARAMETER(float, HeatMapMax)
	SHADER_PARAMETER(float, GroundTruthLerp)
	SHADER_PARAMETER(uint32, GroundTruthBufferSize)
	SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<float>, PositionGroundTruthBuffer)
	SHADER_PARAMETER_SRV(Buffer<uint>, VertexMapBuffer)
END_SHADER_PARAMETER_STRUCT()

FString UMLDeformerGraphDebugDataInterface::GetDisplayName() const
{
	return TEXT("MLD Model Debug");
}

void UMLDeformerGraphDebugDataInterface::GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& InOutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const
{
	InOutBuilder.AddNestedStruct<FMLDeformerGraphDebugDataInterfaceParameters>(UID);
}

TCHAR const* UMLDeformerGraphDebugDataInterface::TemplateFilePath = TEXT("/Plugin/MLDeformerFramework/Private/MLDeformerModelHeatMap.ush");

TCHAR const* UMLDeformerGraphDebugDataInterface::GetShaderVirtualPath() const
{
	return TemplateFilePath;
}

void UMLDeformerGraphDebugDataInterface::GetShaderHash(FString& InOutKey) const
{
	GetShaderFileHash(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5).AppendString(InOutKey);
}

void UMLDeformerGraphDebugDataInterface::GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const
{
	TMap<FString, FStringFormatArg> TemplateArgs =
	{
		{ TEXT("DataInterfaceName"), InDataInterfaceName },
	};

	FString TemplateFile;
	LoadShaderSourceFile(TemplateFilePath, EShaderPlatform::SP_PCD3D_SM5, &TemplateFile, nullptr);
	OutHLSL += FString::Format(*TemplateFile, TemplateArgs);
}

UComputeDataProvider* UMLDeformerGraphDebugDataInterface::CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const
{
	UMLDeformerGraphDebugDataProvider* Provider = NewObject<UMLDeformerGraphDebugDataProvider>();
	Provider->DeformerComponent = Cast<UMLDeformerComponent>(InBinding);
	if (Provider->DeformerComponent)
	{
		Provider->DeformerAsset = Provider->DeformerComponent->GetDeformerAsset();
	}

	#if WITH_EDITORONLY_DATA
		if (Provider->DeformerAsset != nullptr)
		{
			Provider->Init();
		}
	#endif

	return Provider;
}

FComputeDataProviderRenderProxy* UMLDeformerGraphDebugDataProvider::GetRenderProxy()
{
#if WITH_EDITORONLY_DATA
	if (DeformerComponent && DeformerAsset && DeformerComponent->GetModelInstance() && DeformerComponent->GetModelInstance()->IsValidForDataProvider())
	{
		UE::MLDeformer::FMLDeformerGraphDebugDataProviderProxy* Proxy = new UE::MLDeformer::FMLDeformerGraphDebugDataProviderProxy(DeformerComponent, DeformerAsset, this);
		UMLDeformerModel* Model = DeformerAsset->GetModel();
		if (Model->GetVizSettings()->GetHeatMapMode() == EMLDeformerHeatMapMode::GroundTruth)
		{
			const int32 GroundTruthFrame = DeformerComponent->GetModelInstance()->GetGroundTruthFrameIndex();
			Model->SampleGroundTruthPositionsAtFrame(GroundTruthFrame, Proxy->GetGroundTruthPositions());
		}
		Proxy->HandleZeroGroundTruthPositions();
		return Proxy;
	}
#endif

	// Return default invalid proxy.
	return new FComputeDataProviderRenderProxy();
}

#if WITH_EDITORONLY_DATA
namespace UE::MLDeformer
{
	FMLDeformerGraphDebugDataProviderProxy::FMLDeformerGraphDebugDataProviderProxy(UMLDeformerComponent* DeformerComponent, UMLDeformerAsset* DeformerAsset, UMLDeformerGraphDebugDataProvider* InProvider)
		: FComputeDataProviderRenderProxy()
	{
		Provider = InProvider;

		if (DeformerComponent && DeformerAsset)
		{
			const UMLDeformerModel* Model = DeformerAsset->GetModel();	
			const UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
			const UMLDeformerModelInstance* ModelInstance = DeformerComponent->GetModelInstance();

			USkinnedMeshComponent* SkinnedMeshComp = (ModelInstance) ? ModelInstance->GetSkeletalMeshComponent() : nullptr;
			if (SkinnedMeshComp && SkinnedMeshComp->IsRegistered())
			{
				SceneInterface = SkinnedMeshComp->GetScene();
				ComponentId = SkinnedMeshComp->GetPrimitiveSceneId();
			}

			VertexMapBufferSRV = Model->GetVertexMapBuffer().ShaderResourceViewRHI;
			HeatMapMode = static_cast<int32>(VizSettings->GetHeatMapMode());
			HeatMapMax = 1.0f / FMath::Max(VizSettings->GetHeatMapMax(), 0.00001f);
			GroundTruthLerp = (ModelInstance && ModelInstance->GetSkeletalMeshComponent() && ModelInstance->GetSkeletalMeshComponent()->GetPredictedLODLevel() == 0) ? VizSettings->GetGroundTruthLerp() : 0.0f;
		}
	}

	void FMLDeformerGraphDebugDataProviderProxy::HandleZeroGroundTruthPositions()
	{
		if (GroundTruthPositions.Num() == 0)
		{	
			// We didn't get valid ground truth vertices.
			// Make non empty array for later buffer generation.
			GroundTruthPositions.Add(FVector3f::ZeroVector);

			// Silently disable relevant debug things.
			if (HeatMapMode == static_cast<int32>(EMLDeformerHeatMapMode::GroundTruth))
			{
				HeatMapMode = -1;
				HeatMapMax = 0.0f;
				GroundTruthLerp = 0.0f;
			}
		}
	}

	bool FMLDeformerGraphDebugDataProviderProxy::IsValid(FValidationData const& InValidationData) const
	{
		if (InValidationData.ParameterStructSize != sizeof(FMLDeformerGraphDebugDataInterfaceParameters))
		{
			return false;
		}

		FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
		if (!SkeletalMeshObject || VertexMapBufferSRV == nullptr)
		{
			return false;
		}

		// Only support showing heatmaps in groundtruth mode on LOD 0.
		const FSkeletalMeshRenderData& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
		if (SkeletalMeshRenderData.GetPendingFirstLODIdx(0) != 0 && HeatMapMode == static_cast<int32>(EMLDeformerHeatMapMode::GroundTruth))
		{
			return false;
		}

		return true;
	}

	void FMLDeformerGraphDebugDataProviderProxy::AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData)
	{
		GroundTruthBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateStructuredDesc(sizeof(float), 3 * GroundTruthPositions.Num()), TEXT("MLDeformer.GroundTruthPositions"));
		GroundTruthBufferSRV = GraphBuilder.CreateSRV(GroundTruthBuffer);
		GraphBuilder.QueueBufferUpload(GroundTruthBuffer, GroundTruthPositions.GetData(), sizeof(FVector3f) * GroundTruthPositions.Num(), ERDGInitialDataFlags::None);
	}

	void FMLDeformerGraphDebugDataProviderProxy::GatherDispatchData(FDispatchData const& InDispatchData)
	{
		FSkeletalMeshObject* SkeletalMeshObject = FSkeletalMeshDeformerHelpers::GetSkeletalMeshObject(SceneInterface, ComponentId);
		const FSkeletalMeshRenderData& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
		const FSkeletalMeshLODRenderData* LodRenderData = SkeletalMeshRenderData.GetPendingFirstLOD(0);
		const TStridedView<FMLDeformerGraphDebugDataInterfaceParameters> ParameterArray = MakeStridedParameterView<FMLDeformerGraphDebugDataInterfaceParameters>(InDispatchData);
		for (int32 InvocationIndex = 0; InvocationIndex < ParameterArray.Num(); ++InvocationIndex)
		{
			FMLDeformerGraphDebugDataInterfaceParameters& Parameters = ParameterArray[InvocationIndex];
			Parameters.NumVertices = LodRenderData->GetNumVertices();
			Parameters.HeatMapMode = HeatMapMode;
			Parameters.HeatMapMax = HeatMapMax;
			Parameters.GroundTruthLerp = GroundTruthLerp;
			Parameters.GroundTruthBufferSize = GroundTruthPositions.Num();
			Parameters.PositionGroundTruthBuffer = GroundTruthBufferSRV;
			Parameters.VertexMapBuffer = VertexMapBufferSRV;
		}
	}
}	// namespace UE::MLDeformer
#endif // WITH_EDITORONLY_DATA

