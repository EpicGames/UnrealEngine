// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerInputInfo.h"
#include "MLDeformerAsset.h"
#include "MLDeformerModel.h"
#include "MLDeformerObjectVersion.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/Skeleton.h"
#include "ReferenceSkeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Components/SkeletalMeshComponent.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MLDeformerInputInfo)
void UMLDeformerInputInfo::Reset()
{
	BoneNames.Empty();
	CurveNames.Empty();
	NumBaseMeshVertices = 0;
	NumTargetMeshVertices = 0;
	SkeletalMesh = nullptr;
}

void UMLDeformerInputInfo::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);
	Context.AddTag(FAssetRegistryTag("MLDeformer.Trained.SkeletalMesh", SkeletalMesh.IsValid() ? SkeletalMesh.ToString() : TEXT("None"), FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.Trained.NumBaseMeshVertices", FString::FromInt(NumBaseMeshVertices), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.Trained.NumTargetMeshVertices", FString::FromInt(NumTargetMeshVertices), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.Trained.NumBones", FString::FromInt(GetNumBones()), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.Trained.NumCurves", FString::FromInt(GetNumCurves()), FAssetRegistryTag::TT_Numerical));
}

void UMLDeformerInputInfo::OnPostLoad()
{
}

void UMLDeformerInputInfo::CopyMembersFrom(UMLDeformerInputInfo* Other)
{
	BoneNames = Other->BoneNames;
	CurveNames = Other->CurveNames;
	NumBaseMeshVertices = Other->NumBaseMeshVertices;
	NumTargetMeshVertices = Other->NumTargetMeshVertices;
	SkeletalMesh = Other->SkeletalMesh;
}

bool UMLDeformerInputInfo::IsEmpty() const
{ 
	return (BoneNames.IsEmpty() && CurveNames.IsEmpty());
}

int32 UMLDeformerInputInfo::GetNumBones() const
{ 
	return BoneNames.Num();
}

FName UMLDeformerInputInfo::GetBoneName(int32 Index) const
{ 
	return BoneNames[Index];
}

int32 UMLDeformerInputInfo::GetNumCurves() const
{ 
	return CurveNames.Num();
}

TArray<FName>& UMLDeformerInputInfo::GetBoneNames()
{ 
	return BoneNames;
}

const TArray<FName>& UMLDeformerInputInfo::GetBoneNames() const
{ 
	return BoneNames;
}

TArray<FName>& UMLDeformerInputInfo::GetCurveNames()
{ 
	return CurveNames;
}

const TArray<FName>& UMLDeformerInputInfo::GetCurveNames() const
{ 
	return CurveNames;
}

FName UMLDeformerInputInfo::GetCurveName(int32 Index) const
{ 
	return CurveNames[Index];
}

int32 UMLDeformerInputInfo::GetNumBaseMeshVertices() const
{ 
	return NumBaseMeshVertices;
}

void UMLDeformerInputInfo::SetNumBaseVertices(int32 NumVerts)
{ 
	NumBaseMeshVertices = NumVerts;
}

int32 UMLDeformerInputInfo::GetNumTargetMeshVertices() const
{
	return NumTargetMeshVertices;
}

void UMLDeformerInputInfo::SetNumTargetVertices(int32 NumVerts)
{
	NumTargetMeshVertices = NumVerts;
}

bool UMLDeformerInputInfo::IsCompatible(const USkeletalMeshComponent* InSkeletalMeshComponent) const
{
	USkeletalMesh* ComponentSkeletalMesh = Cast<USkeletalMesh>(InSkeletalMeshComponent->GetSkinnedAsset());
	if (SkeletalMesh.IsNull())
	{
		return true;
	}

	if (ComponentSkeletalMesh == nullptr)
	{
		return false;
	}

	if ((SkeletalMesh.IsValid() && !SkeletalMesh.IsNull()) && FSoftObjectPath(ComponentSkeletalMesh) != SkeletalMesh)
	{
		return false;
	}

	// Verify that all required bones are there.
	const FReferenceSkeleton& RefSkeleton = ComponentSkeletalMesh->GetRefSkeleton();
	for (const FName BoneName : BoneNames)
	{
		const int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);
		if (BoneIndex == INDEX_NONE) // We're missing the required bone. The network needs to input the transform for this bone.
		{
			return false;
		}
	}

	// Verify that all required curves are there.
	TArray<FName> CurveList;
	BuildCurveList(ComponentSkeletalMesh, CurveList);
	for (const FName CurveName : CurveNames)
	{
		if (!CurveList.Contains(CurveName))
		{
			return false;
		}
	}

	return true;
}

bool UMLDeformerInputInfo::IsCompatible(USkeletalMesh* InSkeletalMesh) const
{
	if (SkeletalMesh.IsNull())
	{
		return true;
	}

	if (InSkeletalMesh == nullptr)
	{
		return false;
	}

	if ((SkeletalMesh.IsValid() && !SkeletalMesh.IsNull()) && FSoftObjectPath(InSkeletalMesh) != SkeletalMesh)
	{
		return false;
	}

	return true;
}

void UMLDeformerInputInfo::BuildCurveList(USkeletalMesh* InSkeletalMesh, TArray<FName>& OutCurveNames)
{
	TSet<FName> UniqueNames;
	if (InSkeletalMesh)
	{
		// Collect all curves on the skeleton.
		const USkeleton* Skeleton = InSkeletalMesh->GetSkeleton();
		if (Skeleton)
		{
			TArray<FName> SkeletonCurveNames;
			Skeleton->GetCurveMetaDataNames(SkeletonCurveNames);
			UniqueNames.Append(SkeletonCurveNames);
		}

		// Collect all curves on the skeletal mesh.
		TArray<FName> MeshCurveNames;
		const UAnimCurveMetaData* SkelMeshCurveData = InSkeletalMesh->GetAssetUserData<UAnimCurveMetaData>();
		if (SkelMeshCurveData)
		{
			TArray<FName> SkelMeshCurveNames;
			SkelMeshCurveData->GetCurveMetaDataNames(SkelMeshCurveNames);
			UniqueNames.Append(SkelMeshCurveNames);
		}
	}
	OutCurveNames = UniqueNames.Array();
}

FString UMLDeformerInputInfo::GenerateCompatibilityErrorString(const USkeletalMeshComponent* InSkeletalMeshComponent) const
{
	USkeletalMesh* ComponentSkeletalMesh = Cast<USkeletalMesh>(InSkeletalMeshComponent->GetSkinnedAsset());
	if (ComponentSkeletalMesh == nullptr)
	{
		return FString();
	}

	// Verify that all required bones are there.
	const FReferenceSkeleton& RefSkeleton = ComponentSkeletalMesh->GetRefSkeleton();
	FString ErrorString;
	for (const FName BoneName : BoneNames)
	{
		const int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);
		if (BoneIndex == INDEX_NONE) // We're missing the required bone. The network needs to input the transform for this bone.
		{
			ErrorString += FString::Format(TEXT("Required bone '{0}' is missing.\n"), {*BoneName.ToString()});
		}
	}

	// Verify that all required curves are there.
	TArray<FName> CurveList;
	BuildCurveList(ComponentSkeletalMesh, CurveList);
	for (const FName CurveName : CurveNames)
	{
		if (!CurveList.Contains(CurveName))
		{
			ErrorString += FString::Format(TEXT("Required curve '{0}' is missing.\n"), { *CurveName.ToString() });
		}
	}
	
	// Check vertex count.
#if WITH_EDITORONLY_DATA
	if ((NumBaseMeshVertices > 0 && NumTargetMeshVertices > 0) &&
		NumBaseMeshVertices != ComponentSkeletalMesh->GetNumImportedVertices())
	{
		ErrorString += FString::Format(TEXT("The number of vertices that the network was trained on ({0} verts) doesn't match the skeletal mesh '{1}' ({2} verts)..\n"), 
			{
				NumBaseMeshVertices, 
				ComponentSkeletalMesh->GetName(),
				ComponentSkeletalMesh->GetNumImportedVertices(),
			} );
	}
#endif

	if (SkeletalMesh.IsValid() && FSoftObjectPath(ComponentSkeletalMesh) != SkeletalMesh)
	{
		ErrorString += FString::Format(TEXT("Model was trained on Skeletal Mesh '{0}', which is not the same as '{1}' it tries to apply on.\n"), {*SkeletalMesh.ToString(), *FSoftObjectPath(ComponentSkeletalMesh).ToString()});
	}

	return ErrorString;	
}

FString UMLDeformerInputInfo::GenerateCompatibilityErrorString(USkeletalMesh* InSkeletalMesh) const
{
	if (InSkeletalMesh == nullptr)
	{
		return FString();
	}

	// Verify that all required bones are there.
	const FReferenceSkeleton& RefSkeleton = InSkeletalMesh->GetRefSkeleton();
	FString ErrorString;
	for (const FName BoneName : BoneNames)
	{
		const int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);
		if (BoneIndex == INDEX_NONE) // We're missing the required bone. The network needs to input the transform for this bone.
		{
			ErrorString += FString::Format(TEXT("Required bone '{0}' is missing.\n"), {*BoneName.ToString()});
		}
	}

	// Verify that all required curves are there.
	TArray<FName> CurveList;
	BuildCurveList(InSkeletalMesh, CurveList);
	for (const FName CurveName : CurveNames)
	{
		if (!CurveList.Contains(CurveName))
		{
			ErrorString += FString::Format(TEXT("Required curve '{0}' is missing.\n"), { *CurveName.ToString() });
		}
	}
	
	// Check vertex count.
#if WITH_EDITORONLY_DATA
	if ((NumBaseMeshVertices > 0 && NumTargetMeshVertices > 0) &&
		NumBaseMeshVertices != InSkeletalMesh->GetNumImportedVertices())
	{
		ErrorString += FString::Format(TEXT("The number of vertices that the network was trained on ({0} verts) doesn't match the skeletal mesh '{1}' ({2} verts)..\n"), 
			{
				NumBaseMeshVertices, 
				InSkeletalMesh->GetName(),
				InSkeletalMesh->GetNumImportedVertices(),
			} );
	}
#endif

	if (SkeletalMesh.IsValid() && FSoftObjectPath(InSkeletalMesh) != SkeletalMesh)
	{
		ErrorString += FString::Format(TEXT("Model was trained on Skeletal Mesh '{0}', which is not the same as '{1}' it tries to apply on.\n"), {*SkeletalMesh.ToString(), *FSoftObjectPath(InSkeletalMesh).ToString()});
	}

	return ErrorString;	
}

void UMLDeformerInputInfo::ExtractCurveValues(USkeletalMeshComponent* SkelMeshComponent, TArray<float>& OutValues, int32 NumFloatsPerCurve) const
{
	const int32 NumCurves = CurveNames.Num();
	const int32 NumCurveFloats = NumCurves * NumFloatsPerCurve;
	OutValues.Reset(NumCurveFloats);
	OutValues.AddZeroed(NumCurveFloats);
	for (int32 Index = 0; Index < NumCurves; ++Index)
	{
		const FName CurveName = CurveNames[Index];
		float CurveValue;
		SkelMeshComponent->GetCurveValue(CurveName, 0.0f, CurveValue);
		OutValues[Index * NumFloatsPerCurve] = CurveValue;
	}
}

void UMLDeformerInputInfo::RotationToTwoVectorsAsSixFloats(TArrayView<FTransform> Transforms, float* OutputBuffer)
{
	for (int32 Index = 0; Index < Transforms.Num(); ++Index)
	{
		// Precalculate some things, essentially like a quaternion to matrix conversion.
		const FQuat Rotation = Transforms[Index].GetRotation();
		const float X = Rotation.X;
		const float Y = Rotation.Y;
		const float Z = Rotation.Z;
		const float W = Rotation.W;
		const float x2 = X + X;    const float y2 = Y + Y;    const float z2 = Z + Z;
		const float xx = X * x2;   const float xy = X * y2;   const float xz = X * z2;
		const float yy = Y * y2;   const float yz = Y * z2;   const float zz = Z * z2;
		const float wx = W * x2;   const float wy = W * y2;   const float wz = W * z2;

		const int32 OutputOffset = Index * 6;

		// X Column of the matrix.
		OutputBuffer[OutputOffset + 0] = 1.0f - (yy + zz);
		OutputBuffer[OutputOffset + 1] = xy - wz;
		OutputBuffer[OutputOffset + 2] = xz + wy;

		// Y Column of the matrix.
		OutputBuffer[OutputOffset + 3] = xy + wz;
		OutputBuffer[OutputOffset + 4] = 1.0f - (xx + zz);
		OutputBuffer[OutputOffset + 5] = yz - wx;
	}
}

void UMLDeformerInputInfo::RotationToTwoVectorsAsSixFloats(const FQuat& Rotation, float* SixFloatsOutputBuffer)
{
	// Precalculate some things, essentially like a quaternion to matrix conversion.
	const float X = Rotation.X;
	const float Y = Rotation.Y;
	const float Z = Rotation.Z;
	const float W = Rotation.W;
	const float x2 = X + X;    const float y2 = Y + Y;    const float z2 = Z + Z;
	const float xx = X * x2;   const float xy = X * y2;   const float xz = X * z2;
	const float yy = Y * y2;   const float yz = Y * z2;   const float zz = Z * z2;
	const float wx = W * x2;   const float wy = W * y2;   const float wz = W * z2;

	// X Column of the matrix.
	SixFloatsOutputBuffer[0] = 1.0f - (yy + zz);
	SixFloatsOutputBuffer[1] = xy - wz;
	SixFloatsOutputBuffer[2] = xz + wy;

	// Y Column of the matrix.
	SixFloatsOutputBuffer[3] = xy + wz;
	SixFloatsOutputBuffer[4] = 1.0f - (xx + zz);
	SixFloatsOutputBuffer[5] = yz - wx;
}

void UMLDeformerInputInfo::ExtractBoneRotations(USkeletalMeshComponent* SkelMeshComponent, TArray<float>& OutRotations) const
{
	const TArray<FTransform>& BoneTransforms = SkelMeshComponent->GetBoneSpaceTransforms();
	const int32 NumBones = GetNumBones();
	const int32 NumFloats = NumBones * 6; // 2 Columns of the rotation matrix.
	OutRotations.Reset(NumFloats);
	OutRotations.AddUninitialized(NumFloats);
	for (int32 Index = 0; Index < NumBones; ++Index)
	{
		const FName BoneName = GetBoneName(Index);
		const int32 SkelMeshBoneIndex = SkelMeshComponent->GetBoneIndex(BoneName);
		const FQuat Rotation = (SkelMeshBoneIndex != INDEX_NONE) ? BoneTransforms[SkelMeshBoneIndex].GetRotation() : FQuat::Identity;
		RotationToTwoVectorsAsSixFloats(
			Rotation,
			OutRotations.GetData() + Index * 6);
	}
}

int32 UMLDeformerInputInfo::CalcNumNeuralNetInputs(int32 NumFloatsPerBone, int32 NumFloatsPerCurve) const
{
	return 
		BoneNames.Num() * NumFloatsPerBone +
		CurveNames.Num() * NumFloatsPerCurve;
}

const FSoftObjectPath& UMLDeformerInputInfo::GetSkeletalMesh() const
{
	return SkeletalMesh;
}

void UMLDeformerInputInfo::SetSkeletalMesh(USkeletalMesh* InSkeletalMesh)
{
	SkeletalMesh = InSkeletalMesh;
}
