// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerModel.h"
#include "MLDeformerInputInfo.h"
#include "MLDeformerModelInstance.h"
#include "MLDeformerVizSettings.h"
#include "MLDeformerAsset.h"
#include "MLDeformerComponent.h"
#include "MLDeformerObjectVersion.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Animation/AnimSequence.h"
#include "RHICommandList.h"
#include "UObject/UObjectGlobals.h"
#include "RHICommandList.h"
#include "AssetRegistry/AssetData.h"
#include "Components/SkeletalMeshComponent.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "Misc/TransactionObjectEvent.h"
#include "UObject/ObjectSaveContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MLDeformerModel)

namespace UE::MLDeformer
{
	void FVertexMapBuffer::InitRHI(FRHICommandListBase& RHICmdList)
	{
		if (VertexMap.Num() > 0)
		{
			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateVertex<uint32>(TEXT("UMLDeformerModel::FVertexMapBuffer"), VertexMap.Num())
				.AddUsage(EBufferUsageFlags::Static | EBufferUsageFlags::ShaderResource)
				.DetermineInitialState()
				.SetInitActionInitializer();

			TRHIBufferInitializer<uint32> InitialData = RHICmdList.CreateBufferInitializer(CreateDesc);

			for (int32 Index = 0; Index < VertexMap.Num(); ++Index)
			{
				InitialData[Index] = static_cast<uint32>(VertexMap[Index]);
			}
			VertexMap.Empty();

			VertexBufferRHI = InitialData.Finalize();
			ShaderResourceViewRHI = RHICmdList.CreateShaderResourceView(
				VertexBufferRHI, 
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R32_UINT));
		}
		else
		{
			VertexBufferRHI = nullptr;
			ShaderResourceViewRHI = nullptr;
		}
	}
}	// namespace UE::MLDeformer

FString UMLDeformerModel::GetDisplayName() const
{ 
	return GetClass()->GetFName().ToString();
}

UMLDeformerInputInfo* UMLDeformerModel::CreateInputInfo()
{ 
	return NewObject<UMLDeformerInputInfo>(this, NAME_None, RF_Transactional);
}

UMLDeformerModelInstance* UMLDeformerModel::CreateModelInstance(UMLDeformerComponent* Component)
{ 
	return NewObject<UMLDeformerModelInstance>(Component);
}

void UMLDeformerModel::Init(UMLDeformerAsset* InDeformerAsset) 
{ 
	check(InDeformerAsset);
	DeformerAsset = InDeformerAsset; 
	if (InputInfo == nullptr)
	{
		InputInfo = CreateInputInfo();
	}
}

void UMLDeformerModel::Serialize(FArchive& Archive)
{
	UMLDeformerVizSettings* VizSettingsBackup = nullptr;
	bool bProcessedDataOnCook = false;
	ON_SCOPE_EXIT
	{
		if (bProcessedDataOnCook && GetRecoverStrippedDataAfterCook())
		{
			#if WITH_EDITORONLY_DATA
				VizSettings = VizSettingsBackup;
				VizSettingsBackup = nullptr;
			#endif
		}
	};

	#if WITH_EDITOR
		if (Archive.IsSaving() && Archive.IsCooking())
		{
			bProcessedDataOnCook = true;
			#if WITH_EDITORONLY_DATA
				VizSettingsBackup = VizSettings;
				AnimSequence_DEPRECATED = nullptr;
				VizSettings = nullptr;
			#endif
		}
	#endif

	Super::Serialize(Archive);
	Archive.UsingCustomVersion(UE::MLDeformer::FMLDeformerObjectVersion::GUID);
}

void UMLDeformerModel::PreSave(FObjectPreSaveContext SaveContext)
{
	Super::PreSave(SaveContext);
	
#if WITH_EDITOR
	InitVertexMap();
#endif	
}

UMLDeformerAsset* UMLDeformerModel::GetDeformerAsset() const
{ 
	return DeformerAsset.Get(); 
}

void UMLDeformerModel::PostLoad()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMLDeformerModel::PostLoad)

	Super::PostLoad();
	
	#if WITH_EDITOR
		InitVertexMap();
		UpdateCachedNumVertices();
	#endif

	InitGPUData();

	UMLDeformerAsset* MLDeformerAsset = Cast<UMLDeformerAsset>(GetOuter());
	Init(MLDeformerAsset);

	if (InputInfo)
	{
		// If we are dealing with an input info that doesn't have a skeletal mesh, then use the 
		// currently set skeletal mesh. This is for backward compatibility, from before we put the skeletal mesh inside the input info.
		if (InputInfo->GetSkeletalMesh() == nullptr)
		{
			InputInfo->SetSkeletalMesh(SkeletalMesh);
		}

		InputInfo->OnPostLoad();
	}

	#if WITH_EDITOR
		UpdateMemoryUsage();
	#endif

	SetFlags(RF_Transactional);
}

// Used for the FBoenReference, so it knows what skeleton to pick bones from.
USkeleton* UMLDeformerModel::GetSkeleton(bool& bInvalidSkeletonIsError, const IPropertyHandle* PropertyHandle)
{
	bInvalidSkeletonIsError = false;
	USkeletalMesh* SkelMesh = GetSkeletalMesh();
	return (SkelMesh != nullptr) ? SkelMesh->GetSkeleton() : nullptr;
}

void UMLDeformerModel::BeginDestroy()
{
	PostEditPropertyDelegate.RemoveAll(this);

	BeginReleaseResource(&VertexMapBuffer);
	RenderResourceDestroyFence.BeginFence();
	Super::BeginDestroy();
}

bool UMLDeformerModel::IsReadyForFinishDestroy()
{
	return Super::IsReadyForFinishDestroy() && RenderResourceDestroyFence.IsFenceComplete();
}

void UMLDeformerModel::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	Context.AddTag(FAssetRegistryTag("MLDeformer.ModelType", GetClass()->GetName(), FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.IsTrained", IsTrained() ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.NumBaseMeshVerts", FString::FromInt(NumBaseMeshVerts), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.NumTargetMeshVerts", FString::FromInt(NumTargetMeshVerts), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.SkeletalMesh", SkeletalMesh ? FAssetData(SkeletalMesh).ToSoftObjectPath().ToString() : TEXT("None"), FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.MaxNumLODs", FString::FromInt(GetMaxNumLODs()), FAssetRegistryTag::TT_Numerical));

	#if WITH_EDITORONLY_DATA
		Context.AddTag(FAssetRegistryTag("MLDeformer.NumBones", FString::FromInt(BoneIncludeList.Num()), FAssetRegistryTag::TT_Numerical));
		Context.AddTag(FAssetRegistryTag("MLDeformer.NumCurves", FString::FromInt(CurveIncludeList.Num()), FAssetRegistryTag::TT_Numerical));
		Context.AddTag(FAssetRegistryTag("MLDeformer.MaxTrainingFrames", FString::FromInt(MaxTrainingFrames), FAssetRegistryTag::TT_Numerical));
	#endif

	if (InputInfo)
	{
		InputInfo->GetAssetRegistryTags(Context);
	}
}

void UMLDeformerModel::InitGPUData()
{
	BeginReleaseResource(&VertexMapBuffer);
	VertexMapBuffer.Init(VertexMap);
	BeginInitResource(&VertexMapBuffer);
}

void UMLDeformerModel::FloatArrayToVector3Array(const TArray<float>& FloatArray, TArray<FVector3f>& OutVectorArray)
{
	check(FloatArray.Num() % 3 == 0);
	const int32 NumVerts = FloatArray.Num() / 3;
	OutVectorArray.Reset();
	OutVectorArray.SetNumUninitialized(NumVerts);
	for (int32 VertexIndex = 0; VertexIndex < NumVerts; ++VertexIndex)
	{
		const int32 FloatBufferOffset = VertexIndex * 3;
		OutVectorArray[VertexIndex] = FVector3f(FloatArray[FloatBufferOffset + 0], FloatArray[FloatBufferOffset + 1], FloatArray[FloatBufferOffset + 2]);
	}
}

bool UMLDeformerModel::IsCompatibleDebugActor(const AActor* Actor, UMLDeformerComponent** OutDebugComponent) const
{
	// Set it to nullptr first, in case we exit this method early.
	if (OutDebugComponent)
	{
		*OutDebugComponent = nullptr;
	}

	if (Actor == nullptr || !IsValid(Actor))
	{
		return false;
	}

	// Iterate over all skeletal mesh components, see if one matches our currently loaded character.
	USkeletalMesh* SkelMesh = nullptr;
	for (const UActorComponent* Component : Actor->GetComponents())
	{
		const USkeletalMeshComponent* SkelMeshComponent = Cast<USkeletalMeshComponent>(Component);
		if (!SkelMeshComponent)
		{
			continue;
		}

		if (SkelMeshComponent->GetSkeletalMeshAsset() == SkeletalMesh)
		{
			SkelMesh = SkeletalMesh;
			break;
		}
	}

	// If we haven't found a matching skeletal mesh, we can ignore this actor.
	if (!SkelMesh)
	{
		return false;
	}

	// Now check if we have an ML Deformer component on the actor uses the same ML Deformer asset.
	for (UActorComponent* Component : Actor->GetComponents())
	{
		UMLDeformerComponent* MLDeformerComponent = Cast<UMLDeformerComponent>(Component);
		if (!MLDeformerComponent)
		{
			continue;
		}

		if (MLDeformerComponent->GetDeformerAsset() == GetDeformerAsset())
		{
			if (OutDebugComponent)
			{
				*OutDebugComponent = MLDeformerComponent;
			}
			return true;
		}
	}

	return false;
}


void UMLDeformerModel::SetRecoverStrippedDataAfterCook(bool bRecover)
{ 
#if WITH_EDITORONLY_DATA
	bRecoverStrippedDataAfterCook = bRecover;
#endif
}

bool UMLDeformerModel::GetRecoverStrippedDataAfterCook() const
{ 
	#if WITH_EDITORONLY_DATA
		return bRecoverStrippedDataAfterCook;
	#else
		return true;
	#endif
}

void UMLDeformerModel::SetTrainingDevice(const FString& DeviceName)
{
	if (!TrainingDeviceList.Contains(DeviceName))
	{
		UE_LOGF(LogMLDeformer, Warning, "Training device '%ls' not found, falling back to CPU.", *DeviceName);
		TrainingDevice.Empty();
		return;
	}

	TrainingDevice = DeviceName;
}

#if WITH_EDITOR
	TArray<FName> UMLDeformerModel::GetVertexAttributeNames() const
	{
		const USkeletalMesh* SkelMesh = GetSkeletalMesh();
		if (!SkelMesh)
		{
			return {};
		}

		const int32 LODIndex = 0;
		if (!SkelMesh->HasMeshDescription(LODIndex))
		{
			return {};
		}

		const FMeshDescription* MeshDescription = SkelMesh->GetMeshDescription(LODIndex);		
		const TAttributesSet<FVertexID> AttributesSet = MeshDescription->VertexAttributes();

		TArray<FName> AttributeNames;
		AttributesSet.ForEach([&AttributeNames, &AttributesSet](const FName AttributeName, const auto AttributesRef)
		{
			const bool bIsAutoGenerated = (AttributesRef.GetFlags() & EMeshAttributeFlags::AutoGenerated) != EMeshAttributeFlags::None;
			if (!bIsAutoGenerated && AttributesSet.HasAttributeOfType<float>(AttributeName))
			{
				AttributeNames.Add(AttributeName);
			}
		});

		return AttributeNames;
	}

	void UMLDeformerModel::UpdateNumTargetMeshVertices()
	{ 
		NumTargetMeshVerts = 0;
	}

	void UMLDeformerModel::UpdateNumBaseMeshVertices()
	{
		USkeletalMesh* SkelMesh = GetSkeletalMesh();
		if (SkelMesh)
		{
			NumBaseMeshVerts = UMLDeformerModel::ExtractNumImportedSkinnedVertices(SkelMesh);
		}
	}

	void UMLDeformerModel::PostTransacted(const FTransactionObjectEvent& Event)
	{
		Super::PostTransacted(Event);
		OnPostTransacted().Broadcast(Event);
	}

	void UMLDeformerModel::PreEditUndo()
	{
		Super::PreEditUndo();
		OnPreEditUndo().Broadcast();
	}

	void UMLDeformerModel::PostEditUndo()
	{
		Super::PostEditUndo();
		OnPostEditUndo().Broadcast();
	}

	void UMLDeformerModel::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
	{
		Super::PostEditChangeProperty(PropertyChangedEvent);

		const FProperty* Property = PropertyChangedEvent.Property;
		if (Property == nullptr)
		{
			return;
		}

		OnPostEditChangeProperty().Broadcast(PropertyChangedEvent);
	}

	void UMLDeformerModel::UpdateCachedNumVertices()
	{
		UpdateNumBaseMeshVertices();
		UpdateNumTargetMeshVertices();
	}

	int32 UMLDeformerModel::ExtractNumImportedSkinnedVertices(const USkeletalMesh* SkeletalMesh)
	{
		return SkeletalMesh ? SkeletalMesh->GetNumImportedVertices() : 0;
	}

	void UMLDeformerModel::InvalidateMemUsage()
	{
		bInvalidateMemUsage = true;
	}

	uint64 UMLDeformerModel::GetCookedAssetSizeInBytes() const
	{
		return CookedAssetSizeInBytes;
	}

	uint64 UMLDeformerModel::GetMainMemUsageInBytes() const
	{
		return MemUsageInBytes;
	}

	uint64 UMLDeformerModel::GetGPUMemUsageInBytes() const
	{
		return GPUMemUsageInBytes;
	}

	bool UMLDeformerModel::IsMemUsageInvalidated() const
	{
		return bInvalidateMemUsage;
	}

	uint64 UMLDeformerModel::GetEditorAssetSizeInBytes() const
	{
		return EditorAssetSizeInBytes;
	}

	void UMLDeformerModel::UpdateMemoryUsage()
	{
		bInvalidateMemUsage = false;

		// Start everything at 0 bytes.
		MemUsageInBytes = 0;
		GPUMemUsageInBytes = 0;
		EditorAssetSizeInBytes = 0;
		CookedAssetSizeInBytes = 0;

		// Get the resource size of the ML Deformer model.
		UMLDeformerModel* Model = const_cast<UMLDeformerModel*>(this);
		EditorAssetSizeInBytes += static_cast<uint64>(Model->GetResourceSizeBytes(EResourceSizeMode::Type::EstimatedTotal));

		// Set the main mem usage and cooked sizes also to this.
		// We are going to subtract from this later, to simulate a cook, as we know which data we strip at cook time for example.
		CookedAssetSizeInBytes += EditorAssetSizeInBytes;
		MemUsageInBytes += EditorAssetSizeInBytes;

		// Add the VertexMap buffer to the GPU memory usage.
		if (VertexMapBuffer.VertexBufferRHI.IsValid())
		{
			GPUMemUsageInBytes += VertexMapBuffer.VertexBufferRHI->GetSize();
		}
	}
#endif	// #if WITH_EDITOR

#if WITH_EDITORONLY_DATA
	void UMLDeformerModel::InitVertexMap()
	{
		VertexMap.Empty();
		const USkeletalMesh* SkelMesh = GetSkeletalMesh();
		if (SkelMesh)
		{
			const FSkeletalMeshModel* SkeletalMeshModel = SkelMesh->GetImportedModel();
			if (SkeletalMeshModel && SkeletalMeshModel->LODModels.IsValidIndex(0))
			{
				VertexMap = SkeletalMeshModel->LODModels[0].MeshToImportVertexMap;
			}
		}
	}
#endif	// WITH_EDITORDATA_ONLY
