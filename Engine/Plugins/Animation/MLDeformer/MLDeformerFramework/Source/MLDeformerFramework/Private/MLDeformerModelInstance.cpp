// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerModelInstance.h"
#include "MLDeformerModel.h"
#include "MLDeformerModule.h"
#include "MLDeformerAsset.h"
#include "MLDeformerInputInfo.h"
#include "MLDeformerComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "RenderingThread.h"
#include "RenderGraphBuilder.h"
#include "DrawDebugHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MLDeformerModelInstance)

CSV_DEFINE_CATEGORY_MODULE(MLDEFORMERFRAMEWORK_API, MLDeformer, false);

#if STATS
DEFINE_STAT(STAT_MLDeformerInference);
#endif

namespace UE::MLDeformer
{
	TAutoConsoleVariable<bool> CVarMLDeformerDebugDrawInputPoses(
		TEXT("MLDeformer.DebugDrawInputPoses"),
		false,
		TEXT("Draw the input pose for each active ML Deformer.")
	);
}

namespace
{
	void CalculateBoneSpaceTransforms(const FReferenceSkeleton& RefSkel, const TArray<FTransform>& ComponentSpaceTransforms, TArray<FTransform>& OutBoneSpaceTransforms)
	{
		for (int32 Index = 0; Index < RefSkel.GetNum(); ++Index)
		{
			const int32 ParentIndex = RefSkel.GetParentIndex(Index);
			if (ParentIndex != INDEX_NONE)
			{
				OutBoneSpaceTransforms[Index] = ComponentSpaceTransforms[Index].GetRelativeTransform(ComponentSpaceTransforms[ParentIndex]);
				OutBoneSpaceTransforms[Index].NormalizeRotation();
			}
			else
			{
				OutBoneSpaceTransforms[Index] = ComponentSpaceTransforms[Index];
			}
		}
	}
}	// Anonymous namespace


void UMLDeformerModelInstance::DrawDebugInputPoses()
{
	// If we use a leader component, we have to get the transforms from that component.
	USkeletalMeshComponent* FinalSkelMeshComponent = GetFinalSkeletalMeshComponent();
	const UWorld* World = GetWorld();
	const USkinnedMeshComponent* LeaderPoseComponent = FinalSkelMeshComponent->LeaderPoseComponent.Get();
	if (LeaderPoseComponent)
	{
		const FTransform& ComponentTransform = FinalSkelMeshComponent->GetComponentTransform();
		const TArray<FTransform>& LeaderComponentSpaceTransforms = LeaderPoseComponent->GetComponentSpaceTransforms();
		if (!LeaderComponentSpaceTransforms.IsEmpty())
		{
			const USkinnedAsset* SkinnedAsset = LeaderPoseComponent->GetSkinnedAsset();
			const FReferenceSkeleton& RefSkel = SkinnedAsset->GetRefSkeleton();
			const int32 NumTrainingBones = AssetBonesToSkelMeshMappings.Num();
			for (int32 Index = 0; Index < NumTrainingBones; ++Index)
			{
				const int32 ComponentBoneIndex = AssetBonesToSkelMeshMappings[Index];
				if (LeaderComponentSpaceTransforms.IsValidIndex(ComponentBoneIndex))
				{
					const FTransform& ComponentSpaceBoneTransform = LeaderComponentSpaceTransforms[ComponentBoneIndex];
					const int32 ParentIndex = RefSkel.GetParentIndex(ComponentBoneIndex);
					const FTransform WorldTransform = ComponentSpaceBoneTransform * ComponentTransform;
					if (LeaderComponentSpaceTransforms.IsValidIndex(ParentIndex))
					{
						const FTransform WorldParentTransform = LeaderComponentSpaceTransforms[ParentIndex] * ComponentTransform;
						DrawDebugLine(World, WorldTransform.GetTranslation(), WorldParentTransform.GetTranslation(), FColor::Yellow, false, -1.0f, 255, 0.0f);
						DrawDebugPoint(World, WorldTransform.GetTranslation(), 4.0f, FColor::Yellow, false, -1.0f, 255);
						DrawDebugCoordinateSystem(World, WorldTransform.GetTranslation(), WorldTransform.GetRotation().Rotator(), 7.5f, false, -1.0f, 255, 0.5f);						
					}
					else
					{
						DrawDebugPoint(World, WorldTransform.GetTranslation(), 4.0f, FColor::Yellow, false, -1.0f, 255);
						DrawDebugCoordinateSystem(World, WorldTransform.GetTranslation(), WorldTransform.GetRotation().Rotator(), 7.5f, false, -1.0f, 255, 0.5f);						
					}
				}
			}
		}
	}
	else // No leader pose is used.
	{
		if (FinalSkelMeshComponent == SkeletalMeshComponent)
		{
			const FTransform& ComponentTransform = FinalSkelMeshComponent->GetComponentTransform();
			const TArray<FTransform>& ComponentSpaceTransforms = FinalSkelMeshComponent->GetComponentSpaceTransforms();
			if (!ComponentSpaceTransforms.IsEmpty())
			{
				const USkinnedAsset* SkinnedAsset = FinalSkelMeshComponent->GetSkinnedAsset();
				const FReferenceSkeleton& RefSkel = SkinnedAsset->GetRefSkeleton();
				const int32 NumTrainingBones = AssetBonesToSkelMeshMappings.Num();
				for (int32 Index = 0; Index < NumTrainingBones; ++Index)
				{
					const int32 ComponentBoneIndex = AssetBonesToSkelMeshMappings[Index];
					if (ComponentSpaceTransforms.IsValidIndex(ComponentBoneIndex))
					{
						const FTransform& ComponentSpaceBoneTransform = ComponentSpaceTransforms[ComponentBoneIndex];
						const FTransform WorldTransform = ComponentSpaceBoneTransform * ComponentTransform;
						const int32 ParentIndex = RefSkel.GetParentIndex(ComponentBoneIndex);
						if (ComponentSpaceTransforms.IsValidIndex(ParentIndex))
						{
							const FTransform WorldParentTransform = ComponentSpaceTransforms[ParentIndex] * ComponentTransform;
							DrawDebugLine(World, WorldTransform.GetTranslation(), WorldParentTransform.GetTranslation(), FColor::Yellow, false, -1.0f, 255, 0.0f);
							DrawDebugPoint(World, WorldTransform.GetTranslation(), 4.0f, FColor::Yellow, false, -1.0f, 255);
							DrawDebugCoordinateSystem(World, WorldTransform.GetTranslation(), WorldTransform.GetRotation().Rotator(), 7.5f, false, -1.0f, 255, 0.5f);
						}
						else
						{
							DrawDebugPoint(World, WorldTransform.GetTranslation(), 4.0f, FColor::Yellow, false, -1.0f, 255);
							DrawDebugCoordinateSystem(World, WorldTransform.GetTranslation(), WorldTransform.GetRotation().Rotator(), 7.5f, false, -1.0f, 255, 1.0f);						
						}
					}
				}
			}
		}
		else // We are debugging and grabbing transforms from another component.
		{
			USkeletalMesh* SkelMeshAsset = FinalSkelMeshComponent->GetSkeletalMeshAsset();
			check(SkelMeshAsset);

			const FTransform& ComponentTransform = SkeletalMeshComponent->GetComponentTransform();
			
			const FReferenceSkeleton& RefSkel = SkelMeshAsset->GetRefSkeleton();
			const TArray<FTransform>& ComponentSpaceTransforms = FinalSkelMeshComponent->GetComponentSpaceTransforms();
			if (!ComponentSpaceTransforms.IsEmpty())
			{
				const int32 NumTrainingBones = AssetBonesToSkelMeshMappings.Num();
				for (int32 Index = 0; Index < NumTrainingBones; ++Index)
				{
					const int32 ComponentBoneIndex = AssetBonesToSkelMeshMappings[Index];
					if (ComponentSpaceTransforms.IsValidIndex(ComponentBoneIndex))
					{
						const FTransform& ComponentSpaceBoneTransform = ComponentSpaceTransforms[ComponentBoneIndex];
						const FTransform WorldTransform = ComponentSpaceBoneTransform * ComponentTransform;
						const int32 ParentIndex = RefSkel.GetParentIndex(ComponentBoneIndex);
						if (ComponentSpaceTransforms.IsValidIndex(ParentIndex))
						{
							const FTransform WorldParentTransform = ComponentSpaceTransforms[ParentIndex] * ComponentTransform;
							DrawDebugLine(World, WorldTransform.GetTranslation(), WorldParentTransform.GetTranslation(), FColor::Yellow, false, -1.0f, 255, 0.0f);
							DrawDebugPoint(World, WorldTransform.GetTranslation(), 4.0f, FColor::Yellow, false, -1.0f, 255);
							DrawDebugCoordinateSystem(World, WorldTransform.GetTranslation(), WorldTransform.GetRotation().Rotator(), 7.5f, false, -1.0f, 255, 0.5f);
						}
						else
						{
							DrawDebugPoint(World, WorldTransform.GetTranslation(), 4.0f, FColor::Yellow, false, -1.0f, 255);
							DrawDebugCoordinateSystem(World, WorldTransform.GetTranslation(), WorldTransform.GetRotation().Rotator(), 7.5f, false, -1.0f, 255, 1.0f);						
						}
					}
				}
			}
		}
	}
}

void UMLDeformerModelInstance::BeginDestroy()
{
	RenderCommandFence.BeginFence();
	Super::BeginDestroy();
}

bool UMLDeformerModelInstance::IsReadyForFinishDestroy()
{
	// Wait for the render commands to finish, because some neural network might still be executing, using the inference handle
	// that we are about to delete.
	return Super::IsReadyForFinishDestroy() && RenderCommandFence.IsFenceComplete();
}

USkeletalMeshComponent* UMLDeformerModelInstance::GetSkeletalMeshComponent() const
{ 
	return SkeletalMeshComponent.Get();
}

UMLDeformerModel* UMLDeformerModelInstance::GetModel() const
{ 
	return Model.Get();
}

void UMLDeformerModelInstance::SetModel(UMLDeformerModel* InModel)
{ 
	Model = InModel;
}

UMLDeformerComponent* UMLDeformerModelInstance::GetMLDeformerComponent() const
{ 
	return Cast<UMLDeformerComponent>(GetOuter());
}

void UMLDeformerModelInstance::SetHasPostInitialized(bool bHasInitialized)
{ 
	bHasPostInitialized = bHasInitialized;
}

bool UMLDeformerModelInstance::HasPostInitialized() const
{ 
	return bHasPostInitialized;
}

bool UMLDeformerModelInstance::IsCompatible() const
{ 
	return bIsCompatible;
}

const FString& UMLDeformerModelInstance::GetCompatibilityErrorText() const
{ 
	return ErrorText;
}

void UMLDeformerModelInstance::Init(USkeletalMeshComponent* SkelMeshComponent)
{
	SkeletalMeshComponent = SkelMeshComponent;

	if (SkelMeshComponent == nullptr)
	{
		AssetBonesToSkelMeshMappings.Empty();

		UMLDeformerAsset* Asset = GetModel()->GetDeformerAsset();
		if (Asset && Asset->GetModel()->GetInputInfo())
		{
			const FSoftObjectPath& SkelMeshPath = Asset->GetModel()->GetInputInfo()->GetSkeletalMesh();
			if (SkelMeshPath.IsValid())
			{
				UE_LOGF(LogMLDeformer, Error, "ML Deformer '%ls' is trained on skeletal mesh '%ls', which cannot be found on actor!", *Asset->GetName(), *SkelMeshPath.ToString());
			}
		}

		UpdateCompatibilityStatus();
		return;
	}

	if (Model->DoesSupportBones())
	{
		USkeletalMesh* SkelMesh = SkelMeshComponent->GetSkeletalMeshAsset();
		if (SkelMesh)
		{
			// Init the bone mapping table.
			const UMLDeformerInputInfo* InputInfo = Model->GetInputInfo();
			const int32 NumAssetBones = InputInfo->GetNumBones();
			AssetBonesToSkelMeshMappings.Reset();
			AssetBonesToSkelMeshMappings.AddUninitialized(NumAssetBones);
			TrainingBoneTransforms.SetNumUninitialized(NumAssetBones);

			// For each bone in the deformer asset, find the matching bone index inside the skeletal mesh component.
			for (int32 Index = 0; Index < NumAssetBones; ++Index)
			{
				const FName BoneName = InputInfo->GetBoneName(Index);
				const int32 SkelMeshBoneIndex = SkeletalMeshComponent->GetBaseComponent()->GetBoneIndex(BoneName);
				AssetBonesToSkelMeshMappings[Index] = SkelMeshBoneIndex;
			}
		}
	}

	// Perform a compatibility check.
	UpdateCompatibilityStatus();
}

void UMLDeformerModelInstance::UpdateCompatibilityStatus()
{
	bIsCompatible = SkeletalMeshComponent && SkeletalMeshComponent->GetSkeletalMeshAsset() && CheckCompatibility(SkeletalMeshComponent.Get(), true).IsEmpty();
}

FString UMLDeformerModelInstance::CheckCompatibility(USkeletalMeshComponent* InSkelMeshComponent, bool bLogIssues)
{
	ErrorText = FString();

	// If we're not compatible, generate a compatibility string.
	USkeletalMesh* SkelMesh = InSkelMeshComponent ? InSkelMeshComponent->GetSkeletalMeshAsset() : nullptr;
	UMLDeformerInputInfo* InputInfo = Model->GetInputInfo();
	if (SkelMesh && !InputInfo->IsCompatible(InSkelMeshComponent) && Model->GetDeformerAsset() && Model->IsTrained())
	{
		ErrorText += InputInfo->GenerateCompatibilityErrorString(InSkelMeshComponent);
		if (bLogIssues)
		{
			UE_LOGF(LogMLDeformer, Error, "ML Deformer '%ls' isn't compatible with Skeletal Mesh '%ls'.\nReason(s):\n%ls\n", 
				*Model->GetDeformerAsset()->GetName(), 
				*SkelMesh->GetName(), 
				*ErrorText);
		}
	}

	return ErrorText;
}

USkeletalMeshComponent* UMLDeformerModelInstance::GetFinalSkeletalMeshComponent() const
{
	#if WITH_EDITOR
		const UMLDeformerComponent* MLDeformerComponent = GetMLDeformerComponent();
		check(MLDeformerComponent);

		const AActor* DebugActor = MLDeformerComponent->GetDebugActor();
		if (DebugActor)
		{
			const USkeletalMesh* ModelSkelMesh = GetModel()->GetSkeletalMesh();
			if (ModelSkelMesh)
			{
				for (UActorComponent* Component : DebugActor->GetComponents())
				{
					USkeletalMeshComponent* DebugActorSkelMeshComponent = Cast<USkeletalMeshComponent>(Component);
					if (DebugActorSkelMeshComponent && DebugActorSkelMeshComponent->GetSkeletalMeshAsset() == ModelSkelMesh)
					{
						return DebugActorSkelMeshComponent;
					}
				}
			}
		}
	#endif

	return SkeletalMeshComponent.Get();
}

void UMLDeformerModelInstance::UpdateBoneTransforms()
{
	// If we use a leader component, we have to get the transforms from that component.
	USkeletalMeshComponent* FinalSkelMeshComponent = GetFinalSkeletalMeshComponent();
	const USkinnedMeshComponent* LeaderPoseComponent = FinalSkelMeshComponent->LeaderPoseComponent.Get();
	if (LeaderPoseComponent)
	{
		const TArray<FTransform>& LeaderTransforms = LeaderPoseComponent->GetComponentSpaceTransforms();
		if (!LeaderTransforms.IsEmpty())
		{
			const USkinnedAsset* SkinnedAsset = LeaderPoseComponent->GetSkinnedAsset();
			const FReferenceSkeleton& RefSkel = SkinnedAsset->GetRefSkeleton();
			const int32 NumTrainingBones = AssetBonesToSkelMeshMappings.Num();
			for (int32 Index = 0; Index < NumTrainingBones; ++Index)
			{
				const int32 ComponentBoneIndex = AssetBonesToSkelMeshMappings[Index];
				if (LeaderTransforms.IsValidIndex(ComponentBoneIndex))
				{
					const FTransform& ComponentSpaceTransform = LeaderTransforms[ComponentBoneIndex];
					const int32 ParentIndex = RefSkel.GetParentIndex(ComponentBoneIndex);
					if (LeaderTransforms.IsValidIndex(ParentIndex))
					{
						TrainingBoneTransforms[Index] = ComponentSpaceTransform.GetRelativeTransform(LeaderTransforms[ParentIndex]);
						TrainingBoneTransforms[Index].NormalizeRotation();
					}
					else
					{
						TrainingBoneTransforms[Index] = ComponentSpaceTransform;
					}
				}
			}
		}
	}
	else // No leader pose is used.
	{
		if (FinalSkelMeshComponent == SkeletalMeshComponent)
		{
			const TArray<FTransform>& ComponentTransforms = FinalSkelMeshComponent->GetComponentSpaceTransforms();
			if (!ComponentTransforms.IsEmpty())
			{
				const USkinnedAsset* SkinnedAsset = FinalSkelMeshComponent->GetSkinnedAsset();
				const FReferenceSkeleton& RefSkel = SkinnedAsset->GetRefSkeleton();
				const int32 NumTrainingBones = AssetBonesToSkelMeshMappings.Num();
				for (int32 Index = 0; Index < NumTrainingBones; ++Index)
				{
					const int32 ComponentBoneIndex = AssetBonesToSkelMeshMappings[Index];
					if (ComponentTransforms.IsValidIndex(ComponentBoneIndex))
					{
						const FTransform& ComponentSpaceTransform = ComponentTransforms[ComponentBoneIndex];
						const int32 ParentIndex = RefSkel.GetParentIndex(ComponentBoneIndex);
						if (ComponentTransforms.IsValidIndex(ParentIndex))
						{
							TrainingBoneTransforms[Index] = ComponentSpaceTransform.GetRelativeTransform(ComponentTransforms[ParentIndex]);
							TrainingBoneTransforms[Index].NormalizeRotation();
						}
						else
						{
							TrainingBoneTransforms[Index] = ComponentSpaceTransform;
						}
					}
				}
			}
		}
		else // We are debugging and grabbing transforms from another component.
		{
			USkeletalMesh* SkelMeshAsset = FinalSkelMeshComponent->GetSkeletalMeshAsset();
			check(SkelMeshAsset);

			// Allocate enough space in our bone space transforms buffer.
			// We need to calculate the bone space transforms, because when debugging only component space transforms are updated.
			DebugBoneSpaceTransforms.Reset();
			DebugBoneSpaceTransforms.SetNumUninitialized(FinalSkelMeshComponent->GetComponentSpaceTransforms().Num());
			const FReferenceSkeleton& RefSkel = SkelMeshAsset->GetRefSkeleton();
			CalculateBoneSpaceTransforms(RefSkel, FinalSkelMeshComponent->GetComponentSpaceTransforms(), DebugBoneSpaceTransforms);

			// Now that we have the new bone space transforms, store them so they will be passed into our neural network later.
			const int32 NumTrainingBones = AssetBonesToSkelMeshMappings.Num();
			for (int32 Index = 0; Index < NumTrainingBones; ++Index)
			{
				const int32 ComponentBoneIndex = AssetBonesToSkelMeshMappings[Index];
				if (DebugBoneSpaceTransforms.IsValidIndex(ComponentBoneIndex))
				{					
					TrainingBoneTransforms[Index] = DebugBoneSpaceTransforms[ComponentBoneIndex];
				}
			}
		}
	}
}

int64 UMLDeformerModelInstance::SetBoneTransforms(float* OutputBuffer, int64 OutputBufferSize, int64 StartIndex)
{
	// Get the transforms for the bones we used during training.
	// These are in the space relative to their parent.
	UpdateBoneTransforms();

	// Make sure we don't write past the OutputBuffer. Six, because of two columns of the 3x3 rotation matrix.
	const int32 AssetNumBones = Model->GetInputInfo()->GetNumBones();
	checkfSlow((StartIndex + AssetNumBones * 6) <= OutputBufferSize, TEXT("Writing bones past the end of the input buffer."));

	// Write 6 floats to the buffer, for each bone.
	UMLDeformerInputInfo::RotationToTwoVectorsAsSixFloats(TrainingBoneTransforms, OutputBuffer + StartIndex);

	// Return the new buffer offset, where we stopped writing.
	return StartIndex + AssetNumBones * 6;
}

int64 UMLDeformerModelInstance::SetCurveValues(float* OutputBuffer, int64 OutputBufferSize, int64 StartIndex)
{
	const UMLDeformerInputInfo* InputInfo = Model->GetInputInfo();

	int64 Index = StartIndex;
	const int32 AssetNumCurves = InputInfo->GetNumCurves();
	checkf((Index + AssetNumCurves) <= OutputBufferSize, TEXT("Writing curves past the end of the input buffer"));

	// Write the curve weights to the output buffer.
	for (int32 CurveIndex = 0; CurveIndex < AssetNumCurves; ++CurveIndex)
	{
		const FName CurveName = InputInfo->GetCurveName(CurveIndex);
		float CurveValue;
		SkeletalMeshComponent->GetCurveValue(CurveName, 0.0f, CurveValue);
		OutputBuffer[Index++] = CurveValue;
	}

	return Index;
}

int64 UMLDeformerModelInstance::SetNeuralNetworkInputValues(float* InputData, int64 NumInputFloats)
{
	// Feed data to the network inputs.
	int64 BufferOffset = 0;
	if (Model->DoesSupportBones())
	{
		BufferOffset = SetBoneTransforms(InputData, NumInputFloats, BufferOffset);
	}

	if (Model->DoesSupportCurves())
	{
		BufferOffset = SetCurveValues(InputData, NumInputFloats, BufferOffset);
	}

	return BufferOffset;
}

bool UMLDeformerModelInstance::IsValidForDataProvider() const
{
	return true;
}

bool UMLDeformerModelInstance::HasValidTransforms() const
{
	if (!SkeletalMeshComponent)
	{
		return false;
	}

	USkeletalMeshComponent* FinalSkeletalMeshComponent = GetFinalSkeletalMeshComponent();
	const USkinnedMeshComponent* LeaderPoseComponent = FinalSkeletalMeshComponent->LeaderPoseComponent.Get();
	if (LeaderPoseComponent)
	{
		if (LeaderPoseComponent->GetComponentSpaceTransforms().IsEmpty())
		{
			return false;
		}
	}
	else if (FinalSkeletalMeshComponent->GetBoneSpaceTransformsView().IsEmpty())
	{
		return false;
	}

	// Make sure we use the same Skeletal Mesh on the current skeletal mesh component as that we trained the deformer on.
	// This is already checked before by compatibility checks, but it can still fail when some other game code suddenly
	// changes the Skeletal Mesh on the SkeletalMeshComponent after the ML Deformer component has been initialized already.
	// This is just an extra safety check.
	USkeletalMesh* SkelMesh = FinalSkeletalMeshComponent->GetSkeletalMeshAsset();
	UMLDeformerInputInfo* InputInfo = Model->GetInputInfo();
	if (SkelMesh && InputInfo)
	{
		if (FSoftObjectPath(SkelMesh) != InputInfo->GetSkeletalMesh())
		{
			return false;
		}
	}
	else if (!SkelMesh && InputInfo && InputInfo->GetSkeletalMesh().IsValid())
	{
		return false;
	}

	return true;
}

void UMLDeformerModelInstance::Tick(float DeltaTime, float ModelWeight)
{
	// Post init hasn't yet succeeded, try again.
	// This could for example happen when you add an ML Deformer component, but your SkeletalMesh isn't setup yet, but later becomes valid.
	if (Model && !HasPostInitialized())
	{
		PostMLDeformerComponentInit();
	}

	bool bExecuteCalled = false;
	if (ModelWeight > 0.0001f && IsCompatible() && HasValidTransforms() && SetupInputs())
	{
		// Execute the model instance.
		// For models using neural networks this will perform the inference, 
		// calculate the network outputs and possibly use them, depending on how the model works.
		Execute(ModelWeight);
		const bool bDrawDebugInputPoses = UE::MLDeformer::CVarMLDeformerDebugDrawInputPoses->GetBool();
		if (bDrawDebugInputPoses)
		{
			DrawDebugInputPoses();
		}

		bExecuteCalled = true;
	}
	else
	{
		HandleZeroModelWeight();
	}

	// Execute the post tick.
	PostTick(bExecuteCalled);
}

#if WITH_EDITOR
	void UMLDeformerModelInstance::CopyDataFromCurrentDebugActor()
	{
		UMLDeformerComponent* MLDeformerComponent = GetMLDeformerComponent();
		if (MLDeformerComponent)
		{
			// Check if we're debugging at all.
			const AActor* DebugActor = MLDeformerComponent->GetDebugActor();
			if (DebugActor == nullptr)
			{
				return;
			}

			// Make sure we use the same ML Deformer asset and skeletal mesh etc.
			UMLDeformerComponent* DebugMLDeformerComponent = nullptr;
			if (!Model->IsCompatibleDebugActor(DebugActor, &DebugMLDeformerComponent))
			{
				return;
			}
			check(DebugMLDeformerComponent);

			// Now copy the actual data.
			CopyDataFromDebugActor(DebugActor, DebugMLDeformerComponent);
		}
	}

	void UMLDeformerModelInstance::SetGroundTruthFrameIndex(int32 FrameIndex)
	{ 
		GroundTruthFrameIndex = FrameIndex;
	}

	int32 UMLDeformerModelInstance::GetGroundTruthFrameIndex() const
	{ 
		return GroundTruthFrameIndex;
	}
#endif
