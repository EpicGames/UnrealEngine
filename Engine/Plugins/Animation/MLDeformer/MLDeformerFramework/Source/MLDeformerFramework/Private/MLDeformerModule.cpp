// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerModule.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "ShaderCore.h"

// Log category.
MLDEFORMERFRAMEWORK_API DEFINE_LOG_CATEGORY(LogMLDeformer)

#define LOCTEXT_NAMESPACE "MLDeformerModule"

IMPLEMENT_MODULE(UE::MLDeformer::FMLDeformerModule, MLDeformerFramework)

namespace UE::MLDeformer
{
	void FMLDeformerModule::StartupModule()
	{
		// Register an additional shader path for our shaders used inside the deformer graph system.
		const FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("MLDeformerFramework"))->GetBaseDir(), TEXT("Shaders"));
		AddShaderSourceDirectoryMapping(TEXT("/Plugin/MLDeformerFramework"), PluginShaderDir);

		CVarMLDeformerMaxLODLevels = IConsoleManager::Get().RegisterConsoleVariable(
			TEXT("sg.MLDeformerMaxLODLevelsOnCook"),
			8,
			TEXT("Limit the number of ML Deformer LOD levels that can be generated. Can be used to setup per device or platform limits. Minimum value is 1."),
			ECVF_Scalability);
	}

	void FMLDeformerModule::ShutdownModule()
	{
		IConsoleManager::Get().UnregisterConsoleObject(CVarMLDeformerMaxLODLevels);
		IModuleInterface::ShutdownModule();
	}

}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
