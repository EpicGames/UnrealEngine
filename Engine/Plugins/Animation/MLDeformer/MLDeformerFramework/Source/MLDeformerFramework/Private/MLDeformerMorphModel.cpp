// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerMorphModel.h"
#include "MLDeformerMorphModelInstance.h"
#include "MLDeformerModelInstance.h"
#include "MLDeformerComponent.h"
#include "MLDeformerMorphModelInputInfo.h"
#include "MLDeformerObjectVersion.h"
#include "MLDeformerModule.h"
#include "Components/ExternalMorphSet.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/MorphTargetVertexInfoBuffers.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MLDeformerMorphModel)

#define LOCTEXT_NAMESPACE "MLDeformerMorphModel"

UMLDeformerMorphModel::UMLDeformerMorphModel(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UMLDeformerMorphModel::Serialize(FArchive& Archive)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMLDeformerMorphModel::Serialize)

	bool bModifiedPropertiesForCook = false;
	TArray<FVector3f> SavedMorphTargetDeltas;
	auto ModifyPropertiesForCook = [this, &bModifiedPropertiesForCook, &SavedMorphTargetDeltas]()
	{
		bModifiedPropertiesForCook = true;
		SavedMorphTargetDeltas = MoveTemp(MorphTargetDeltas);
		MorphTargetDeltas.Empty();
	};

	auto RestorePropertiesForCook = [this, &bModifiedPropertiesForCook, &SavedMorphTargetDeltas]()
	{
		if (!bModifiedPropertiesForCook)
		{
			return;
		}
		MorphTargetDeltas = MoveTemp(SavedMorphTargetDeltas);
	};

	ON_SCOPE_EXIT
	{
		if (GetRecoverStrippedDataAfterCook())
		{
			RestorePropertiesForCook();
		}
	};

	int32 NumSaveLODs = 0;
	if (Archive.IsSaving())
	{
		// Strip editor only data on cook.
		NumSaveLODs = GetNumLODs();
		if (Archive.IsCooking())
		{
			ModifyPropertiesForCook();

			// Check if we want to limit the number of LODs (can be per platform/device).
			UE::MLDeformer::FMLDeformerModule& MLDeformerModule = FModuleManager::LoadModuleChecked<UE::MLDeformer::FMLDeformerModule>("MLDeformerFramework");
			const int32 MaxLODLevels = FMath::Clamp(MLDeformerModule.GetMaxLODLevelsOnCookCVar().GetInt(), 1, 1000);	// Limit to 1000 LODs, which should never be reached.

			// Get lowest value between what we generated, console variable and the UI/property max lods value.
			NumSaveLODs = FMath::Min3(NumSaveLODs, MaxLODLevels, GetMaxNumLODs());

			UE_LOGF(LogMLDeformer, Display, "Cooking MLD asset '%ls' with %d LOD levels", *GetFullName(), NumSaveLODs);
		}
		else
		{
			// Get lowest number between how many LODs we have generated and the number of LODs we setup in the UI/Property.
			NumSaveLODs = FMath::Min(NumSaveLODs, GetMaxNumLODs());
		}
	}

	Super::Serialize(Archive);
	Archive.UsingCustomVersion(UE::MLDeformer::FMLDeformerObjectVersion::GUID);

	// Check if we have initialized our compressed morph buffers.
	bool bHasMorphData = false;
	if (Archive.IsSaving())
	{
		// Save all LOD levels, strip out LODs we don't want.
		Archive << NumSaveLODs;
		for (int32 LOD = 0; LOD < NumSaveLODs; ++LOD)
		{
			bHasMorphData = GetMorphTargetSet(LOD).IsValid() ? GetMorphTargetSet(LOD)->MorphBuffers.IsMorphCPUDataValid() : false;
			Archive << bHasMorphData;

			// Load or save the compressed morph buffers, if they exist.
			if (bHasMorphData)
			{
				Archive << GetMorphTargetSet(LOD)->MorphBuffers;
			}
		}
	}

	if (Archive.IsLoading())
	{
		// If we only support 1 LOD, in older files.
		if (Archive.CustomVer(UE::MLDeformer::FMLDeformerObjectVersion::GUID) < UE::MLDeformer::FMLDeformerObjectVersion::LODSupportAdded)
		{
			ClearMorphTargetSets();
			AddMorphSets(1);
			Archive << bHasMorphData;

			// Load or save the compressed morph buffers, if they exist.
			if (bHasMorphData)
			{
				Archive << GetMorphTargetSet(0)->MorphBuffers;
			}
		}
		else // We support multiple LOD levels.
		{
			int32 NumLODs = 1;
			Archive << NumLODs;

			ClearMorphTargetSets();
			AddMorphSets(NumLODs);

			for (int32 LOD = 0; LOD < NumLODs; ++LOD)
			{
				Archive << bHasMorphData;

				// Load or save the compressed morph buffers, if they exist.
				if (bHasMorphData)
				{
					Archive << GetMorphTargetSet(LOD)->MorphBuffers;
				}
			}
		}
	}
}

UMLDeformerInputInfo* UMLDeformerMorphModel::CreateInputInfo()
{
	return NewObject<UMLDeformerMorphModelInputInfo>(this, NAME_None, RF_Transactional);
}

void UMLDeformerMorphModel::PostLoad()
{
	Super::PostLoad();

	// If we have an input info, but it isn't one inherited from the MorphInputInfo, try to create a new one.
	// This is because we introduced a UMLDeformerMorphModelInputInfo later on, and we want to convert old assets to use this new class.
	UMLDeformerInputInfo* CurrentInputInfo = GetInputInfo();
	if (CurrentInputInfo && !CurrentInputInfo->IsA<UMLDeformerMorphModelInputInfo>())
	{
		UMLDeformerMorphModelInputInfo* MorphInputInfo = Cast<UMLDeformerMorphModelInputInfo>(CreateInputInfo());
		MorphInputInfo->CopyMembersFrom(CurrentInputInfo);
		CurrentInputInfo->ConditionalBeginDestroy();
		check(MorphInputInfo); // The input info class should be inherited from the UMLDeformerMorphModelInputInfo class.
		SetInputInfo(MorphInputInfo);
	}

	UpdateStatistics();

#if WITH_EDITOR
	InvalidateMemUsage();
#endif
}

void UMLDeformerMorphModel::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	Context.AddTag(FAssetRegistryTag("MLDeformer.MorphModel.ClampMorphWeights", bClampMorphWeights ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.MorphModel.InvertMaskChannel", bInvertMaskChannel? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.MorphModel.IncludeNormals", bIncludeNormals ? TEXT("True") : TEXT("False"), FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.MorphModel.NumMorphTargets", FString::FromInt(GetNumMorphTargets(0)), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.MorphModel.DeltaZeroThreshold", FString::Printf(TEXT("%f"), MorphDeltaZeroThreshold), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.MorphModel.CompressionLevel", FString::Printf(TEXT("%f"), MorphCompressionLevel), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.MorphModel.CompressedSize", FString::FromInt(CompressedMorphDataSizeInBytes), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.MorphModel.UncompressedSize", FString::FromInt(UncompressedMorphDataSizeInBytes), FAssetRegistryTag::TT_Numerical));
	Context.AddTag(FAssetRegistryTag("MLDeformer.MorphModel.NumLODs", FString::FromInt(GetNumLODs()), FAssetRegistryTag::TT_Numerical));
}

int32 UMLDeformerMorphModel::GetNumMorphTargets(int32 LOD) const
{
	if (GetNumLODs() > LOD)
	{
		return GetMorphTargetSet(LOD).IsValid() ? GetMorphTargetSet(LOD)->MorphBuffers.GetNumMorphs() : 0;
	}
	return 0;
}

bool UMLDeformerMorphModel::CanDynamicallyUpdateMorphTargets() const
{
	if (GetNumBaseMeshVerts() > 0)
	{
		return (GetMorphTargetDeltas().Num() % GetNumBaseMeshVerts() == 0) || bHasOnlyEmptyMorphs;
	}

	return false;
}

UMLDeformerModelInstance* UMLDeformerMorphModel::CreateModelInstance(UMLDeformerComponent* Component)
{
	return NewObject<UMLDeformerMorphModelInstance>(Component);
}

void UMLDeformerMorphModel::SetMorphTargetDeltaFloats(const TArray<float>& Deltas)
{
	FloatArrayToVector3Array(Deltas, GetMorphTargetDeltas());
}

void UMLDeformerMorphModel::SetMorphTargetDeltas(const TArray<FVector3f>& Deltas)
{
	MorphTargetDeltas = Deltas;
}

void UMLDeformerMorphModel::ClearMorphTargetSets()
{
	const int32 NumLODs = GetNumLODs();
	for (int32 LOD = 0; LOD < NumLODs; ++LOD)
	{
		if (GetMorphTargetSet(LOD).IsValid())
		{
			FMorphTargetVertexInfoBuffers& MorphBuffer = GetMorphTargetSet(LOD)->MorphBuffers;
			if (MorphBuffer.IsRHIInitialized() && MorphBuffer.IsInitialized())
			{
				ReleaseResourceAndFlush(&MorphBuffer);
			}
		}
	}

	MorphTargetSets.Empty();
}

void UMLDeformerMorphModel::AddMorphSets(int32 NumToAdd)
{
	for (int32 Index = 0; Index < NumToAdd; ++Index)
	{
		TSharedPtr<FExternalMorphSet> NewSet = MakeShared<FExternalMorphSet>();
		NewSet->Name = GetClass()->GetFName();
		MorphTargetSets.Add(NewSet);
	}
}

int32 UMLDeformerMorphModel::GetMorphTargetDeltaStartIndex(int32 MorphTargetIndex) const
{
	if (GetMorphTargetDeltas().Num() == 0)
	{
		return INDEX_NONE;
	}

	return GetNumBaseMeshVerts() * MorphTargetIndex;
}

void UMLDeformerMorphModel::BeginDestroy()
{
	for (int32 LOD = 0; LOD < GetNumLODs(); ++LOD)
	{
		if (GetMorphTargetSet(LOD).IsValid())
		{
			BeginReleaseResource(&GetMorphTargetSet(LOD)->MorphBuffers);
		}
	}
	RenderCommandFence.BeginFence();
	Super::BeginDestroy();
}

bool UMLDeformerMorphModel::IsReadyForFinishDestroy()
{
	// Wait for associated render resources to be released.
	return Super::IsReadyForFinishDestroy() && RenderCommandFence.IsFenceComplete();
}

void UMLDeformerMorphModel::SetMorphTargetsErrorOrder(const TArray<int32>& MorphTargetOrder, const TArray<float>& ErrorValues)
{
	MorphTargetErrorOrder = MorphTargetOrder;
	MorphTargetErrors = ErrorValues;
}

void UMLDeformerMorphModel::UpdateStatistics()
{
	CompressedMorphDataSizeInBytes = 0;
	const int32 NumLODs = GetNumLODs();
	for (int32 LOD = 0; LOD < NumLODs; ++LOD)
	{
		CompressedMorphDataSizeInBytes += GetMorphTargetSet(LOD).IsValid() ? GetMorphTargetSet(LOD)->MorphBuffers.GetMorphDataSizeInBytes() : 0;		
	}
	UncompressedMorphDataSizeInBytes = GetMorphTargetDeltas().Num() * GetMorphTargetDeltas().GetTypeSize();
}

void UMLDeformerMorphModel::SetMorphTargetsMinMaxWeights(const TArray<FFloatInterval>& MinMaxValues)
{
	MorphTargetsMinMaxWeights = MinMaxValues;
}

void UMLDeformerMorphModel::SetMorphTargetsMinMaxWeights(const TArray<float>& MinValues, const TArray<float>& MaxValues)
{
	check(MinValues.Num() == MaxValues.Num());
	const int32 NumWeights = MinValues.Num();

	MorphTargetsMinMaxWeights.Reset();
	MorphTargetsMinMaxWeights.AddUninitialized(NumWeights);
	for (int32 Index = 0; Index < MinValues.Num(); ++Index)
	{
		MorphTargetsMinMaxWeights[Index].Min = MinValues[Index];
		MorphTargetsMinMaxWeights[Index].Max = MaxValues[Index];
	}
}

void UMLDeformerMorphModel::ClampMorphTargetWeights(TArrayView<float> WeightsArray)
{
	if (MorphTargetsMinMaxWeights.Num() != WeightsArray.Num())
	{
		return;
	}

	for (int32 MorphIndex = 0; MorphIndex < WeightsArray.Num(); ++MorphIndex)
	{
		const FFloatInterval& MorphMinMax = MorphTargetsMinMaxWeights[MorphIndex];
		WeightsArray[MorphIndex] = FMath::Clamp(WeightsArray[MorphIndex], MorphMinMax.Min, MorphMinMax.Max);;
	}
}

TArrayView<const float> UMLDeformerMorphModel::GetMorphTargetErrorValues() const
{
	return MorphTargetErrors;
}

TArrayView<const int32> UMLDeformerMorphModel::GetMorphTargetErrorOrder() const
{ 
	return MorphTargetErrorOrder;
}

float UMLDeformerMorphModel::GetMorphTargetError(int32 MorphIndex) const
{
	return MorphTargetErrors[MorphIndex];
}

void UMLDeformerMorphModel::SetMorphTargetError(int32 MorphIndex, float Error)
{
	MorphTargetErrors[MorphIndex] = Error;
}

#if WITH_EDITORONLY_DATA
FName UMLDeformerMorphModel::GetGlobalMaskAttributeName() const
{
	return VertexAttributeName;
}
#endif


#if WITH_EDITOR
void UMLDeformerMorphModel::UpdateMemoryUsage()
{
	Super::UpdateMemoryUsage();

	// Remove the raw uncompressed deltas from the cooked size and memory usage, as they are stripped during cook.
	// This means the game itself won't have this data in the asset or memory.
	CookedAssetSizeInBytes -= UncompressedMorphDataSizeInBytes;
	MemUsageInBytes -= UncompressedMorphDataSizeInBytes;

	// Add the compressed morph target data size.
	// We add this to both the GPU memory, and the cooked asset size.
	// The morph targets are stored in a compressed way inside the asset.
	const uint64 GPUMorphSize = GetCompressedMorphDataSizeInBytes();
	GPUMemUsageInBytes += GPUMorphSize;
	CookedAssetSizeInBytes += GPUMorphSize;

	EditorAssetSizeInBytes += CompressedMorphDataSizeInBytes;
}

void UMLDeformerMorphModel::FinalizeMorphTargets()
{
	GetMorphTargetDeltas().Empty();
	UpdateStatistics();
	UpdateMemoryUsage();
}

bool UMLDeformerMorphModel::HasRawMorph() const
{
	return !GetMorphTargetDeltas().IsEmpty();
}
#endif

#undef LOCTEXT_NAMESPACE

