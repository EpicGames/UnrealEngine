// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerMorphModelInputInfo.h"
#include "MLDeformerMorphModel.h"
#include UE_INLINE_GENERATED_CPP_BY_NAME(MLDeformerMorphModelInputInfo)

void UMLDeformerMorphModelInputInfo::Serialize(FArchive& Archive)
{
	// Strip editor only data.
#if WITH_EDITORONLY_DATA
	TArray<float> InputItemMaskBufferBackup;
	bool bProcessedDataOnCook = false;
	ON_SCOPE_EXIT
	{
		UMLDeformerModel* Model = Cast<UMLDeformerModel>(GetOuter());
		if (bProcessedDataOnCook && Model && Model->GetRecoverStrippedDataAfterCook())
		{
			InputItemMaskBuffer = MoveTemp(InputItemMaskBufferBackup);
		}
	};

	if (Archive.IsSaving() && Archive.IsCooking())
	{
		InputItemMaskBufferBackup = MoveTemp(InputItemMaskBuffer);
		InputItemMaskBuffer.Empty();
		bProcessedDataOnCook = true;
	}
#endif

	Super::Serialize(Archive);
}

#if WITH_EDITORONLY_DATA
	TArray<float>& UMLDeformerMorphModelInputInfo::GetInputItemMaskBuffer()
	{ 
		return InputItemMaskBuffer;
	}

	const TArray<float>& UMLDeformerMorphModelInputInfo::GetInputItemMaskBuffer() const
	{ 
		return InputItemMaskBuffer;
	}

	const TArrayView<const float> UMLDeformerMorphModelInputInfo::GetMaskForItem(int32 MaskItemIndex) const
	{
		const UMLDeformerMorphModel* MorphModel = Cast<UMLDeformerMorphModel>(GetOuter());
		check(MorphModel);

		if (InputItemMaskBuffer.IsEmpty())
		{
			return TArrayView<const float>();
		}

		const int32 NumVerts = MorphModel->GetNumBaseMeshVerts();
		check(MaskItemIndex * NumVerts + NumVerts <= InputItemMaskBuffer.Num());
		return TArrayView<const float>(&InputItemMaskBuffer[MaskItemIndex * NumVerts], NumVerts);
	}
#endif
