// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerMorphModelInstance.h"
#include "MLDeformerMorphModel.h" 
#include "MLDeformerComponent.h" 
#include "Components/ExternalMorphSet.h"
#include "Components/SkeletalMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MLDeformerMorphModelInstance)

TAtomic<int32> UMLDeformerMorphModelInstance::NextFreeMorphSetID(0);

void UMLDeformerMorphModelInstance::Init(USkeletalMeshComponent* SkelMeshComponent)
{
	Super::Init(SkelMeshComponent);

	// Generate a unique ID for our morph target set.
	ExternalMorphSetID = NextFreeMorphSetID++;
}

bool UMLDeformerMorphModelInstance::IsValidForDataProvider() const
{
	return true;
}

void UMLDeformerMorphModelInstance::PostTick(bool bExecuteCalled)
{
	#if WITH_EDITOR
		CopyDataFromCurrentDebugActor();
	#endif
}

int32 UMLDeformerMorphModelInstance::GetExternalMorphSetID() const
{ 
	return ExternalMorphSetID;
}

void UMLDeformerMorphModelInstance::BeginDestroy()
{
	// Try to unregister the morph target and morph target set.
	if (SkeletalMeshComponent)
	{
		const UMLDeformerMorphModel* MorphModel = Cast<UMLDeformerMorphModel>(Model);
		if (MorphModel)
		{
			const int32 NumLODs = SkeletalMeshComponent->GetNumLODs();
			for (int32 LOD = 0; LOD < NumLODs; ++LOD)
			{
				SkeletalMeshComponent->RemoveExternalMorphSet(LOD, ExternalMorphSetID);
			}
			SkeletalMeshComponent->RefreshExternalMorphTargetWeights();
		}
	}

	Super::BeginDestroy();
}

void UMLDeformerMorphModelInstance::PostMLDeformerComponentInit()
{
	if (HasPostInitialized())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UMLDeformerMorphModelInstance::PostMLDeformerComponentInit)

	Super::PostMLDeformerComponentInit();

	// Register the external morph targets buffer to the skinned mesh component.
	USkeletalMeshComponent* SkelMeshComponent = GetSkeletalMeshComponent();
	if (SkelMeshComponent && SkelMeshComponent->GetSkeletalMeshAsset())
	{	
		// Get the morph model and its morph target set.
		UMLDeformerMorphModel* MorphModel = Cast<UMLDeformerMorphModel>(Model);
		check(MorphModel);

		check(IsInGameThread());	// We don't want to call this multithreaded, as the AddExternalMorphSets etc isn't thread safe.

		const int32 NumLODs = MorphModel->GetNumLODs();
		for (int32 LOD = 0; LOD < NumLODs; ++LOD)
		{
			TSharedPtr<FExternalMorphSet> MorphTargetSet = MorphModel->GetMorphTargetSet(LOD);

			// Register the morph set. This overwrites the existing one for this model, if it already exists.
			SkelMeshComponent->AddExternalMorphSet(LOD, ExternalMorphSetID, MorphTargetSet);

			// Update the weight information in the Skeletal Mesh.
			SkelMeshComponent->RefreshExternalMorphTargetWeights();

			// When we're in editor mode, keep the CPU data around, so we can re-initialize when needed.
			#if WITH_EDITOR
				MorphTargetSet->MorphBuffers.SetEmptyMorphCPUDataOnInitRHI(false);
			#else
				MorphTargetSet->MorphBuffers.SetEmptyMorphCPUDataOnInitRHI(true);
			#endif

			// Only release render resources in the editor build. Editor builds must re-initialize the render resources
			// as the morph targets can change after training. Non-editor builds assume the data does not change, and there
			// is no need to re-init.
			FMorphTargetVertexInfoBuffers& MorphBuffers = MorphTargetSet->MorphBuffers;
			#if WITH_EDITOR
				BeginReleaseResource(&MorphBuffers);
			#endif

			// Reinitialize the GPU compressed buffers.
			if (MorphBuffers.IsMorphCPUDataValid() && MorphBuffers.GetNumMorphs() > 0)
			{
				// In a non-editor build this will clear the CPU data.
				// That also means it can't re-init the resources later on again.
				BeginInitResource(&MorphBuffers);
			}
		}

		SetHasPostInitialized(true);
	}
}

FExternalMorphSetWeights* UMLDeformerMorphModelInstance::FindWeightData(int32 LOD) const
{
	// Check if our LOD index is valid first, as we might not have registered yet.
	if (!SkeletalMeshComponent || !SkeletalMeshComponent->IsValidExternalMorphSetLODIndex(LOD))
	{
		return nullptr;
	}

	// Grab the weight data for this morph set.
	// This could potentially fail if we are applying this deformer to the wrong skeletal mesh component.
	return SkeletalMeshComponent->GetExternalMorphWeights(LOD).MorphSets.Find(ExternalMorphSetID);
}

void UMLDeformerMorphModelInstance::HandleZeroModelWeight()
{
	if (!SkeletalMeshComponent)
	{
		return;
	}

	const int32 LOD = SkeletalMeshComponent->GetPredictedLODLevel();
	FExternalMorphSetWeights* WeightData = FindWeightData(LOD);
	if (WeightData)
	{
		WeightData->ZeroWeights();
	}
}
