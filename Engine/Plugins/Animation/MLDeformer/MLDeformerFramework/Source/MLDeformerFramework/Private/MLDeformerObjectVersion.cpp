// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerObjectVersion.h"
#include "UObject/DevObjectVersion.h"

namespace UE::MLDeformer
{
	// Unique MLDeformer Object version ID.
	const FGuid FMLDeformerObjectVersion::GUID(0x23D9FD27, 0x4F6A0FDF, 0xEA055480, 0x7D4B3AB0);
	
	// Register the custom version with Core.
	FDevVersionRegistration MLDeformerObjectVersionRegistration(FMLDeformerObjectVersion::GUID, FMLDeformerObjectVersion::LatestVersion, TEXT("Dev-MLDeformer"));
}
