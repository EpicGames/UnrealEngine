// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerPerfCounter.h"
#include "HAL/PlatformTime.h"

namespace UE::MLDeformer
{
	FMLDeformerPerfCounter::FMLDeformerPerfCounter()
	{
		SetHistorySize(100);
	}

	void FMLDeformerPerfCounter::SetHistorySize(int32 NumHistoryItems)
	{
		CycleHistory.Reset();
		CycleHistory.SetNumZeroed(NumHistoryItems);
		Reset();
	}

	void FMLDeformerPerfCounter::BeginSample()
	{
		StartTime = FPlatformTime::Cycles();
	}

	void FMLDeformerPerfCounter::EndSample()
	{
		Cycles = FPlatformTime::Cycles() - StartTime;

		if (NumSamples == 0 || (!CycleHistory.IsEmpty() && (NumSamples % CycleHistory.Num()) == 0))
		{
			CyclesMin = Cycles;
			CyclesMax = Cycles;
		}
		else
		{
			CyclesMin = FMath::Min<int32>(Cycles, CyclesMin);
			CyclesMax = FMath::Max<int32>(Cycles, CyclesMax);
		}

		if (!CycleHistory.IsEmpty())
		{
			CycleHistory[NumSamples % CycleHistory.Num()] = Cycles;
		}

		NumSamples++;
	}

	int32 FMLDeformerPerfCounter::GetCycles() const
	{
		return Cycles;
	}

	int32 FMLDeformerPerfCounter::GetCyclesMin() const
	{
		return CyclesMin;
	}

	int32 FMLDeformerPerfCounter::GetCyclesMax() const
	{
		return CyclesMax;
	}

	int32 FMLDeformerPerfCounter::GetCyclesAverage() const
	{
		if (CycleHistory.IsEmpty() || NumSamples == 0)
		{
			return Cycles;
		}

		int32 Sum = 0;
		const int32 NumRecordedSamples = FMath::Min<int32>(NumSamples, CycleHistory.Num());
		for (int32 Index = 0; Index < NumRecordedSamples; ++Index)
		{
			Sum += CycleHistory[Index];
		}
		return Sum / NumRecordedSamples;
	}

	int32 FMLDeformerPerfCounter::GetNumSamples() const
	{
		return NumSamples;
	}

	int32 FMLDeformerPerfCounter::GetHistorySize() const
	{
		return CycleHistory.Num();
	}

	void FMLDeformerPerfCounter::Reset()
	{
		NumSamples = 0;
		CyclesMax = 0;
		CyclesMin = 0;
	}
}	// namespace UE::MLDeformer
