// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerTrainingDataProcessorSettings.h"
#include "MLDeformerModel.h"
#include "Animation/Skeleton.h"
#include "Animation/AnimSequence.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MLDeformerTrainingDataProcessorSettings)

#define LOCTEXT_NAMESPACE "MLDeformerTrainingDataProcessorSettings"

#if WITH_EDITOR
bool UMLDeformerTrainingDataProcessorSettings::IsValid(const USkeleton* Skeleton) const
{
	if (!Skeleton)
	{
		return false;
	}

	// If there is nothing to do.
	if (!bReduceFrames && !bRemixPoses)
	{
		return false;
	}

	// Make sure we have at least one valid input animation.
	// As in theory the entries in the list could be empty ones or could all be disabled.
	int32 NumValidAnims = 0;
	int32 TotalNumFrames = 0;
	for (const FMLDeformerTrainingDataProcessorAnim& Anim : AnimList)
	{
		if (Anim.bEnabled && Anim.AnimSequence.LoadSynchronous())
		{
			const USkeleton* AnimSkeleton = Anim.AnimSequence->GetSkeleton();
			if (AnimSkeleton == Skeleton)
			{
				TotalNumFrames += Anim.AnimSequence->GetDataModel() ? Anim.AnimSequence->GetDataModel()->GetNumberOfFrames() : 0;
				NumValidAnims++;
			}
		}
	}

	// We have no valid animations, so there is nothing to do.
	if (NumValidAnims == 0)
	{
		return false;
	}

	// We need at least some frames in the anim sequences.
	if (TotalNumFrames == 0)
	{
		return false;
	}

	// Make sure we have a valid output animation sequence setup.
	if (!OutputAnimSequence.LoadSynchronous())
	{
		return false;
	}

	return true;
}

USkeleton* UMLDeformerTrainingDataProcessorSettings::FindSkeleton() const
{
	UMLDeformerModel* Model = Cast<UMLDeformerModel>(GetOuter());
	if (Model)
	{
		return Model->GetSkeletalMesh() ? Model->GetSkeletalMesh()->GetSkeleton() : nullptr;
	}
	return nullptr;
}

int32 UMLDeformerTrainingDataProcessorSettings::GetNumInputAnimationFrames() const
{
	int32 NumFrames = 0;
	for (const FMLDeformerTrainingDataProcessorAnim& Anim : AnimList)
	{
		if (Anim.bEnabled && Anim.AnimSequence.LoadSynchronous())
		{
			NumFrames += Anim.AnimSequence->GetDataModel() ? Anim.AnimSequence->GetDataModel()->GetNumberOfFrames() : 0;
		}
	}
	return NumFrames;
}
#endif // #if WITH_EDITOR

#undef LOCTEXT_NAMESPACE
