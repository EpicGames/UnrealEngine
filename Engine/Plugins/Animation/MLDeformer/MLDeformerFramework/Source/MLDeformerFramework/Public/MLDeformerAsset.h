// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "MLDeformerAsset.generated.h"

#define UE_API MLDEFORMERFRAMEWORK_API

class UMLDeformerModel;

/**
 * The machine learning deformer asset class.
 * This class contains a Model property, through which most functionality happens.
 */
UCLASS(MinimalAPI, BlueprintType, hidecategories=Object)
class UMLDeformerAsset 
	: public UObject
{
	GENERATED_BODY()

public:
	// UObject overrides.
	UE_API virtual void Serialize(FArchive& Archive) override;
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	// ~END UObject overrides.

	/**
	 * Get the ML Deformer model that is being applied by this asset.
	 * @return A pointer to the model.
	 */
	UMLDeformerModel* GetModel() const			{ return Model.Get(); }

	/**
	 * Set the ML Deformer model that is used by this deformer asset.
	 * @param InModel A pointer to the model object.
	 */
	void SetModel(UMLDeformerModel* InModel)	{ Model = InModel; }

public:
	/** The ML Deformer model, used to deform the mesh. */
	UPROPERTY()
	TObjectPtr<UMLDeformerModel> Model = nullptr;
};

#undef UE_API
