// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "Components/ActorComponent.h"
#include "RenderCommandFence.h"
#include "MLDeformerPerfCounter.h"
#include "MLDeformerComponent.generated.h"

#define UE_API MLDEFORMERFRAMEWORK_API

class UMLDeformerAsset;
class UMLDeformerModelInstance;
class USkeletalMeshComponent;

/**
 * The ML mesh deformer component.
 * This works in combination with a MLDeformerAsset and SkeletalMeshComponent.
 * The component will perform runtime inference of the deformer model setup inside the asset.
 * When you have multiple skeletal mesh components on your actor, this component will try to use the skeletal mesh component that uses
 * the same skeletal mesh as the applied deformer was trained on.
 * If it cannot find that, it will use the first skeletal mesh component it finds.
 */
UCLASS(MinimalAPI, Blueprintable, ClassGroup = Component, BlueprintType, meta = (BlueprintSpawnableComponent))
class UMLDeformerComponent
	: public UActorComponent
{
	GENERATED_UCLASS_BODY()

public:
	// UObject overrides.
	UE_API void BeginDestroy() override;
#if WITH_EDITOR
	UE_API void PreEditChange(FProperty* Property) override;
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	// ~END UObject overrides.

	// UActorComponent overrides.
	UE_API void Activate(bool bReset=false) override;
	UE_API void Deactivate() override;
	// ~END UActorComponent overrides.

	/** 
	 * Setup the ML Deformer, by picking the deformer asset and skeletal mesh component. 
	 * @param InDeformerAsset The ML Deformer asset to apply to the specified skeletal mesh component.	 
	 * @param InSkelMeshComponent The skeletal mesh component to apply the specified deformer to.
	 */
	UE_API void SetupComponent(UMLDeformerAsset* InDeformerAsset, USkeletalMeshComponent* InSkelMeshComponent);

	/** 
	 * Get the current ML Deformer weight. A value of 0 means it is fully disabled, while 1 means fully active.
	 * Values can be anything between 0 and 1.
	 */
	UFUNCTION(BlueprintCallable, Category = "MLDeformer")
	float GetWeight() const										{ return Weight; }

	/**
	 * Set the ML Deformer weight. This determines how active the deformer is. You can see it as a blend weight.
	 * A value of 0 means it is inactive. Certain calculations will be skipped in that case.
	 * A value of 1 means it is fully active.
	 * Values between 0 and 1 blend between the two states.
	 * Call this after you call SetupComponent.
	 * @param NormalizedWeightValue The weight value.
	 */
	UFUNCTION(BlueprintCallable, Category = "MLDeformer")
	void SetWeight(float NormalizedWeightValue)					{ SetWeightInternal(NormalizedWeightValue); }

	/** 
	 * The quality level of the deformer. A value of 0 is the highest quality, 1 is a step lower, etc.
	 * It is up to the models to configure what each quality level means visually.
	 * If the quality level will be clamped to the available quality levels, so if you choose quality level 100, while there are only 3 levels, then quality
	 * level 3 will be used in this case, which represents the lowest available quality.
	 * In morph based models each quality level defines how many morph targets are active at most.
	 * @param InQualityLevel The quality level to switch to.
	 */
	UE_DEPRECATED(5.4, "This function will be removed.")
	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "SetQualityLevel has been deprecated."))
	void SetQualityLevel(int32 InQualityLevel)					{}

	/** 
	 * The quality level of the deformer. A value of 0 is the highest quality, 1 is a step lower, etc.
	 * It is up to the models to configure what each quality level means visually.
	 * If the quality level will be clamped to the available quality levels, so if you choose quality level 100, while there are only 3 levels, then quality
	 * level 3 will be used in this case, which represents the lowest available quality.
	 * In morph based models each quality level defines how many morph targets are active at most.
	 * @param InQualityLevel The quality level to switch to.
	 */
	UE_DEPRECATED(5.4, "This function will be removed.")
	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "GetQualityLevel has been deprecated."))
	int32 GetQualityLevel() const								{ return 0; }

	/**
	 * Get the ML Deformer asset that is used by this component.
	 * @return A pointer to the ML Deformer asset.
	 */
	UFUNCTION(BlueprintCallable, Category = "MLDeformer")
	UMLDeformerAsset* GetDeformerAsset() const					{ return DeformerAsset; }

	/**
	 * Set the deformer asset that is used by this component.
	 * This will trigger the internal ML Deformer instance to be recreated.
	 * @param InDeformerAsset A pointer to the deformer asset.
	 */
	UFUNCTION(BlueprintCallable, Category = "MLDeformer")
	void SetDeformerAsset(UMLDeformerAsset* InDeformerAsset) { SetDeformerAssetInternal(InDeformerAsset); }

	/**
	 * Find the skeletal mesh component to apply the deformer on.
	 * This will return the skeletal mesh component (on this actor) which uses the same skeletal mesh as the passed in ML Deformer asset was trained on.
	 * If there is no such skeletal mesh component then it will return a nullptr.
	 * @param Asset The ML Deformer asset to search a component for.
	 * @return The skeletal mesh component that would be the best.
	 */
	UFUNCTION(BlueprintCallable, Category = "MLDeformer")
	UE_API USkeletalMeshComponent* FindSkeletalMeshComponent(const UMLDeformerAsset* const Asset) const;

	/**
	 * Get the ML Deformer model instance that this component currently uses.
	 * The instance is responsible for running inference and feeding the neural network with inputs.
	 * @return A pointer to the model instance object.
	 */
	UFUNCTION(BlueprintCallable, Category = "MLDeformer")
	UMLDeformerModelInstance* GetModelInstance() const			{ return ModelInstance; }

	/**
	 * Get the skeletal mesh component that the ML Deformer will work on.
	 * The skeletal mesh that is setup inside the skeletal mesh component will be the mesh that will be deformed by this ML Deformer component.
	 * @return A pointer to the skeletal mesh component who's mesh will be deformed by this ML Deformer component.
	 */
	UFUNCTION(BlueprintCallable, Category = "MLDeformer")
	USkeletalMeshComponent* GetSkeletalMeshComponent() const	{ return SkelMeshComponent.Get(); }

	/** Find the skeletal mesh component that this deformer should work on, and set it as our target component. */
	UFUNCTION(BlueprintCallable, Category = "MLDeformer")
	UE_API void UpdateSkeletalMeshComponent();

	/**
	 * Get the final weight that is used when applying the ML Deformer.
	 * Some console command might override the weight that was set to this component. This method will
	 * return the real weight that will be applied, after console command modifications are applied as well.
	 * So this is the actual final weight used on the ML Deformer.
	 * @return The ML Deformer weight value that is used to deform the mesh, where 0 means it is not doing any deformations and 1 means it is fully active.
	 */
	UE_API float GetFinalMLDeformerWeight() const;

	/**
	 * Suppress logging warnings about mesh deformers not being set.
	 * A warning is logged when an ML Deformer is used that requires a deformer graph, but the skeletal mesh has no deformer graph setup.
	 * @param bSuppress Set to true to silent warnings about deformer graphs not being set, while the active ML Model needs one.
	 */
	void SetSuppressMeshDeformerLogWarnings(bool bSuppress)		{ bSuppressMeshDeformerLogWarnings = bSuppress; }

#if WITH_EDITOR
	/**
	 * Get the performance counter that measures how much time is spent inside the Tick function.
	 */
	const UE::MLDeformer::FMLDeformerPerfCounter& GetTickPerfCounter() const	{ return TickPerfCounter; }

	TObjectPtr<AActor> GetDebugActor() const					{ return DebugActor; }
	void SetDebugActor(TObjectPtr<AActor> Actor)				{ DebugActor = Actor; }
#endif

	// Get property names.
	static FName GetDeformerAssetPropertyName()					{ return GET_MEMBER_NAME_CHECKED(UMLDeformerComponent, DeformerAsset); }
	static FName GetWeightPropertyName()						{ return GET_MEMBER_NAME_CHECKED(UMLDeformerComponent, Weight); }

protected:
	// AActorComponent overrides.
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	// ~END AActorComponent overrides.

	/** Set the ML Deformer weight. */
	UE_API virtual void SetWeightInternal(const float NormalizedWeightValue);

	/** Set the ML Deformer asset. */
	UE_API virtual void SetDeformerAssetInternal(UMLDeformerAsset* const InDeformerAsset);

	/** 
	 * Initialize the component. 
	 * This releases any existing deformer instance that is active, and creates a new one.
	 * It then also calls PostMLDeformerComponentInit.
	 * This method is called automatically by SetupComponent.
	 */
	UE_API void Init();

	UE_API void BindDelegates();
	UE_API void UnbindDelegates();
	UE_API void ReleaseModelInstance();

protected:
#if WITH_EDITOR
	/** The performance counter that measures timing of the Tick function. */
	UE::MLDeformer::FMLDeformerPerfCounter TickPerfCounter;

	/**
	 * The actor we are currently debugging. This can be used to copy over specific information from another actor, such as copying over external morph target weights.
	 * When set to a nullptr, then we're not debugging. This actor most likely is inside another UWorld.
	 */
	TObjectPtr<AActor> DebugActor;
#endif

	/** Render command fence that let's us wait for all other commands to finish. */
	FRenderCommandFence RenderCommandFence;

	/** 
	 * The skeletal mesh component we want to grab the bone transforms etc from. 
	 * This can be a nullptr. When it is a nullptr then it will internally try to find the first skeletal mesh component on the actor.
	 * You can see this as an override. You can specify this override through the SetupComponent function.
	 */
	TObjectPtr<USkeletalMeshComponent> SkelMeshComponent;

	/** The delegate handle used to bind to the reinit model instance delegate handle. */
	FDelegateHandle ReinitModelInstanceDelegateHandle;

	/** Suppress mesh deformer logging warnings? This is used by the ML Deformer editor, as we don't want to show some warnings when using that. */
	bool bSuppressMeshDeformerLogWarnings = false;

	/** The deformer asset to use. */
	UPROPERTY(EditAnywhere, DisplayName = "ML Deformer Asset", Category = "ML Deformer")
	TObjectPtr<UMLDeformerAsset> DeformerAsset;

	/** How active is this deformer? Can be used to blend it in and out. */
	UPROPERTY(EditAnywhere, Category = "ML Deformer", meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float Weight = 1.0f;

	/**
	 * The quality level of the deformer. A value of 0 is the highest quality, 1 is a step lower, etc.
	 * It is up to the models to configure what each quality level means visually.
	 * If the quality level will be clamped to the available quality levels, so if you choose quality level 100, while there are only 3 levels, then quality
	 * level 3 will be used in this case, which represents the lowest available quality.
	 * In morph based models each quality level defines how many morph targets are active at most.
	 */
	UPROPERTY()
	int32 QualityLevel_DEPRECATED = 0;

	/** The deformation model instance. This is used to perform the runtime updates and run the inference. */
	UPROPERTY(Transient)
	TObjectPtr<UMLDeformerModelInstance> ModelInstance;
};

#undef UE_API
