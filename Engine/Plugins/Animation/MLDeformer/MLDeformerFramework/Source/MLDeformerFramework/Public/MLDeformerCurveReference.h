// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/NameTypes.h"
#include "Serialization/Archive.h"
#include "MLDeformerCurveReference.generated.h"

/** A reference to a curve implemented as a name. */
USTRUCT()
struct FMLDeformerCurveReference
{
	GENERATED_USTRUCT_BODY()

	FMLDeformerCurveReference(const FName& InCurveName=NAME_None)
		: CurveName(InCurveName)
	{
	}

	bool operator==(const FMLDeformerCurveReference& Other) const
	{
		return (CurveName == Other.CurveName);
	}

	bool operator!=(const FMLDeformerCurveReference& Other) const
	{
		return (CurveName != Other.CurveName);
	}

	friend FArchive& operator<<(FArchive& Ar, FMLDeformerCurveReference& C)
	{
		Ar << C.CurveName;
		return Ar;
	}

	bool Serialize(FArchive& Ar)
	{
		Ar << *this;
		return true;
	}

	/** The name of the curve. */
	UPROPERTY(EditAnywhere, Category = AnimCurveReference)
	FName CurveName;
};
