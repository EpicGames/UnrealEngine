// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "Math/MathFwd.h"

class UGeometryCache;
class USkeletalMesh;
class UAnimSequence;

namespace UE::MLDeformer
{
#if WITH_EDITORONLY_DATA
	/** A mapping between a geometry cache track and a mesh inside a USkeletalMesh. */
	struct FMLDeformerGeomCacheMeshMapping
	{
		int32 MeshIndex = INDEX_NONE;	// The imported model's mesh info index, inside the skeletal mesh.
		int32 TrackIndex = INDEX_NONE;	// The geometry cache track that this mesh is mapped to.
		TArray<int32> SkelMeshToTrackVertexMap;	// This maps imported model individual meshes to the geomcache track's mesh data.
		TArray<int32> ImportedVertexToRenderVertexMap; // Map the imported DCC vertex number to a render vertex. This is just one of the duplicates, which shares the same position.
		TArray<int32> MaterialIndices;	// List of material indices that point inside the SkelMesh. These materials are used on parts of the mesh that are modified by the ML Deformer.
	};

	/**
	 * Check whether the geometry cache has any issues when used in combination with a given skeletal mesh.
	 * Please note that this does not check the actual geometry data for matching vertex counts.
	 * What it does check is if the geometry cache has geometry at all, and whether it has imported vertex numbers 
	 * and whether flattened tracks was enabled, while there are multiple meshes in the skeletal mesh.
	 * @param InSkeletalMesh The skeletal mesh that is being used.
	 * @param InGeomCache The geometry cache that is being used.
	 * @return The error text in case there are errors, or an empty text object in case there were no errors.
	 */
	MLDEFORMERFRAMEWORK_API FText GetGeomCacheErrorText(USkeletalMesh* InSkeletalMesh, UGeometryCache* InGeomCache);

	/**
	 * Check for any issues between a geometry cache and an anim sequence when used together.
	 * This will check if the duration of both are the same or not.
	 * @param InGeomCache The geometry cache object.
	 * @param InAnimSequence The animation sequence object.
	 * @return The error text in case there are errors, or an empty text object in case there were no errors.
	 */
	MLDEFORMERFRAMEWORK_API FText GetGeomCacheAnimSequenceErrorText(const UGeometryCache* InGeomCache, const UAnimSequence* InAnimSequence);

	/**
	 * Check for any mesh related issues between a skeletal mesh and geometry cache.
	 * This will check if the Skeletal Mesh contains all meshes that are required for the geometry cache tracks.
	 * It also checks for vertex count mismatches.
	 * @param InSkelMesh The skeletal mesh object.
	 * @param InGeomCache The geometry cache object.
	 * @return The error text in case there are errors, or an empty text object in case there were no errors.
	 */
	MLDEFORMERFRAMEWORK_API FText GetGeomCacheMeshMappingErrorText(USkeletalMesh* InSkelMesh, UGeometryCache* InGeomCache);

	/**
	 * Get the number of imported geometry cache vertices.
	 * Imported vertices are the vertex count as you see in the DCC, so a cube would have 8.
	 * This is different from the render vertex count.
	 * @param GeometryCache The geometry cache to get the imported vertex count for.
	 * @return The number of vertices as seen in the DCC, so for example 8 for a cube.
	 */
	MLDEFORMERFRAMEWORK_API int32 ExtractNumImportedGeomCacheVertices(UGeometryCache* GeometryCache);

	/**
	 * Generate the mapping between geometry cache tracks and meshes inside a specific skeletal mesh.
	 * This will check if the Skeletal Mesh contains all meshes that are required for the geometry cache tracks.
	 * It also checks for vertex count mismatches.
	 * This function is called by the GetGeomCacheMeshMappingErrorText function, in case you are just interested in the error checking part.
	 * @param SkelMesh The skeletal mesh object.
	 * @param GeomCache The geometry cache object.
	 * @param OutMeshMapping The mapping that will be generated. This information maps each geometry cache map with a given compatible mesh inside a skeletal mesh.
	 * @param OutFailedImportedMeshNames The array of geometry cache track names for which no mesh could be found inside the skeletal mesh object.
	 * @param OutVertexMisMatchNames The array of geometry cache track names for which the vertex counts didn't match the related meshes inside the skeletal mesh.	 
	 * @param bSuppressLog Set to true to suppress any log messages.
	 */
	MLDEFORMERFRAMEWORK_API void GenerateGeomCacheMeshMappings(USkeletalMesh* SkelMesh, UGeometryCache* GeomCache, TArray<FMLDeformerGeomCacheMeshMapping>& OutMeshMappings, TArray<FString>& OutFailedImportedMeshNames, TArray<FString>& OutVertexMisMatchNames, bool bSuppressLog = false);

	/**
	 * Sample the vertex position data of a geometry cache, at a given time stamp.
	 * This basically allows you to sample ground truth or training target vertex positions.
	 * @param InLODIndex The LOD level to sample for.
	 * @param FrameIndex The frame number to sample.
	 * @param InMeshMappings The geometry cache to skeletal mesh mappings, which can be generated using the GenerateGeomCacheMeshMappings method.
	 * @param InSkelMesh The skeletal mesh object.
	 * @param InGeometryCache The geometry cache object.
	 * @param InAlignmentTransform This is the transformation that will be applied on the sampled positions, as post process. This can be used to rotate or scale the position data.
	 * @param OutPositions The resulting positions, as sampled at the specified time. This array will be resized internally.
	 */
	MLDEFORMERFRAMEWORK_API void SampleGeomCachePositionsAtFrame(
		int32 InLODIndex,
		int32 FrameIndex,
		const TArray<FMLDeformerGeomCacheMeshMapping>& InMeshMappings,
		const USkeletalMesh* InSkelMesh,
		const UGeometryCache* InGeometryCache,
		const FTransform& InAlignmentTransform,
		TArray<FVector3f>& OutPositions);
#endif	// #if WITH_EDITORONLY_DATA
}	// namespace UE::MLDeformer
