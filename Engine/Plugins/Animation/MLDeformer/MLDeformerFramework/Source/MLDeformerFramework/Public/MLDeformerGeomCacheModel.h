// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerModel.h"
#include "MLDeformerVizSettings.h"
#include "MLDeformerInputInfo.h"
#include "MLDeformerGeomCacheHelpers.h"
#include "MLDeformerGeomCacheTrainingInputAnim.h"
#include "GeometryCache.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "MLDeformerGeomCacheModel.generated.h"

#define UE_API MLDEFORMERFRAMEWORK_API

class UMLDeformerAsset;
class UMLDeformerGeomCacheVizSettings;

/**
 * An ML Deformer model that has a geometry cache as target mesh.
 * Use this in combination with UMLDeformerGeomCacheVizSettings, FMLDeformerGeomCacheEditorModel, FMLDeformerGeomCacheVizSettingsDetails and FMLDeformerGeomCacheModelDetails.
 */
UCLASS(MinimalAPI)
class UMLDeformerGeomCacheModel
	: public UMLDeformerModel
{
	GENERATED_BODY()

public:
	// UObject overrides.
	UE_API virtual void PostLoad() override;
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	// ~END UObject overrides.

	// UMLDeformerModel overrides.
#if WITH_EDITORONLY_DATA
	UE_API virtual bool HasTrainingGroundTruth() const override;
	UE_API virtual void SampleGroundTruthPositionsAtFrame(int32 FrameIndex, TArray<FVector3f>& OutPositions) override;
	UE_API virtual void SampleGroundTruthPositions(float SampleTime, TArray<FVector3f>& OutPositions) override;	// Deprecated in base class, will be removed soon.
#endif
#if WITH_EDITOR
	UE_API virtual void UpdateNumTargetMeshVertices() override;
	// ~END UMLDeformerModel overrides.

	/**
	 * Get the visualization settings, already cast to a UMLDeformerGeomCacheVizSettings type.
	 * @return A pointer to the visualization settings.
	 */
	UE_API UMLDeformerGeomCacheVizSettings* GetGeomCacheVizSettings() const;
#endif

#if WITH_EDITORONLY_DATA
	/**
	 * Get the mapping between geometry cache tracks and meshes inside the skeletal mesh.
	 * This lets us know what parts of the skeletal mesh are related to what geometry cache tracks.
	 * Once we have that, we can calculate deltas between the two.
	 * @return The geometry cache mesh to skeletal mesh mappings.
	 */
	TArray<UE::MLDeformer::FMLDeformerGeomCacheMeshMapping>& GetGeomCacheMeshMappings()				{ return MeshMappings; }
	const TArray<UE::MLDeformer::FMLDeformerGeomCacheMeshMapping>& GetGeomCacheMeshMappings() const { return MeshMappings; }

	TArray<FMLDeformerGeomCacheTrainingInputAnim>& GetTrainingInputAnims()							{ return TrainingInputAnims; }
	const TArray<FMLDeformerGeomCacheTrainingInputAnim>& GetTrainingInputAnims() const				{ return TrainingInputAnims; }

	static FName GetTrainingInputAnimsPropertyName()		{ return GET_MEMBER_NAME_CHECKED(UMLDeformerGeomCacheModel, TrainingInputAnims); }

private:
	/** The mappings between the geometry cache tracks and skeletal mesh imported meshes. */
	TArray<UE::MLDeformer::FMLDeformerGeomCacheMeshMapping> MeshMappings;

	/** The geometry cache that represents the target deformations. */
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use the training input anims instead."))
	TSoftObjectPtr<UGeometryCache> GeometryCache_DEPRECATED;

	/** The training animation inputs. */
	UPROPERTY(EditAnywhere, Category = "Inputs")
	TArray<FMLDeformerGeomCacheTrainingInputAnim> TrainingInputAnims;
#endif // WITH_EDITORONLY_DATA
};

#undef UE_API
