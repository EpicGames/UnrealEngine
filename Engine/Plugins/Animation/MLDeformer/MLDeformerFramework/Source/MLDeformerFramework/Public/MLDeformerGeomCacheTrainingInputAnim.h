// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/SoftObjectPtr.h"
#include "MLDeformerTrainingInputAnim.h"
#include "GeometryCache.h"
#include "MLDeformerGeomCacheTrainingInputAnim.generated.h"

#define UE_API MLDEFORMERFRAMEWORK_API

/**
 * An animation that is input to the training process, which has a geometry cache as target.
 */
USTRUCT()
struct FMLDeformerGeomCacheTrainingInputAnim
	: public FMLDeformerTrainingInputAnim
{
	GENERATED_BODY()

public:
#if WITH_EDITORONLY_DATA
	// FMLDeformerTrainingInputAnim overrides
	UE_API virtual int32 GetNumFramesToSample() const override;
	UE_API virtual bool IsValid() const override;
	UE_API virtual int32 ExtractNumAnimFrames() const override;
	// ~END FMLDeformerTrainingInputAnim overrides

	void SetGeometryCache(TSoftObjectPtr<UGeometryCache> GeomCache)				{ GeometryCache = GeomCache; }

	const UGeometryCache* GetGeometryCache() const								{ return GeometryCache.LoadSynchronous(); }
	UGeometryCache* GetGeometryCache()											{ return GeometryCache.LoadSynchronous(); }
	TSoftObjectPtr<UGeometryCache>& GetGeometryCacheSoftObjectPtr()				{ return GeometryCache; }
	const TSoftObjectPtr<UGeometryCache>& GetGeometryCacheSoftObjectPtr() const { return GeometryCache; }
	static FName GetGeomCachePropertyName()										{ return GET_MEMBER_NAME_CHECKED(FMLDeformerGeomCacheTrainingInputAnim, GeometryCache); }

private:
	/** The geometry cache which contains the target deformations. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (DisplayPriority = 10))
	TSoftObjectPtr<UGeometryCache> GeometryCache;
#endif	// #if WITH_EDITORONLY_DATA
};

#undef UE_API
