// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "GeometryCache.h"
#include "MLDeformerVizSettings.h"
#include "MLDeformerGeomCacheVizSettings.generated.h"

/**
 * The visualization settings for a model that has a geometry cache.
 * This can be used in combination with a UMLDeformerGeomCacheModel.
 */
UCLASS(MinimalAPI)
class UMLDeformerGeomCacheVizSettings
	: public UMLDeformerVizSettings
{
	GENERATED_BODY()

public:
#if WITH_EDITORONLY_DATA
	// UMLDeformerModel overrides.
	virtual bool HasTestGroundTruth() const override	{ return (GetTestGroundTruth() != nullptr); }
	// ~END UMLDeformerModel overrides.

	/**
	 * Get the test ground truth geometry cache, which represents the ground truth version of the test animation sequence.
	 * @return A pointer to the geometry cache object.
	 */
	UGeometryCache* GetTestGroundTruth() const			{ return GroundTruth.LoadSynchronous(); }

	/**
	 * Set the test ground truth geometry cache.
	 * @param InGroundTruth The ground truth model.
	 */
	void SetTestGroundTruth(TSoftObjectPtr<UGeometryCache> InGroundTruth)	{ GroundTruth = InGroundTruth; }

	// Get property names.
	static FName GetTestGroundTruthPropertyName()		{ return GET_MEMBER_NAME_CHECKED(UMLDeformerGeomCacheVizSettings, GroundTruth); }

protected:
	/** The geometry cache that represents the ground truth of the test anim sequence. */
	UPROPERTY(EditAnywhere, Category = "Test Assets")
	TSoftObjectPtr<UGeometryCache> GroundTruth = nullptr;
#endif // WITH_EDITORONLY_DATA
};
