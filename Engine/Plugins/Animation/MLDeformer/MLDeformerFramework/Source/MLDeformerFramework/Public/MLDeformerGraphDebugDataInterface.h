// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerModel.h"
#include "OptimusComputeDataInterface.h"
#include "ComputeFramework/ComputeDataProvider.h"
#include "PrimitiveComponentId.h"
#include "MLDeformerGraphDebugDataInterface.generated.h"

#define UE_API MLDEFORMERFRAMEWORK_API

class FRDGBuffer;
class FRDGBufferSRV;
class FRHIShaderResourceView;
class FSceneInterface;
class FSkeletalMeshObject;
class UMLDeformerAsset;
class USkeletalMeshComponent;

/** 
 * Compute Framework Data Interface for MLDeformer debugging data. 
 * This interfaces to editor only data, and so will only give valid results in that context.
 */
UCLASS(MinimalAPI, Category = ComputeFramework)
class UMLDeformerGraphDebugDataInterface
	: public UOptimusComputeDataInterface
{
	GENERATED_BODY()

public:
	// UOptimusComputeDataInterface overrides.
	UE_API virtual FString GetDisplayName() const override;
	UE_API virtual TArray<FOptimusCDIPinDefinition> GetPinDefinitions() const override;
	UE_API virtual TSubclassOf<UActorComponent> GetRequiredComponentClass() const override;
	// ~END UOptimusComputeDataInterface overrides.

	// UComputeDataInterface overrides.
	TCHAR const* GetClassName() const override { return TEXT("MLDeformerGraphDebugData"); }
	virtual bool CanSupportUnifiedDispatch() const override { return true; }
	UE_API virtual void GetSupportedInputs(TArray<FShaderFunctionDefinition>& OutFunctions) const override;
	UE_API virtual void GetShaderParameters(TCHAR const* UID, FShaderParametersMetadataBuilder& OutBuilder, FShaderParametersMetadataAllocations& InOutAllocations) const override;
	UE_API virtual TCHAR const* GetShaderVirtualPath() const override;
	UE_API virtual void GetShaderHash(FString& InOutKey) const override;
	UE_API virtual void GetHLSL(FString& OutHLSL, FString const& InDataInterfaceName) const override;
	UE_API virtual UComputeDataProvider* CreateDataProvider(TObjectPtr<UObject> InBinding, uint64 InInputMask, uint64 InOutputMask) const override;
	// ~END UComputeDataInterface overrides.

private:
	static UE_API TCHAR const* TemplateFilePath;
};

/** Compute Framework Data Provider for MLDeformer debugging data. */
UCLASS(MinimalAPI, BlueprintType, EditInlineNew, Category = ComputeFramework)
class UMLDeformerGraphDebugDataProvider
	: public UComputeDataProvider
{
	GENERATED_BODY()

public:
	virtual void Init() {};

	// UComputeDataProvider overrides.
	UE_API virtual FComputeDataProviderRenderProxy* GetRenderProxy() override;
	// ~END UComputeDataProvider overrides.

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Binding)
	TObjectPtr<UMLDeformerComponent> DeformerComponent = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Binding)
	TObjectPtr<UMLDeformerAsset> DeformerAsset = nullptr;
};

/** Compute Framework Data Provider Proxy for MLDeformer debugging data. */
#if WITH_EDITORONLY_DATA
namespace UE::MLDeformer
{
	class FMLDeformerGraphDebugDataProviderProxy
		: public FComputeDataProviderRenderProxy
	{
	public:
		UE_API FMLDeformerGraphDebugDataProviderProxy(UMLDeformerComponent* DeformerComponent, UMLDeformerAsset* DeformerAsset, UMLDeformerGraphDebugDataProvider* InProvider);

		UE_API virtual void HandleZeroGroundTruthPositions();

		// FComputeDataProviderRenderProxy overrides.
		UE_API bool IsValid(FValidationData const& InValidationData) const override;
		UE_API void AllocateResources(FRDGBuilder& GraphBuilder, FAllocationData const& InAllocationData) override;
		UE_API void GatherDispatchData(FDispatchData const& InDispatchData) override;
		// ~END FComputeDataProviderRenderProxy overrides.

		TArray<FVector3f>& GetGroundTruthPositions() { return GroundTruthPositions; }

	protected:
		TObjectPtr<UMLDeformerGraphDebugDataProvider> Provider = nullptr;
		FSceneInterface* SceneInterface = nullptr;
		FPrimitiveComponentId ComponentId;
		TArray<FVector3f> GroundTruthPositions;
		FRHIShaderResourceView* VertexMapBufferSRV = nullptr;
		FRDGBuffer* GroundTruthBuffer = nullptr;
		FRDGBufferSRV* GroundTruthBufferSRV = nullptr;
		int32 HeatMapMode = 0;
		float HeatMapMax = 0.0f;
		float GroundTruthLerp = 0.0f;
	};
}	// namespace UE::MLDeformer
#endif

#undef UE_API
