// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerModule.h"
#include "MLDeformerInputInfo.generated.h"

#define UE_API MLDEFORMERFRAMEWORK_API

class USkeletalMesh;
class USkeletalMeshComponent;

/**
 * The neural network input information.
 * This contains arrays of names for things such as bones and curves.
 * Knowing what bones etc are used as inputs, and in what order, helps us feeding the data during inference.
 * It can also help us detect issues, for example when the character we apply the deformer to is missing any of those bones.
 */
UCLASS(MinimalAPI)
class UMLDeformerInputInfo
	: public UObject
{
	GENERATED_BODY()

public:
	virtual ~UMLDeformerInputInfo() = default;

	// UObject overrides.
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	// ~END UObject overrides.

	/**
	 * This method is executed post loading of the ML Deformer asset.
	 */
	UE_API virtual void OnPostLoad();

	/**
	 * Check whether the current inputs are compatible with a given skeletal mesh.
	 * For a more extensive check, please use IsCompatible that takes a USkeletalMeshComponent as parameter.
	 * @param InSkeletalMesh The skeletal mesh to check compatibility with. This may not be a nullptr.
	 * @return Returns true when we can safely apply the ML Deformer to a character using this skeletal mesh, otherwise false is returned.
	 * @note Use GenerateCompatibilityErrorString to get the error report.
	 */
	UE_API virtual bool IsCompatible(USkeletalMesh* InSkeletalMesh) const;

	/**
	 * Check whether the current inputs are compatible with a given skeletal mesh component.
	 * @param InSkeletalMeshComponent The skeletal mesh to check compatibility with. This may not be a nullptr.
	 * @return Returns true when we can safely apply the ML Deformer to a character using this skeletal mesh, otherwise false is returned.
	 * @note Use GenerateCompatibilityErrorString to get the error report.
	 */
	UE_API virtual bool IsCompatible(const USkeletalMeshComponent* InSkeletalMeshComponent) const;

	/**
	 * Clear all contents.
	 * This can clear the list of all bone and curves that are part of this input info.
	 */
	UE_API virtual void Reset();

	/**
	 * Get the compatibility error report.
	 * @param InSkeletalMesh The skeletal mesh to check compatibility with.
	 * @return Returns an empty string in case there are no compatibility issues, otherwise it contains a string that describes the issue(s).
	 *         In case a nullptr is passed as SkeletalMesh parameter, an empty string is returned.
	 */
	UE_DEPRECATED(5.7, "Please use the GenerateCompatibilityErrorString that takes a USkeletalMeshComponent as parameter instead.")
	UE_API virtual FString GenerateCompatibilityErrorString(USkeletalMesh* InSkeletalMesh) const;
	
	/**
	 * Get the compatibility error report.
	 * @param InSkeletalMeshComponent The skeletal mesh component to check compatibility with.
	 * @return Returns an empty string in case there are no compatibility issues, otherwise it contains a string that describes the issue(s).
	 *         In case a nullptr is passed as SkeletalMeshComponent parameter, an empty string is returned.
	 */
	UE_API virtual FString GenerateCompatibilityErrorString(const USkeletalMeshComponent* InSkeletalMeshComponent) const;

	/**
	 * Check whether we have any training inputs or not.
	 * This happens when there are no bones or curves to use as inputs.
	 * @return Returns true when there are no bones or curves specified as inputs.
	 */
	UE_API virtual bool IsEmpty() const;

	/**
	 * Calculate how many inputs this input info generates for the neural network.
	 * @param NumFloatsPerBone The number of floats used per bone.
	 * @param NumFloatsPerCurve The number of floats per curve. This can be used when padding is used on curve values.
	 * @return THe number of input float values to the neural network.
	 */
	UE_API virtual int32 CalcNumNeuralNetInputs(int32 NumFloatsPerBone, int32 NumFloatsPerCurve) const;

	/**
	 * Copy members from the other class.
	 * @param Other The other class to copy over members from.
	 **/
	UE_API virtual void CopyMembersFrom(UMLDeformerInputInfo* Other);

	/**
	 * Get the number of bones that we trained on.
	 * @return The number of bones.
	 */
	UE_API int32 GetNumBones() const;

	/**
	 * Get the bone name as an FName, for a given bone we included during training.
	 * @param Index The bone index, which is a number in range of [0..GetNumBones()-1].
	 * @result The name of the bone.
	 */
	UE_API FName GetBoneName(int32 Index) const;

	/**
	 * Get the number of curves that we trained on.
	 * @return The number of curves.
	 */
	UE_API int32 GetNumCurves() const;

	/**
	 * Get the list of bone names that should be included as training inputs.
	 * @return An array of FName objects, one for each bone.
	 */
	UE_API TArray<FName>& GetBoneNames();
	UE_API const TArray<FName>& GetBoneNames() const;

	/**
	 * Get the list of curve names that should be included as training inputs.
	 * @return An array of FName objects, one for each curve.
	 */
	UE_API TArray<FName>& GetCurveNames();
	UE_API const TArray<FName>& GetCurveNames() const;

	/**
	 * Get the curve name as an FName, for a given curve we included during training.
	 * @param Index The curve index, which is a number in range of [0..GetNumCurves()-1].
	 * @result The name of the curve.
	 */
	UE_API FName GetCurveName(int32 Index) const;

	/**
	 * Extract the curve values for all curves we're interested in.
	 * @param SkelMeshComponent The skeletal mesh component to sample from.
	 * @param OutValues The array to write the values to. This array will be reset/resized by this method.
	 * @param NumFloatsPerCurve The number of floats per curve. If larger than 1, the remaining floats (after the first one), will be set to 0.
	 *        So if this is set to 4, you get an array like "0.75, 0, 0, 0", where 0.75 would represent the actual curve value.
	 */
	UE_API virtual void ExtractCurveValues(USkeletalMeshComponent* SkelMeshComponent, TArray<float>& OutValues, int32 NumFloatsPerCurve) const;

	/**
	 * Batch convert a set of transform rotation quaternions into two column vectors.
	 * 
	 */
	static UE_API void RotationToTwoVectorsAsSixFloats(TArrayView<FTransform> Transforms, float* OutputBuffer);

	/**
	 * Convert a rotation quaternion to two basis vectors, each represented as 3 floats, so 6 floats in total.
	 * @param Rotation The rotation quaternion to convert into the six float values.
	 * @param SixFloatsOutputBuffer The float buffer that we will write the 6 floats to. Make sure this buffer is at least 6 floats large.
	 */
	static UE_API void RotationToTwoVectorsAsSixFloats(const FQuat& Rotation, float* SixFloatsOutputBuffer);

	/**
	 * Extract bone space rotations, as a float array.
	 * The number of output rotations are NumBones * 6, where the array contains a set two columns of a 3x3 rotation matrix values.
	 * components of the bone's bone space (local space) rotation quaternion.
	 * @param SkelMeshComponent The skeletal mesh component to sample from.
	 * @param OutRotations The output rotation values. This array will be resized internally.
	 */
	UE_API virtual void ExtractBoneRotations(USkeletalMeshComponent* SkelMeshComponent, TArray<float>& OutRotations) const;

	/**
	 * Get the number of imported vertices in the base mesh, which is the linear skinned skeletal mesh.
	 * @return The number of imported vertices in the base mesh.
	 */
	UE_API int32 GetNumBaseMeshVertices() const;

	/**
	 * Set the number of vertices of our base mesh, which is the linear skinned skeletal mesh.
	 * The vertex count must be the same as in the DCC, so not the render mesh vertex count.
	 * @param NumVerts The number of vertices of our base mesh.
	 */
	UE_API void SetNumBaseVertices(int32 NumVerts);

	/**
	 * Get the number of imported vertices in the target mesh, which is our training target/ground truth mesh.
	 * The number of vertices is the vertex count as in the DCC.
	 * @return The number of imported vertices in the target mesh.
	 */
	UE_API int32 GetNumTargetMeshVertices() const;

	/**
	 * Set the number of target mesh vertices. The target mesh is our training target/ground truth mesh.
	 * The vertex count must be the same as in the DCC, so not the render mesh vertex count.
	 * @param NumVerts The number of vertices in the target mesh.
	 */
	UE_API void SetNumTargetVertices(int32 NumVerts);

	/** 
	 * Get the path to the skeletal mesh that this deformer was trained on.
	 * @return The path to the skeletal mesh we used during training.
	 */
	UE_API const FSoftObjectPath& GetSkeletalMesh() const;

	/**
	 * Set skeletal mesh that we trained on.
	 * @param InSkeletalMesh A pointer to the skeletal mesh.
	 */
	UE_API void SetSkeletalMesh(USkeletalMesh* InSkeletalMesh);

	/**
	 * Builds a list of available curves on the Skeletal Mesh provided, including its skeleton.
	 * This internally adds a merged set of curves that are found on the skeleton and the skeletal mesh itself.
	 * @param InSkeletalMesh The Skeletal Mesh to get the list of curves for.
	 * @param OutCurveNames The list of curve names that have been found.
	 */
	UE_API static void BuildCurveList(USkeletalMesh* InSkeletalMesh, TArray<FName>& OutCurveNames);


protected:
	/**
	 * The path to the skeletal mesh that this model was trained on.
	 */
	UPROPERTY()
	FSoftObjectPath SkeletalMesh;

#if WITH_EDITORONLY_DATA
	/** The list of bone names, but as string. This is deprecated since UE 5.2. */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use BoneNames instead."))
	TArray<FString> BoneNameStrings_DEPRECATED;

	/** The list of curve names, but as string. This is deprecated since UE 5.2. */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use CurveNames instead."))
	TArray<FString> CurveNameStrings_DEPRECATED;
#endif

	/** 
	 * The name of each bone. The inputs to the network are in the order of this array.
	 * So if the array contains ["Root", "Child1", "Child2"] then the first bone transforms that we 
	 * input to the neural network is the transform for "Root", followed by "Child1", followed by "Child2".
	 */
	UPROPERTY()
	TArray<FName> BoneNames;

	/**
	 * The name of each curve. The inputs to the network are in the order of this array.
	 * So if the array contains ["Smile", "LeftEyeClosed", "RightEyeClosed"] then the first curve that we
	 * input to the neural network is the one for "Smile", followed by "LeftEyeClosed", followed by "RightEyeClosed".
	 */
	UPROPERTY()
	TArray<FName> CurveNames;

	/** Number of imported base mesh vertices, so not render vertices. */
	UPROPERTY(AssetRegistrySearchable)
	int32 NumBaseMeshVertices = 0;

	/** Number of imported target mesh vertices, so not render vertices. */
	UPROPERTY()
	int32 NumTargetMeshVertices = 0;
};

#undef UE_API
