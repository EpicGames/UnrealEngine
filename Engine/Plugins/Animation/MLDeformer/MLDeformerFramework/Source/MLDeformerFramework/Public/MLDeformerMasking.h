// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "MLDeformerMasking.generated.h"


/** Specifies whether a mask should be generated or not. */
UENUM()
enum class EMLDeformerMaskingMode : uint8
{
	/** Use a generated mask, generated from the skinning weights of a set of bones. */
	Generated,

	/** Use a vertex attribute on the skeletal mesh. This can be a manually painted map. */
	VertexAttribute
};


/**
 * Information needed to generate the mask for a specific bone.
 * This includes a list of bone names. Each bone has a skinning influence mask.
 * The final mask will be a merge of all the bone masks of bones listed inside this info struct.
 * There will be an array of these structs, one for each bone.
 */
USTRUCT()
struct FMLDeformerMaskInfo
{
	GENERATED_BODY()

public:
	/** The list of bone names that should be included in the mask generation. */
	UPROPERTY()
	TArray<FName> BoneNames;

	/** The masking mode. */
	UPROPERTY()
	EMLDeformerMaskingMode MaskMode = EMLDeformerMaskingMode::Generated;

	/** If the masking mode is set to VertexAttribute then we can check which attribute to use using this member. */
	UPROPERTY()
	FName VertexAttributeName;
};
