// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPtr.h"
#include "Interfaces/Interface_BoneReferenceSkeletonProvider.h"
#include "BoneContainer.h"
#include "RenderCommandFence.h"
#include "RenderResource.h"
#include "Animation/AnimSequence.h"
#include "MLDeformerCurveReference.h"
#include "PerQualityLevelProperties.h"
#include "MLDeformerModel.generated.h"

#define UE_API MLDEFORMERFRAMEWORK_API

class UMLDeformerAsset;
class UMLDeformerVizSettings;
class UMLDeformerModelInstance;
class UMLDeformerComponent;
class UMLDeformerInputInfo;
class UMLDeformerTrainingDataProcessorSettings;

/** The channel to get the mask data from. */
UENUM()
enum class EMLDeformerMaskChannel : uint8
{
	/** Disable the weight mask. */
	Disabled,

	/** The red vertex color channel. */
	VertexColorRed,

	/** The green vertex color channel. */
	VertexColorGreen,

	/** The blue vertex color channel. */
	VertexColorBlue,

	/** The alpha vertex color channel. */
	VertexColorAlpha,

	/** Use a set of vertex attributes on the skeletal mesh. You can create and editor those using the Skeletal Mesh editor plugin. */
	VertexAttribute
};

/** The skinning mode to use as base. */
UENUM()
enum class EMLDeformerSkinningMode : int8
{
	/** Linear blend skinning. This is the fastest at runtime, but can have a harder time reconstructing the ground truth. */
	Linear,

	/** Dual quaternion skinning. This is slower at runtime, but can result in better deformations. */
	DualQuaternion
};


namespace UE::MLDeformer
{
	/**
	 * The vertex map, but in a GPU buffer.
	 * This map basically has a DCC vertex number for every render vertex.
	 * So if a cube requires 32 render vertices, there will be 32 ints inside this buffer, and each item in this buffer
	 * will in this specific example case contain a value between 0 and 7, as a cube has only 8 vertices.
	 */
	class FVertexMapBuffer
		: public FVertexBufferWithSRV
	{
	public:
		/**
		 * Initialize the GPU buffer based on some array with integers.
		 * @param InVertexMap The array of ints we want to store on the GPU. The size of this array should be equal to the number of render vertices of the skeletal mesh.
		 */
		void Init(const TArray<int32>& InVertexMap)	{ VertexMap = InVertexMap; }

	private:
		/**
		 * This does the actual render resource init, which means this creates and fills the buffer on the GPU.
		 * After it successfully initializes, it will empty our VertexMap member array to not store the data in both GPU memory and main memory.
		 */
		UE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

		/** The array of integers we want to store on the GPU. This buffer will be emptied after successfully calling InitRHI. */
		TArray<int32> VertexMap;
	};

}	// namespace UE::MLDeformer

// DEPRECATED: Use FMLDeformerReinitModelInstancesDelegate instead.
DECLARE_EVENT_OneParam(UMLDeformerModel, FMLDeformerModelOnPostEditProperty, FPropertyChangedEvent&)
DECLARE_EVENT_OneParam(UMLDeformerModel, FMLDeformerModelOnPostTransacted, const FTransactionObjectEvent&)
DECLARE_EVENT(UMLDeformerModel, FMLDeformerModelOnPreEditUndo)
DECLARE_EVENT(UMLDeformerModel, FMLDeformerModelOnPostEditUndo)

/**
 * The ML Deformer runtime model base class.
 * All models should be inherited from this class.
 **/
UCLASS(MinimalAPI, Abstract)
class UMLDeformerModel 
	: public UObject	
	, public IBoneReferenceSkeletonProvider
{
	GENERATED_BODY()

public:
	/** Delegate used to signal that the UMLDeformerModelInstance should be reinitialized. The ML Deformer component will connect to this to be informed about needing to reinit the instance. */
	DECLARE_MULTICAST_DELEGATE(FMLDeformerReinitModelInstancesDelegate)

	virtual ~UMLDeformerModel() = default;

	/**
	 * Initialize the ML Deformer model.
	 * This will update the DeformerAsset and InputInfo properties. It internally calls CreateInputInfo if no InputInfo has been set yet.
	 * @param InDeformerAsset The deformer asset that this model will be part of.
	 */
	UE_API virtual void Init(UMLDeformerAsset* InDeformerAsset);

	/**
	 * Initialize the data that should be stored on the GPU. 
	 * This base class will store the VertexMap on the GPU by initializing the VertexMapBuffer member.
	 */
	UE_API virtual void InitGPUData();

	/**
	 * Create the input info for this model.
	 * @return A pointer to the newly created input info.
	 */
	UE_API virtual UMLDeformerInputInfo* CreateInputInfo();

	/**
	 * Create a new instance of this model, to be used in combination with a specific component.
	 * @param Component The ML Deformer component that will own this instance.
	 * @return A pointer to the model instance.
	 */
	UE_API virtual UMLDeformerModelInstance* CreateModelInstance(UMLDeformerComponent* Component);

	/**
	 * Get the display name of this model.
	 * This will also define with what name this model will appear inside the UI.
	 * On default this will return the class name.
	 * @return The name of the model.
	 */
	UE_API virtual FString GetDisplayName() const;

	/**
	 * Defines whether this model supports bone transforms as input or not.
	 * On default this is set to return true as most models have bone rotations as inputs to the neural network.
	 * @result Returns true when this model supports bones, or false otherwise.
	 */
	virtual bool DoesSupportBones() const					{ return true; }

	/**
	 * Defines whether this model supports curves as inputs or not. A curve is just a single float value.
	 * On default this returns true.
	 * @result Returns true when this model supports curves, or false otherwise.
	 */
	virtual bool DoesSupportCurves() const					{ return true; }

	/**
	 * Check if this model supports LOD.
	 * When this returns false, the UI will not show options to setup the maximum number of LOD levels.
	 * @return Returns true if LOD is supported, otherwise false is returned, which means it only works on LOD0.
	 */
	virtual bool DoesSupportLOD() const						{ return false; }

	/**
	 * Check whether the neural network of this model should run on the GPU or not.
	 * This is false on default, which makes it a CPU based neural network.
	 * Some code internally that creates and initializes the neural network will use the return value of this method to mark it to be on CPU or GPU.
	 * @return Returns true if the neural network of this model should run on the GPU. False is returned when it should run on the CPU.
	 */
	virtual bool IsNeuralNetworkOnGPU() const				{ return false; }	// CPU neural network.

	/**
	 * Get the default deformer graph asset path that this model uses, or an empty string if it doesn't require any deformer graph.
	 * An example is some string like: "/DeformerGraph/Deformers/DG_LinearBlendSkin_Morph_Cloth_RecomputeNormals.DG_LinearBlendSkin_Morph_Cloth_RecomputeNormals".
	 * @return The asset path of the deformer graph that should be used on default. This can return an empty string when no deformer graph is required.
	 */
	virtual FString GetDefaultDeformerGraphAssetPath() const	{ return FString(); }

	/** 
	 * Get the number of floats used to represent a single bone rotation, used as input to the neural networks.
	 * @return The number of floats per individual bone.
	 */
	virtual int32 GetNumFloatsPerBone() const				{ return 6; }

	/** 
	 * Get the number of floats used to represent a single bone rotation, used as input to the neural networks.
	 * @return The number of floats per individual bone.
	 */
	virtual int32 GetNumFloatsPerCurve() const				{ return 1; }

	/** Check whether this model has been trained or not. */
	virtual bool IsTrained() const							{ ensureMsgf(false, TEXT("Please override the UMLDeformerModel::IsTrained() inside your model.")); return false; }

	/**
	 * Get the skeletal mesh that is used during training.
	 * You typically want to apply the ML Deformer on this specific skeletal mesh in your game as well.
	 * @return A pointer to the skeletal mesh.
	 */
	const USkeletalMesh* GetSkeletalMesh() const			{ return SkeletalMesh.Get(); }
	USkeletalMesh* GetSkeletalMesh()						{ return SkeletalMesh.Get(); }

	/**
	 * Set the skeletal mesh that this deformer uses.
	 * @param SkelMesh The skeletal mesh.
	 */
	void SetSkeletalMesh(USkeletalMesh* SkelMesh)			{ SkeletalMesh = SkelMesh; }

	/**
	 * Check if a given actor would be a compatible debugging actor.
	 * We check this by checking if it uses the same skeletal mesh and whether it uses the same ML Deformer asset.
	 * Debugging allows us to copy over the morph weights and pose of a character. But for that to be possible we need to 
	 * make sure the other actor uses the same skeletal mesh and ML Deformer asset. This method helps us check that easily.
	 * @param Actor The actor to check compatibility with.
	 * @param OutDebugComponent A pointer to the ML Deformer component that we would be debugging. Can be set to nullptr to get it ignored.
	 * @return Returns true when the provided actor is compatible for debugging, otherwise false is returned.
	 */
	UE_API virtual bool IsCompatibleDebugActor(const AActor* Actor, UMLDeformerComponent** OutDebugComponent = nullptr) const;

	/** 
	 * Get the maximum number of LOD levels that we will generate.
 	 * Some examples:
	 * A value of 1 means we only store one LOD, which means LOD0. 
	 * A value of 2 means we support this ML Deformer on LOD0 and LOD1.
	 * A value of 3 means we support this ML Deformer on LOD0 and LOD1 and LOD2.
	 * We never generate more LOD levels for the ML Deformer than number of LOD levels in the Skeletal Mesh, so if 
	 * this value is set to 100, while the Skeletal Mesh has only 4 LOD levels, we will only generate and store 4 ML Deformer LODs.
	 * The default value of 1 means we do not support this ML Deformer at LOD levels other than LOD0.
	 * @ return The maximum number of LOD levels we will generate.
	 */
	const int32 GetMaxNumLODs() const		{ return MaxNumLODs; }

#if WITH_EDITORONLY_DATA
	/**
	 * Check whether this model currently has a training mesh setup or not.
	 * For example if there is say a GeometryCache as target mesh, it could check whether that property is a nullptr or not.
	 * @return Returns true when the training target mesh has been selected, otherwise false is returned.
	 */
	virtual bool HasTrainingGroundTruth() const				{ return false; }

	UE_DEPRECATED(5.5, "Please use SampleGroundTruthPositionAtFrame instead.")
	virtual void SampleGroundTruthPositions(float SampleTime, TArray<FVector3f>& OutPositions) {}

	/**
	 * Sample the positions from the target (ground truth) mesh, at a specific frame.
	 * @param FrameIndex The frame number to sample the positions at.
	 * @param OutPositions The array that will receive the resulting vertex positions. This array will automatically be resized internally.
	 */
	virtual void SampleGroundTruthPositionsAtFrame(int32 FrameIndex, TArray<FVector3f>& OutPositions) {}
#endif

#if WITH_EDITOR
	/** Get the attribute names of all vertex attributes on the skeletal mesh. This only includes ones that aren't auto generated by the engine. */
	UFUNCTION()
	UE_API TArray<FName> GetVertexAttributeNames() const;

	/**
	 * Update the cached number of vertices of both base and target meshes.
	 */ 
	UE_API virtual void UpdateCachedNumVertices();

	/**
	 * Update the cached number of vertices in the base mesh. 
	 * The number of vertices should be the number of DCC vertices, so not the number of render vertices.
	 * This just updates the NumBaseMeshVerts property.
	 */ 
	UE_API virtual void UpdateNumBaseMeshVertices();

	/**
	 * Update the cached number of target mesh vertices. Every model needs to implement this.
	 * The number of vertices should be the number of DCC vertices, so not the number of render vertices.
	 * This just updates the NumTargetMeshVerts property.
	 */ 
	UE_API virtual void UpdateNumTargetMeshVertices();

	/**
	 * Extract the number of imported (DCC) vertices from a skeletal mesh.
	 * @param SkeletalMesh The skeletal mesh to get the number of imported vertices for.
	 * @return The number of imported vertices, which is the vertex count as seen in the DCC.
	 */ 
	static UE_API int32 ExtractNumImportedSkinnedVertices(const USkeletalMesh* SkeletalMesh);

	/** Call this if you want it to recalculate the memory usage again after the tick has completed. */
	UE_API void InvalidateMemUsage();

	/** 
	 * Check whether we invalidated the memory usage. If this returns true, we need to recompute it.
	 * @return Returns true when we should make a call to UpdateMemoryUsage.
	 */
	UE_API bool IsMemUsageInvalidated() const;

	/**
	 * Get the estimated size of the asset on disk. This is the uncooked asset, which is larger than the cooked one.
	 * @return The size in bytes.
	 */
	UE_API uint64 GetEditorAssetSizeInBytes() const;

	/**
	 * Get the estimated size of this asset on disk, when cooked.
	 * So this is the estimated size of the asset that will be packaged inside your project.
	 * @return The size in bytes.
	 */
	UE_API uint64 GetCookedAssetSizeInBytes() const;

	/**
	 * Get the estimated main memory usage for this model.
	 * @return The number of bytes that this model uses.
	 */
	UE_API uint64 GetMainMemUsageInBytes() const;

	/**
	 * Get the estimated GPU memory usage for this model.
	 * @return The number of bytes that this model uses.
	 */
	UE_API uint64 GetGPUMemUsageInBytes() const;

	/** Force update the cached memory usage. */
	UE_API virtual void UpdateMemoryUsage();
#endif

	// UObject overrides.
	UE_API virtual void Serialize(FArchive& Archive) override;
	UE_API virtual void PreSave(FObjectPreSaveContext SaveContext) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void BeginDestroy() override;
	UE_API virtual bool IsReadyForFinishDestroy() override;
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	// ~END UObject overrides.

	// IBoneReferenceSkeletonProvider overrides.
	UE_API virtual USkeleton* GetSkeleton(bool& bInvalidSkeletonIsError, const IPropertyHandle* PropertyHandle) override;
	// ~END IBoneReferenceSkeletonProvider overrides.

	/**
	 * Get the ML deformer asset that this model is part of.
	 * @return A pointer to the ML deformer asset.
	 */
	UE_API UMLDeformerAsset* GetDeformerAsset() const;

	/**
	 * Get the input information, which is information about the inputs to the deformer.
	 * Inputs are things like bone transforms and curve values.
	 * @return A pointer to the input info object.
	 */
	UMLDeformerInputInfo* GetInputInfo() const					{ return InputInfo.Get(); }

	/**
	 * Get the number of vertices in the base mesh (linear skinned skeletal mesh).
	 * This is the number of vertices in the DCC, so not the render vertex count.
	 * @return The number of imported vertices inside linear skinned skeletal mesh.
	 */
	int32 GetNumBaseMeshVerts() const							{ return NumBaseMeshVerts; }

	/**
	 * Get the number of vertices of the target mesh.
	 * This is the number of vertices in the DCC, so not the render vertex count.
	 * @return The number of imported vertices inside the target mesh.
	 */
	int32 GetNumTargetMeshVerts() const							{ return NumTargetMeshVerts; }

	/**
	 * The mapping that maps from render vertices into dcc vertices.
	 * The length of this array is the same as the number of render vertices in the skeletal mesh.
	 * For a cube with 32 render vertices, the item values would be between 0..7 as in the dcc the cube has 8 vertices.
	 * @return A reference to the array that contains the DCC vertex number for each render vertex.
	 */
	const TArray<int32>& GetVertexMap() const					{ return VertexMap; }

	/**
	 * Manually set the vertex map. This normally gets initialized automatically.
	 * @param Map The original vertex number indices for each render vertex.
	 * @see GetVertexMap.
	 */
	void SetVertexMap(const TArray<int32>& Map)					{ VertexMap = Map; }

	/**
	 * Get the GPU buffer of the VertexMap.
	 * @return A reference to the GPU buffer resource object that holds the vertex map.
	 * @see GetVertexMap
	 */
	const UE::MLDeformer::FVertexMapBuffer& GetVertexMapBuffer() const	{ return VertexMapBuffer; }

	/**
	 * Get the delegate which will be called when to inform when the model instance needs to be reinitialized.
	 * The ML Deformer component will hook into this for example. This is broadcast when something changes in the structure, such as when
	 * training finished and the neural network changed.
	 */
	FMLDeformerReinitModelInstancesDelegate& GetReinitModelInstanceDelegate()	{ return ReinitModelInstanceDelegate; }

	/**
	 * Specify whether we want to recover stripped data that is removed from this model when cooking. Examples of this data could be the raw uncompressed vertex deltas of all morph targets.
	 * On default we strip the data, save the cooked asset, and then recover the stripped data again. But using this method you can disable that, by setting it to false.
	 * Generally we only want to disable this during some automated tests.
	 */
	UE_API void SetRecoverStrippedDataAfterCook(bool bRecover);

	/**
	 * Check whether we want to recover stripped data that is removed from this model when cooking. Examples of this data could be the raw uncompressed vertex deltas of all morph targets.
	 * On default we strip the data, save the cooked asset, and then recover the stripped data again. But some automated cooking tests might want this disabled.
	 */
	UE_API bool GetRecoverStrippedDataAfterCook() const;

	UFUNCTION()
	const TArray<FString>& GetTrainingDeviceList() const			{ return TrainingDeviceList; }

	UFUNCTION(BlueprintCallable, Category = "Training")
	const FString& GetTrainingDevice() const						{ return TrainingDevice; }

	UFUNCTION()
	UE_API void SetTrainingDevice(const FString& DeviceName);
	void SetTrainingDeviceToCpu()									{ TrainingDevice = "Cpu"; }
	void SetTrainingDeviceList(const TArray<FString>& Devices)		{ TrainingDeviceList = Devices; }


#if WITH_EDITORONLY_DATA
	// UObject overrides.
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostTransacted(const FTransactionObjectEvent& Event) override;
	UE_API virtual void PreEditUndo() override;
	UE_API virtual void PostEditUndo() override;
	// ~END UObject overrides.

	/**
	 * Initialize the vertex map.
	 * This will grab the skeletal mesh imported model and copy over the imported mapping from there.
	 * This will make sure the VertexMap member is initialized.
	 * @see GetVertexMap
	 */
	UE_API void InitVertexMap();

	/**
	 * The delegate that gets fired when a property value changes.
	 * @return A reference to the delegate.
	 */
	FMLDeformerModelOnPostEditProperty& OnPostEditChangeProperty()	{ return PostEditPropertyDelegate; }

	/**
	 * Get the delegate that gets fired when a transaction is done on this model.
	 * @return A reference to the delegate.
	 */
	FMLDeformerModelOnPostTransacted& OnPostTransacted()		{ return PostTransactedDelegate; }

	/**
	 * Get the delegate that gets fired before we do undo.
	 * @return A reference to the delegate.
	 */
	FMLDeformerModelOnPreEditUndo& OnPreEditUndo()				{ return PreEditUndoDelegate; }

	/**
	 * Get the delegate that gets fired after we do undo.
	 * @return A reference to the delegate.
	 */
	FMLDeformerModelOnPostEditUndo& OnPostEditUndo()			{ return PostEditUndoDelegate; }

	/**
	 * Get the visualization settings for this model. These settings are only used in the editor.
	 * Visualization settings contain settings for the left side of the ML Deformer asset editor, containing 
	 * settings like what the mesh spacing is, whether to draw labels, what the test anim sequence is, etc.
	 * @return A pointer to the visualization settings. You can cast this to the type specific for your model, in case you inherited from 
	 *         the UMLDeformerVizSettings base class. This never return a nullptr.
	 */
	UMLDeformerVizSettings* GetVizSettings() const				{ return VizSettings; }

	/**
	 * Get the maximum number of training frames to use during training.
	 * Your training anim sequence might contain say 10000 frames, but for quickly iterating you might
	 * want to train on only 2000 frames instead. You can do this by setting the maximum training frames to 2000.
	 * @return The max number of frames to use during training.
	 */
	int32 GetTrainingFrameLimit() const							{ return MaxTrainingFrames; }

	/**
	 * Set the maximum number of frames to train on.
	 * For example if your training data has 10000 frames, but you wish to only train on 2000 frames, you can set this to 2000.
	 * @param MaxNumFrames The maximum number of frames to train on.
	 */
	void SetTrainingFrameLimit(int32 MaxNumFrames)				{ MaxTrainingFrames = MaxNumFrames; }

	/**
	 * Get the target mesh alignment transformation.
	 * This is a transformation that is applied to the vertex positions of the target mesh, before we calculate the deltas
	 * between the linear skinned mesh and the target mesh.
	 * This is useful when you imported target mesh that isn't scaled the same, or perhaps it is rotated 90 degrees over the x axis.
	 * The alignment transform is then used to correct this and align both base and target mesh.
	 * @return The alignment transformation. When set to Identity it will not do anything, which is its default.
	 */
	const FTransform& GetAlignmentTransform() const				{ return AlignmentTransform; }

	/**
	 * Set the alignment transform, which is the transform applied to the target mesh vertices, before calculating the deltas.
	 * @param Transform The transformation to apply.
	 * @see GetAlignmentTransform.
	 */
	void SetAlignmentTransform(const FTransform& Transform)		{ AlignmentTransform = Transform; }

	/**
	 * Get the list of bones that we configured to be included during training.
	 * A bone reference is basically just a name of a bone.
	 * This can be different from the bone list inside the InputInfo property though.
	 * Please keep in mind that the BoneIncludeList is what is setup in the UI, but the InputInfo contains 
	 * the actual list of what we trained on, so what we use during inference.
	 * A user could change this bone list after they trained a model, in which case the input info and this bone include list
	 * will contain different items.
	 * @return The list of bones that should be included when performing the next training session.
	 */
	TArray<FBoneReference>& GetBoneIncludeList()				{ return BoneIncludeList; }
	const TArray<FBoneReference>& GetBoneIncludeList() const	{ return BoneIncludeList; }

	/**
	 * Set the list of bones that should be included during training.
	 * If the list is empty, all bones will be included.
	 * @param List The list of bones to include.
	 */
	void SetBoneIncludeList(const TArray<FBoneReference>& List)	{ BoneIncludeList = List; }

	/**
	 * Get the list of curves that we configured to be included during training.
	 * A curve reference is basically just a name of a curve.
	 * This can be different from the curve list inside the InputInfo property though.
	 * Please keep in mind that the CurveIncludeList is what is setup in the UI, but the InputInfo contains 
	 * the actual list of what we trained on, so what we use during inference.
	 * A user could change this curve list after they trained a model, in which case the input info and this curve include list
	 * will contain different items.
	 * @return The list of curves that should be included when performing the next training session.
	 */
	TArray<FMLDeformerCurveReference>& GetCurveIncludeList()				{ return CurveIncludeList; }
	const TArray<FMLDeformerCurveReference>& GetCurveIncludeList() const	{ return CurveIncludeList; }

	/**
	 * Set the list of curves that should be included during training.
	 * If the list is empty, all curves will be included.
	 * @param List The list of curves to include.
	 */
	void SetCurveIncludeList(const TArray<FMLDeformerCurveReference>& List)	{ CurveIncludeList = List; }

	/**
	 * Set the visualization settings object.
	 * You need to call this in the constructor of your model.
	 * @param VizSettings The visualization settings object for this model.
	 */
	void SetVizSettings(UMLDeformerVizSettings* VizSettingsObject)			{ VizSettings = VizSettingsObject; }

	/**
	 * Get the settings that have been configured inside the Training Data Processor tool.
	 * These settings are stored as part of the ML Deformer model. When this hasn't been set up it will return a nullptr.
	 * @return A pointer to the training data processor settings, or a nullptr in case no settings have been setup before.
	 */
	UMLDeformerTrainingDataProcessorSettings* GetTrainingDataProcessorSettings() const			{ return TrainingDataProcessorSettings; }

	/**
	 * Set the object that is used to store the settings for the training data processor tool.
	 * @param Settings The training settings, which will be saved along with the ML Deformer model.
	 */
	void SetTrainingDataProcessorSettings(UMLDeformerTrainingDataProcessorSettings* Settings)	{ TrainingDataProcessorSettings = Settings; }

	// Get property names.
	static FName GetSkeletalMeshPropertyName()			{ return GET_MEMBER_NAME_CHECKED(UMLDeformerModel, SkeletalMesh); }
	static FName GetAlignmentTransformPropertyName()	{ return GET_MEMBER_NAME_CHECKED(UMLDeformerModel, AlignmentTransform); }
	static FName GetBoneIncludeListPropertyName()		{ return GET_MEMBER_NAME_CHECKED(UMLDeformerModel, BoneIncludeList); }
	static FName GetCurveIncludeListPropertyName()		{ return GET_MEMBER_NAME_CHECKED(UMLDeformerModel, CurveIncludeList); }
	static FName GetMaxTrainingFramesPropertyName()		{ return GET_MEMBER_NAME_CHECKED(UMLDeformerModel, MaxTrainingFrames); }
	static FName GetMaxNumLODsPropertyName()			{ return GET_MEMBER_NAME_CHECKED(UMLDeformerModel, MaxNumLODs); }
	static FName GetTrainingDevicePropertyName()		{ return GET_MEMBER_NAME_CHECKED(UMLDeformerModel, TrainingDevice); }
#endif	// #if WITH_EDITORONLY_DATA

protected:
	/**
	 * Set the training input information.
	 * @param Input A pointer to the input information object.
	 * @see CreateInputInfo
	 */
	void SetInputInfo(UMLDeformerInputInfo* Input)		{ InputInfo = Input; }

	/**
	 * Convert an array of floats to an array of Vector3's.
	 * A requirement is that the number of items in the float array is a multiple of 3 (xyz).
	 * The order of the float items must be like this: (x, y, z, x, y, z, x, y, z, ...).
	 * The OutVectorArray will be automatically resized internally.
	 * @param FloatArray The array of floats to build an array of Vector3's for.
	 * @param OutVectorArray The array that will contain the vectors instead of floats.
	 */
	UE_API void FloatArrayToVector3Array(const TArray<float>& FloatArray, TArray<FVector3f>& OutVectorArray);

	/**
	 * Set the number of vertices in the base mesh.
	 * This is the number of imported (dcc) vertices, so not render vertices.
	 * @param NumVerts The number of vertices.
	 */
	void SetNumBaseMeshVerts(int32 NumVerts)			{ NumBaseMeshVerts = NumVerts; }

	/**
	 * Set the number of vertices in the target mesh.
	 * This is the number of imported (dcc) vertices, so not render vertices.
	 * @param NumVerts The number of vertices.
	 */
	void SetNumTargetMeshVerts(int32 NumVerts)			{ NumTargetMeshVerts = NumVerts; }

protected:
#if WITH_EDITOR
	/** Should we recalculate the memory usage? */
	bool bInvalidateMemUsage = true;

	/** Estimated main memory usage. */
	uint64 MemUsageInBytes = 0;

	/** Estimated editor asset size. */
	uint64 EditorAssetSizeInBytes = 0;

	/** Estimated cooked asset size. */
	uint64 CookedAssetSizeInBytes = 0;

	/** Estimated GPU memory usage. */
	uint64 GPUMemUsageInBytes = 0;
#endif

#if WITH_EDITORONLY_DATA
	/**
	 * The animation sequence to apply to the base mesh. This has to match the animation of the target mesh's geometry cache. 
	 * Internally we force the Interpolation property for this motion to be "Step".
	 */
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use the training input anims instead."))
	TSoftObjectPtr<UAnimSequence> AnimSequence_DEPRECATED;
#endif

private:
	/** The deformer asset that this model is part of. */
	TObjectPtr<UMLDeformerAsset> DeformerAsset = nullptr;

	/** The delegate that gets fired when a property is being modified. */
	FMLDeformerModelOnPostEditProperty PostEditPropertyDelegate;

	/** The delegate to capture PostTransacted on this model uobject. */
	FMLDeformerModelOnPostTransacted PostTransactedDelegate;

	/** The delegate to capture PreEditUndo on this model uobject. */
	FMLDeformerModelOnPreEditUndo PreEditUndoDelegate;

	/** The delegate to capture PostEditUndo on this model uobject. */
	FMLDeformerModelOnPostEditUndo PostEditUndoDelegate;

	/** GPU buffers for Vertex Map. */
	UE::MLDeformer::FVertexMapBuffer VertexMapBuffer;

	/** Fence used in render thread cleanup on destruction. */
	FRenderCommandFence RenderResourceDestroyFence;

	/** Delegate used to trigger reinitialization of the model instance. */
	FMLDeformerReinitModelInstancesDelegate ReinitModelInstanceDelegate;

	/** Cached number of skeletal mesh vertices. */
	UPROPERTY()
	int32 NumBaseMeshVerts = 0;

	/** Cached number of target mesh vertices. */
	UPROPERTY()
	int32 NumTargetMeshVerts = 0;

	/** The device used for training. On default it will init to Cuda's preferred device, or Cpu if no such device present. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Training Settings", meta = (GetOptions = "GetTrainingDeviceList", NoResetToDefault))
	FString TrainingDevice;

	/** The list of training devices that will show in the combo box in the UI. */
	TArray<FString> TrainingDeviceList;

	/** 
	 * How many Skeletal Mesh LOD levels should we generate MLD lods for at most?
	 * Some examples:
	 * A value of 1 means we only store one LOD, which means LOD0. 
	 * A value of 2 means we support this ML Deformer on LOD0 and LOD1.
	 * A value of 3 means we support this ML Deformer on LOD0 and LOD1 and LOD2.
	 * We never generate more LOD levels for the ML Deformer than number of LOD levels in the Skeletal Mesh, so if 
	 * this value is set to 100, while the Skeletal Mesh has only 4 LOD levels, we will only generate and store 4 ML Deformer LODs.
	 * The default value of 1 means we do not support this ML Deformer at LOD levels other than LOD0.
	 * When cooking, the console variable "sg.MLDeformer.MaxLODLevelsOnCook" can be used to set the maximum value per device or platform.
	 */
	UPROPERTY(EditAnywhere, Category = "LOD Generation Settings", meta = (ClampMin = "1"))
	int32 MaxNumLODs = 1;

	/** 
	 * The information about the neural network inputs. This contains things such as bone names and curve names.
	 */
	UPROPERTY()
	TObjectPtr<UMLDeformerInputInfo> InputInfo;

	/** This is an index per vertex in the mesh, indicating the imported vertex number from the source asset. */
	UPROPERTY()
	TArray<int32> VertexMap;

	/** The settings for the training data processor. */
	UPROPERTY()
	TObjectPtr<UMLDeformerTrainingDataProcessorSettings> TrainingDataProcessorSettings;

	/** The skeletal mesh that represents the linear skinned mesh. */
	UPROPERTY(EditAnywhere, Category = "Base Mesh")
	TObjectPtr<USkeletalMesh> SkeletalMesh;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UMLDeformerVizSettings> VizSettings;

	/** Specifies whether bone transformations should be included as inputs during the training process. */
	UE_DEPRECATED(5.3, "This property has been removed.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property has been removed and isn't used anymore."))
	bool bIncludeBones_DEPRECATED = true;

	/** Specifies whether curve values (a float per curve) should be included as inputs during the training process. */
	UE_DEPRECATED(5.3, "This property has been removed.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property has been removed and isn't used anymore."))
	bool bIncludeCurves_DEPRECATED = false;

	/** The transform that aligns the Geometry Cache to the SkeletalMesh. This will mostly apply some scale and a rotation, but no translation. */
	UPROPERTY(EditAnywhere, Category = "Inputs", DisplayName = "Target Alignment Transform")
	FTransform AlignmentTransform = FTransform::Identity;

	/** The bones to include during training. When none are provided, all bones of the Skeleton will be included. */
	UPROPERTY(meta = (EditCondition = "bIncludeBones"))
	TArray<FBoneReference> BoneIncludeList;

	/** The curves to include during training. When none are provided, all curves of the Skeleton will be included. */
	UPROPERTY(meta = (EditCondition = "bIncludeCurves"))
	TArray<FMLDeformerCurveReference> CurveIncludeList;

	/** The maximum numer of training frames (samples) to train on. Use this to train on a sub-section of your full training data. */
	UPROPERTY(EditAnywhere, Category = "Training Settings", meta = (ClampMin = "1"))
	int32 MaxTrainingFrames = 1000000;

	/**
	 * Sometimes there can be some vertices that cause some issues that cause deltas to be very long. We can ignore these deltas by setting a cutoff value. 
	 * Deltas that are longer than the cutoff value (in units), will be ignored and set to zero length. 
	 */
	UE_DEPRECATED(5.5, "This property has been removed.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property has been removed and isn't used anymore."))
	float DeltaCutoffLength_DEPRECATED = 30.0f;

	/** 
	 * Do we want to recover stripped data at the end of the Serialize call during cook?
	 * This is enabled on default, but some automated tests might disable this.
	 */
	bool bRecoverStrippedDataAfterCook = true;
#endif
};

#undef UE_API
