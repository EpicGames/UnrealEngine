// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerModule.h"
#include "UObject/ObjectPtr.h"
#include "RenderCommandFence.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "MLDeformerModelInstance.generated.h"

#define UE_API MLDEFORMERFRAMEWORK_API

class UMLDeformerModel;
class USkeletalMeshComponent;
class UMLDeformerComponent;

DECLARE_STATS_GROUP(TEXT("MLDeformer"), STATGROUP_MLDeformer, STATCAT_Advanced);
DECLARE_CYCLE_STAT_EXTERN(TEXT("ML Deformer Inference"), STAT_MLDeformerInference, STATGROUP_MLDeformer, );

namespace UE::MLDeformer
{
	extern UE_API TAutoConsoleVariable<bool> CVarMLDeformerDebugDrawInputPoses;
}

/**
 * An instance of the ML Deformer model.
 * The ML Deformer model contains shared data, while this instance contains data unique to the actor it is being applied to.
 * So if you have 1 ML Deformer model, applied to 100 enemy actors, you will get 100 ML Deformer Model Instance objects.
 * The model instance is created and managed by the ML Deformer component.
 * The main task of the model instance is to feed the neural network with the right input values and run the neural network
 * inference at runtime, and possibly do something with the output.
 */
UCLASS(MinimalAPI)
class UMLDeformerModelInstance
	: public UObject
{
	GENERATED_BODY()

public:
	UMLDeformerModelInstance() = default;

	// UObject overrides.
	UE_API virtual void BeginDestroy() override;
	UE_API virtual bool IsReadyForFinishDestroy() override;
	// ~END UObject overrides.
	
	/**
	 * Initialize the model instance.
	 * This internally builds the structures to quickly grab bone transforms.
	 * If you override this method, you probably also want to call the Init of this base class inside your overloaded method.
	 * @param SkelMeshComponent The skeletal mesh component that we will grab bone transforms and curve values from.
	 */
	UE_API virtual void Init(USkeletalMeshComponent* SkelMeshComponent);

	/**
	 * Initialize this model, after the ML deformer component has been initialized.
	 * This is a post-step after component initialization. It can be used to register things such as 
	 * external morph target sets that are generated by this model.
	 */
	virtual void PostMLDeformerComponentInit() {}

	/**
	 * Update the deformer instance.
	 * If you use a neural network, this should provide inputs to the neural network.
	 * @param DeltaTime The delta time since the last frame, in seconds.
	 * @param ModelWeight The weight of the model, between 0 and 1, where 0 means it has no influence, and 1 means it is fully active.
	 */
	UE_API virtual void Tick(float DeltaTime, float ModelWeight);

	/** 
	 * Executed at the end of the tick. This is always executed when Tick is called, also if the model weight is 0 etc. 
	 * When the model weight is tiny or setting the inputs failed, the Execute function wasn't called during Tick.
	 * The parameter that is passed to this method will let you know whether Execute got called or not.
	 * @param bExecuteCalled This is true when we called Execute, otherwise it is false.
	 */
	virtual void PostTick(bool bExecuteCalled) {}

	/**
	 * Check whether the deformer is compatible with a given skeletal mesh component.
	 * This internally also edits the value returned by GetCompatibilityErrorText().
	 * @param InSkelMeshComponent The skeletal mesh component to check compatibility with.
	 * @param bLogIssues Set to true to automatically log any compatibility errors.
	 * @return Returns the error string. When the returned string is empty, there were no errors and thus
	 *         the specified skeletal mesh component is compatible.
	 */
	UE_API virtual FString CheckCompatibility(USkeletalMeshComponent* InSkelMeshComponent, bool bLogIssues=false);

	/**
	 * Check if we are in a valid state for the deformer graph's data provider.
	 * In the base class this will check whether the neural network is valid, if the vertex map buffer is valid
	 * and whether the neural network instance handle is valid.
	 * @return Returns true if the data provider in would be in a valid state, otherwise false is returned.
	 */
	UE_API virtual bool IsValidForDataProvider() const;

#if WITH_EDITOR
	/**
	 * Copy over data from the debug actor, if any is used.
	 * You can get the debug actor using UMLDeformerComponent::GetDebugActor().
	 * This should make calls to the CopyDataFromDebugActor, which takes the actor pointer and ML Deformer component as parameters.
	 * Compatibility checks between the debug actor and the current model instance should be done inside this method.
	 * This could for example copy over morph target weights etc.
	 */
	UE_API virtual void CopyDataFromCurrentDebugActor();

	/**
	 * Copy data from a specific debug actor.
	 * This method already assumes DebugActor and MLDeformerComponent are compatible with the current model instance.
	 * @param DebugActor The debug actor object to copy data from.
	 * @param DebugMLDeformerComponent The debug ML Deformer component to copy data from.
	 */
	virtual void CopyDataFromDebugActor(const AActor* DebugActor, const UMLDeformerComponent* DebugMLDeformerComponent) {}

	/**
	 * Set the frame index that we use to sample the ground truth vertex positions when showing the heat map in ground truth mode.
	 * @param FrameIndex The frame number in the geometry cache that we sample the ground truth positions from.
	 */
	UE_API void SetGroundTruthFrameIndex(int32 FrameIndex);

	/**
	 * Get the frame index that we use to sample the ground truth vertex positions when showing the heat map in ground truth mode.
	 * @return The frame number inside the geometry cache that we sample the ground truth positions from.
	 */
	UE_API int32 GetGroundTruthFrameIndex() const;
#endif

	/**
	 * Is the deformer asset used compatible with the skeletal mesh component used during the Init call?
	 * @return True if compatible, false if not.
	 */
	UE_API bool IsCompatible() const;

	/** 
	 * Get the compatibility error text. This will be a non-empty string in case IsCompatible() return false. 
	 * @return Returns the string containing more information about the reasons for compatibility issues.
	 */
	UE_API const FString& GetCompatibilityErrorText() const;

	/**
	 * Get the skeletal mesh component we're working with.
	 * @return A pointer to the skeletal mesh component.
	 */
	UE_API USkeletalMeshComponent* GetSkeletalMeshComponent() const;

	/** Update the compatibility status, as returned by IsCompatible() and GetCompatibilityErrorText(). */
	UE_API void UpdateCompatibilityStatus();

	/** Get the model that this is an instance of. */
	UE_API UMLDeformerModel* GetModel() const;

	/** Set the deformer model that this is an instance of. */
	UE_API void SetModel(UMLDeformerModel* InModel);

	/** Set whether we had a successful call to UMLDeformerModel::PostMLDeformerComponentInit. */
	UE_API void SetHasPostInitialized(bool bHasInitialized);

	/** Check whether we already called UMLDeformerModel::PostMLDeformerComponentInit. */
	UE_API bool HasPostInitialized() const;

	/**
	 * Get the ML Deformer component that this instance is part of.
	 * @return A pointer to the ML Deformer component.
	 */
	UE_API UMLDeformerComponent* GetMLDeformerComponent() const;

	/**
	 * Get the skeletal mesh component to get transforms from etc.
	 * If we are debugging this will return the debugged skeletal mesh component, otherwise the skel mesh component
	 * of our own actor is returned.
	 * @return The skeletal mesh component to grab input data from.
	 */
	UE_API USkeletalMeshComponent* GetFinalSkeletalMeshComponent() const;

protected:
	/**
	 * Draw the input poses that are currently input to this instance.
	 * This is linked to the "MLDeformer.DebugDrawInputPoses" console command. 
	 */
	UE_API void DrawDebugInputPoses();
	
	/**
	 * Update the neural network input values directly inside its input tensor.
	 * This will copy over the bone transforms (and possibly morph and curve weights) into this flat array.
	 * @param InputData The buffer of floats to write the input values to.
	 * @param NumInputFloats The number of floats of the InputData buffer. Do not write more than this number of floats to the InputData buffer.
	 * @return Returns the new buffer element index. For example if there were 2 bones and 3 curves, and each bone takes 6 inputs and each curve takes one 
	 * input, then after calling this base class method, it will return 15 (2*6 + 3*1). You could call this base class method and then write data after that, starting
	 * from the offset that is returned.
	 */
	UE_API virtual int64 SetNeuralNetworkInputValues(float* InputData, int64 NumInputFloats);

	/**
	 * Setup the neural network for this frame.
	 * This has to perform compatibility checks, valid pointer checks, and call the SetNeuralNetworkInputValues.
	 * After this method is executed, the neural network should be ready to execute.
	 * The UMLDeformerModelInstance::Tick method internally calls both the SetupNeuralNetworkForFrame and RunNeuralNetwork methods.
	 * @return Returns true when the setup is done correctly and the neural network is ready to be executed. Otherwise false is returned, which 
	 * can happen when the NeuralNetwork pointer is invalid, when the model is not set, when the network is not compatible, etc.
	 */
	virtual bool SetupInputs() { return true; }

	/**
	 * Execute the model instance, which can run the neural network inference in case the model uses a neural network.
	 * This already assumes that compatibility checks are done, and that the network inputs are set etc.
	 * @param ModelWeight The weight of the model, must be between 0 and 1.
	 */
	virtual void Execute(float ModelWeight) {}

	/**
	 * Handle when the model weight is zero.
	 * This is called when ticking with a zero model weight.
	 * The SetupNeuralNetworkForFrame and RunNeuralNetwork calls are skipped in case we have a model weight near zero, as optimization.
	 * Instead, this method is called in that case. It can be used to for example set morph target weights to zero.
	 */
	virtual void HandleZeroModelWeight() {}

	/**
	 * Set the bone transformations inside a given output buffer, starting from a given StartIndex.
	 * @param OutputBuffer The buffer we need to write the transformations to.
	 * @param OutputBufferSize The number of floats inside the output buffer. Can be used to detect buffer overflows.
	 * @param StartIndex The index where we want to start writing the transformations to.
	 * @return The new start index into the output buffer, to be used when you write more data after this.
	 */
	UE_API virtual int64 SetBoneTransforms(float* OutputBuffer, int64 OutputBufferSize, int64 StartIndex);

	/**
	 * Set the animation curve values inside a given output buffer, starting from a given StartIndex.
	 * @param OutputBuffer The buffer we need to write the weights to.
	 * @param OutputBufferSize The number of floats inside the output buffer. Can be used to detect buffer overflows.
	 * @param StartIndex The index where we want to start writing the weights to.
	 * @return The new start index into the output buffer, to be used when you write more data after this.
	 */
	UE_API virtual int64 SetCurveValues(float* OutputBuffer, int64 OutputBufferSize, int64 StartIndex);

	/**
	 * Updates the bone transforms array.
	 */
	UE_API void UpdateBoneTransforms();

	/** 
	 * Check whether we have valid transforms in the skeletal mesh component.
	 * When this returns false, we can't really execute the deformer.
	 * @return Returns true if the skeletal mesh component, or its leader component has non-empty transform buffers.
	 */
	UE_API bool HasValidTransforms() const;

protected:
	/** The fence that let's us wait for all render commands to finish, before we continue. */
	FRenderCommandFence RenderCommandFence;

	/** The ML Deformer model that this is an instance of. */
	UPROPERTY(Transient)
	TObjectPtr<UMLDeformerModel> Model = nullptr;

	/** The skeletal mesh component we work with. This is mainly used for compatibility checks. */
	UPROPERTY(Transient)
	TObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent;

	/** The cached current local space bone transforms for the current frame. */
	TArray<FTransform> TrainingBoneTransforms;

	/** The debug bone space transforms. */
	TArray<FTransform> DebugBoneSpaceTransforms;

	/** Maps the ML deformer asset bone index to a skeletal mesh component bone index. */
	TArray<int32> AssetBonesToSkelMeshMappings;

	/** The compatibility error text, in case bIsCompatible is false. */
	FString ErrorText;

	/** Are the deformer asset and the used skeletal mesh component compatible? */
	bool bIsCompatible = false;

	/** Has this instance had a successful UMLDeformerModel::PostMLDeformerComponentInit call? */
	bool bHasPostInitialized = false;

#if WITH_EDITOR
	/** The current ground truth frame index. This is used for displaying heatmaps versus the ground truth accurately. */
	int32 GroundTruthFrameIndex = 0;
#endif
};

#undef UE_API
