// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Modules/ModuleManager.h"
#include "HAL/IConsoleManager.h"

#define UE_API MLDEFORMERFRAMEWORK_API

// The log category for the ML Deformer framework.
MLDEFORMERFRAMEWORK_API DECLARE_LOG_CATEGORY_EXTERN(LogMLDeformer, Log, All);

namespace UE::MLDeformer
{
	/**
	 * The runtime module for the ML Deformer.
	 */
	class FMLDeformerModule
		: public IModuleInterface
	{
	public:
		// IModuleInterface overrides.
		UE_API void StartupModule() override;
		UE_API void ShutdownModule() override;
		// ~END IModuleInterface overrides.

		const IConsoleVariable& GetMaxLODLevelsOnCookCVar() const		{ return *CVarMLDeformerMaxLODLevels; }

	private:
		IConsoleVariable* CVarMLDeformerMaxLODLevels = nullptr;
	};
}	// namespace UE::MLDeformer

#undef UE_API
