// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerGeomCacheModel.h"
#include "MLDeformerMorphModelQualityLevel.h"
#include "MLDeformerMorphModel.generated.h"

#define UE_API MLDEFORMERFRAMEWORK_API

struct FExternalMorphSet;
struct FExternalMorphSetWeights;
class USkinnedMeshComponent;

/**
 * The morph model base class.
 * This is the base class for models that generate and drive morph targets.
 * Use this in combination with UMLDeformerMorphModelInstance or inherited classes.
 */
UCLASS(MinimalAPI)
class UMLDeformerMorphModel
	: public UMLDeformerGeomCacheModel
{
	GENERATED_BODY()

public:
	UE_API UMLDeformerMorphModel(const FObjectInitializer& ObjectInitializer);

	// UMLDeformerModel overrides.
	virtual bool IsNeuralNetworkOnGPU() const override				{ return false; }	// CPU based neural network.
	virtual bool DoesSupportLOD() const override					{ return true; }	// We support deformations in different LOD levels.
	UE_API virtual void Serialize(FArchive& Archive) override;
	UE_API virtual UMLDeformerInputInfo* CreateInputInfo() override;
	UE_API virtual UMLDeformerModelInstance* CreateModelInstance(UMLDeformerComponent* Component) override;
#if WITH_EDITOR
	UE_API virtual void UpdateMemoryUsage() override;
	UE_API virtual void FinalizeMorphTargets();
	UE_API bool HasRawMorph() const;
#endif
	// ~END UMLDeformerModel overrides.

	// UObject overrides.
	UE_API virtual void BeginDestroy() override;
	UE_API virtual bool IsReadyForFinishDestroy() override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	// ~END UObject overrides.

	UE_API int32 GetNumMorphTargets(int32 LOD) const;
	uint64 GetCompressedMorphDataSizeInBytes() const				{ return CompressedMorphDataSizeInBytes; }
	uint64 GetUncompressedMorphDataSizeInBytes() const				{ return UncompressedMorphDataSizeInBytes; }

	float GetMorphDeltaZeroThreshold() const						{ return MorphDeltaZeroThreshold; }
	float GetMorphCompressionLevel() const							{ return MorphCompressionLevel; }
	float GetMaxMorphTargetErrorInUnits() const						{ return !MorphTargetErrorOrder.IsEmpty() ? MorphTargetErrors[MorphTargetErrorOrder[0]] : 0.0f; }
	bool GetIncludeMorphTargetNormals() const						{ return bIncludeNormals; }
	EMLDeformerMaskChannel GetMaskChannel() const					{ return MaskChannel; }
	bool GetInvertMaskChannel() const								{ return bInvertMaskChannel; }
	bool GetHasOnlyEmptyMorphs() const								{ return bHasOnlyEmptyMorphs; }

	UFUNCTION(BlueprintPure, Category = "MLDeformerMorphModel")
	UE_API bool CanDynamicallyUpdateMorphTargets() const;

	void SetMorphDeltaZeroThreshold(float Threshold)				{ MorphDeltaZeroThreshold = Threshold; }
	void SetMorphCompressionLevel(float Tolerance)					{ MorphCompressionLevel = Tolerance; }
	void SetIncludeMorphTargetNormals(bool bInclude)				{ bIncludeNormals = bInclude; }
	void SetMaskChannel(EMLDeformerMaskChannel Channel)				{ MaskChannel = Channel; }
	void SetInvertMaskChannel(bool bInvert)							{ bInvertMaskChannel = bInvert; }
	void SetClampMorphTargetsWeights(bool bEnabled)					{ bClampMorphWeights = bEnabled; }
	void SetHasOnlyEmptyMorphs(bool bOnlyEmpty)						{ bHasOnlyEmptyMorphs = bOnlyEmpty; }


	// Get property names.
	static FName GetMorphDeltaZeroThresholdPropertyName()			{ return GET_MEMBER_NAME_CHECKED(UMLDeformerMorphModel, MorphDeltaZeroThreshold); }
	static FName GetMorphCompressionLevelPropertyName()				{ return GET_MEMBER_NAME_CHECKED(UMLDeformerMorphModel, MorphCompressionLevel); }
	static FName GetIncludeMorphTargetNormalsPropertyName()			{ return GET_MEMBER_NAME_CHECKED(UMLDeformerMorphModel, bIncludeNormals); }
	static FName GetMaskChannelPropertyName()						{ return GET_MEMBER_NAME_CHECKED(UMLDeformerMorphModel, MaskChannel); }
	static FName GetInvertMaskChannelPropertyName()					{ return GET_MEMBER_NAME_CHECKED(UMLDeformerMorphModel, bInvertMaskChannel); }
	static FName GetCompressedMorphDataSizeInBytesPropertyName()	{ return GET_MEMBER_NAME_CHECKED(UMLDeformerMorphModel, CompressedMorphDataSizeInBytes); }
	static FName GetUncompressedMorphDataSizeInBytesPropertyName()	{ return GET_MEMBER_NAME_CHECKED(UMLDeformerMorphModel, UncompressedMorphDataSizeInBytes); }
	static FName GetClampMorphTargetWeightsPropertyName()			{ return GET_MEMBER_NAME_CHECKED(UMLDeformerMorphModel, bClampMorphWeights); }

#if WITH_EDITORONLY_DATA
	static FName GetGlobalMaskAttributePropertyName()				{ return GET_MEMBER_NAME_CHECKED(UMLDeformerMorphModel, VertexAttributeName); }
#endif


	/** Update the statistics properties of this model. */
	UE_API virtual void UpdateStatistics();

	/**
	 * Set the per vertex deltas, as a set of floats. Each vertex delta must have 3 floats.
	 * These deltas are used to generate compressed morph targets internally. You typically call this method from inside
	 * the python training script once your morph target deltas have been generated there.
	 * Concatenate all deltas into one buffer, so like this [morphdeltas_target0, morphdeltas_target1, ..., morphdeltas_targetN].
	 * The vertex ordering should be: [(x, y, z), (x, y, z), (x, y, z)].
	 * This is the same as SetMorphTargetDeltas, except that this takes an array of floats instead of vectors.
	 * @param Deltas The array of floats that contains the deltas. The number of items in the array must be equal to (NumMorphs * NumBaseMeshVerts * 3).
	 */
	UFUNCTION(BlueprintCallable, Category = "MLDeformerMorphModel")
	UE_API void SetMorphTargetDeltaFloats(const TArray<float>& Deltas);

	/**
	 * Set the morph target model deltas as an array of 3D vectors.
	 * These deltas are used to generate compressed morph targets internally. You typically call this method from inside
	 * the python training script once your morph target deltas have been generated there.
	 * Concatenate all deltas into one buffer, so like this [morphdeltas_target0, morphdeltas_target1, ..., morphdeltas_targetN].
	 * This is the same as SetMorphTargetDeltaFloats, except that it takes vectors instead of floats.
	 * @param Deltas The array of 3D vectors that contains the vertex deltas. The number of items in the array must be equal to (NumMorphs * NumBaseMeshVerts).
	 */
	UFUNCTION(BlueprintCallable, Category = "MLDeformerMorphModel")
	UE_API void SetMorphTargetDeltas(const TArray<FVector3f>& Deltas);

	/**
	 * Set the order of importance during LOD, for the morph targets.
	 * Basically this specifies the sorted order of the morph targets, based on the error they introduce when disabling them.
	 * @param MorphTargetOrder The array of morph target indices, starting with the most important morph target, and ending with the least important morph target's index.
	 * @param ErrorValues The error value for each morph target (not sorted), so index 0 contains the error value of the first morph target, which isn't necesarrily the one with highest error.
	 */
	UFUNCTION(BlueprintCallable, Category = "MLDeformerMorphModel")
	UE_API void SetMorphTargetsErrorOrder(const TArray<int32>& MorphTargetOrder, const TArray<float>& ErrorValues);

	/**
	 * Set the maximum weights for each morph target when we pass the training inputs into the trained neural network.
	 * After training we run the inputs through the trained network, and get the weights for each morph target.
	 * The array passed to this function must be the maximum of the absolute weight value over all training samples for each specific morph.
	 * We use this to scale the morph target deltas to get an estimate of the maximum length they will ever have, which we can in turn use
	 * to estimate how important a given morph target is.
	 * Keep in mind that these are the maximum of the absolute weights. As weights can be negative, we first take the absolute value of them.
	 * Training python scripts mostly will call this function to set the values.
	 * @param MaxWeights The maximum of the absolute weight values.
	 */
	UE_DEPRECATED(5.4, "This method will be removed.")
	UFUNCTION(BlueprintCallable, Category = "MLDeformerMorphModel")
	void SetMorphTargetsMaxWeights(const TArray<float>& MaxWeights) {}

	/** 
	 * Set the minimum and maximum values that the morph targets weights have seen during training.
	 * We can clamp the network output weights within these ranges to try to make sure that the output doesn't go wild
	 * when we give inputs that are far outside of the range we have seen during training.
	 * The size of the array must equal the number of morph targets, or empty. In case it is empty, clamping will be ignored, even when enabled.
	 * @param MinValues The minimum weigth values, one for each morph target.
	 * @param MaxValues The maximum weight values, one for each morph target.
	 */
	UFUNCTION(BlueprintCallable, Category = "MLDeformerMorphModel")
	UE_API void SetMorphTargetsMinMaxWeights(const TArray<float>& MinValues, const TArray<float>& MaxValues);

	/**
	 * Set the minimum and maximum values that the morph targets weights have seen during training.
	 * We can clamp the network output weights within these ranges to try to make sure that the output doesn't go wild
	 * when we give inputs that are far outside of the range we have seen during training.
	 * The size of the array must equal the number of morph targets, or empty. In case it is empty, clamping will be ignored, even when enabled.
	 * @param MinMaxValues The array of float intervals that contain the min and max values, one for each morph target.
	 */
	UE_API void SetMorphTargetsMinMaxWeights(const TArray<FFloatInterval>& MinMaxValues);

	/**
	 * Get the estimated maximum weight values that the morph targets will ever have. It is not guaranteed that the weights will never be larger than these values though.
	 * These values are used to estimate the importance levels of the morph targets.
	 * @see SetMorphTargetsMaxWeights.
	 */

	/**
	 * Get the min and max morph target weight values that we seen on the output of the training data set.
	 * These min and max values can be used to clamp the morph targets within this range.
	 * This can be used to prevent weights from getting very different values on unseen input data, and thus can prevent visually 'exploding' correctives.
	 */
	const TArray<FFloatInterval>& GetMorphTargetsMinMaxWeights() const	{ return MorphTargetsMinMaxWeights; }

	/**
	 * Check whether clamping of morph target weights is enabled.
	 * If enabled, the weights that are output will be clamped to the minimum and maximum values that are defined in the values returned by
	 * GetMorphTargetsMinMaxWeights().
	 * It is up to the model to actually call the ClampMorphTargetWeights method though.
	 * You can check if a model supports weight clamping by calling FMLDeformerMorphModelEditorModel::IsMorphWeightClampingSupported().
	 * If that would return false, even enabling clamping using SetClampMorphTargetsWeights(true) will not do anything.
	 * @return Returns true when clamping is enabled, otherwise false is returned.
	 */
	bool IsMorphWeightClampingEnabled() const							{ return bClampMorphWeights; }

	/**
	 * Perform the actual weight clamping to the values as set inside the MorphTargetsMinMaxWeights member.
	 * @param WeightsArray The array of input weights, that will be clamped. This method will modify the array values.
	 */
	UE_API void ClampMorphTargetWeights(TArrayView<float> WeightsArray);

	/**
	 * Get the morph target delta vectors array.
	 * The layout of this array is [morphdeltas_target0, morphdeltas_target1, ..., morphdeltas_targetN].
	 * So the total number of items in the array returned equals (NumMorphTargets * NumBaseMeshVerts).
	 */
	const TArray<FVector3f>& GetMorphTargetDeltas() const				{ return MorphTargetDeltas; }
	TArray<FVector3f>& GetMorphTargetDeltas()							{ return MorphTargetDeltas; }

	/**
	 * Get the morph target set.
	 */
	TSharedPtr<FExternalMorphSet> GetMorphTargetSet(int32 LOD) const	{ return MorphTargetSets.IsValidIndex(LOD) ? MorphTargetSets[LOD] : TSharedPtr<FExternalMorphSet>(); }

	/** Get the number of LOD levels that we have deltas for. */
	int32 GetNumLODs() const											{ return MorphTargetSets.Num(); }
	UE_API void ClearMorphTargetSets();
	UE_API void AddMorphSets(int32 NumToAdd);

	/**
	 * Get the start index into the array of deltas (vectors3's), for a given morph target.
	 * This does not perform a bounds check to see if MorphTargetIndex is in a valid range, so be aware.
	 * @param MorphTargetIndex The morph target index.
	 * @return The start index, or INDEX_NONE in case there are no deltas.
	 */
	UE_API int32 GetMorphTargetDeltaStartIndex(int32 MorphTargetIndex) const;

	/**
	 * Get the morph target error values.
	 * This is an error value for each morph target, which indicates how much error you would introduce when disabling that specific morph target.
	 * It can be used as some sort of LOD system, where we disable morph targets that are marked with an error below a specific threshold.
	 * Keep in mind that the first morph target always contains the means, and is always active. So actually you have to +1 the index.
	 * @return The array view of the error values.
	 */
	UE_API TArrayView<const float> GetMorphTargetErrorValues() const;

	/**
	 * Get the array of morph target indices, sorted by error value, from largest to smallest error.
	 * So the first array value contains the index of the most important morph target and the last element contains the least important morph target.
	 * @return An array view of the morph target indices.
	 */
	UE_API TArrayView<const int32> GetMorphTargetErrorOrder() const;

	/**
	 * Get the estimated error that disabling a specific morph target would introduce.
	 * This error is represented in units, so in cm.
	 * @param MorphIndex The morph target index to get the error for.
	 * @return The estimated error value, in cm.
	 */
	UE_API float GetMorphTargetError(int32 MorphIndex) const;

	/**
	 * Set the estimated error that disabling a specific morph target would introduce.
	 * This error is represented in units, so in cm.
	 * @param MorphIndex The morph target index to get the error for.
	 * @param Error The estimated error value, in cm.
	 */
	UE_API void SetMorphTargetError(int32 MorphIndex, float Error);

#if WITH_EDITORONLY_DATA
	/** Get the name of the attribute used for the global mask. */
	UE_API FName GetGlobalMaskAttributeName() const;
#endif

private:
	/** A morph target set for each LOD. The number of LODs that have a morph target set can be smaller than the number of LOD levels in the Skeletal Mesh, but should never be larger. */
	TArray<TSharedPtr<FExternalMorphSet>> MorphTargetSets;

	/** 
	 * Should we enable morph target weight clamping?
	 * The minimum and maximum values that it will be clamped against will be the min/max morph target weight values
	 * that have been seen when running the training dataset through the network.
	 * The advantage of clamping is that it can make deformations more stable when we have input poses that have not been seen during training.
	 * We basically prevent the weights from 'exploding' and getting very large values, which could make the mesh look very bad.
	 */
	UPROPERTY(EditAnywhere, Category = "Morph Targets")
	bool bClampMorphWeights = true;

	/**
	 * The entire set of uncompressed morph target deltas, 3 per vertex, for each morph target, as one flattened buffer.
	 * So the size of this buffer is: (NumVertsPerMorphTarget * 3 * NumMorphTargets).
	 * You can get the number of vertices per morph target using Model->GetInputInfo()->GetNumBaseMeshVerts().
	 * This property is Transient, because we don't want it included in undo/redo, as this can be a few hundred megabytes large.
	 * Instead, this data is serialized manually inside the Serialize method.
	 */
	UPROPERTY(NonTransactional)
	TArray<FVector3f> MorphTargetDeltas;

	/** The number of morph targets. */
	UPROPERTY()
	int32 NumMorphTargets_DEPRECATED = 0;

	/** The compressed memory usage of the morph targets. This is approximately what this MLD asset will use in your packaged project. */
	UPROPERTY()
	uint64 CompressedMorphDataSizeInBytes = 0;

	/** The uncompressed memory usage of the morph targets. Please note that your packaged project will use the compressed amount of memory, and not the uncompresed amount of memory. */
	UPROPERTY()
	uint64 UncompressedMorphDataSizeInBytes = 0;

	/** 
	 * An array of integers where the first element is the most important morph target and the last element is the least important one.
	 * The number of elements in the array is equal to the total number of morph targets.
	 */
	UPROPERTY()
	TArray<int32> MorphTargetErrorOrder;

	/** An error value, for each morph target. */
	UPROPERTY()
	TArray<float> MorphTargetErrors;

	/** The maximum absolute weight values of the morph targets, during training. One value for each morph target. */
	UPROPERTY()
	TArray<float> MaxMorphWeights_DEPRECATED;

	/** 
	 * The minimum and maximum weight values that the morph targets weights have seen during training.
	 * We can clamp the network output weights within these ranges to try to make sure that the output doesn't go wild
	 * when we give inputs that are far outside of the range we have seen during training.
	 * It is possible that this array is empty, in which case this clamping isn't supported.
	 * The size of the array equals the number of morph targets.
	 */
	UPROPERTY()
	TArray<FFloatInterval> MorphTargetsMinMaxWeights;

	/** 
	 * The list of quality levels, where the first item represents the highest quality and the last element the lowest quality level.
	 * The number in each quality level represents the number of active morph targets for that quality level.
	 * These numbers will be clamped internally to be within valid ranges in case they go beyond the amount of morphs that exist.
	 */
	UPROPERTY()
	TArray<FMLDeformerMorphModelQualityLevel> QualityLevels_DEPRECATED;

	/**
	 * Include vertex normals in the morph targets?
	 * The advantage of this can be that it is higher performance than recomputing the normals.
	 * The disadvantage is it can result in lower quality and uses more memory for the stored morph targets.
	 * In most cases you want this unchecked and calculate the normals using a deformer graph or using the skeletal mesh's tangent recompute settings in the section details.
	 */
	UPROPERTY(EditAnywhere, Category = "Morph Targets", meta = (EditCondition = "CanDynamicallyUpdateMorphTargets()"))
	bool bIncludeNormals = false;

	/**
	 * Morph target delta values that are smaller than or equal to this threshold will be zeroed out.
	 * This essentially removes small deltas from morph targets, which will lower the memory usage at runtime, however when set too high it can also introduce visual artifacts.
	 * A value of 0 will result in the highest quality morph targets, at the cost of higher runtime memory usage.
	 * On the left side in the MLD asset editor you can see the estimated GPU Memory Usage. That value should change when you modify this property.
	 */
	UPROPERTY(EditAnywhere, Category = "Morph Targets", DisplayName = "Delta Zero Threshold", meta = (ClampMin = "0.0", ClampMax = "1.0", ForceUnits="cm", EditCondition = "CanDynamicallyUpdateMorphTargets()"))
	float MorphDeltaZeroThreshold = 0.001f;

	/** 
	 * The morph target compression level. Higher values result in larger compression, but could result in visual artifacts.
	 * Most of the times this is a value between 1 and 50. It is best to first try increasing the "Delta Zero Threshold" property as high as you can though.
	 * Once that is set to the highest acceptable value, try increasing this morph comrpession level as high as visually acceptable.
	 * On the left side in the MLD asset editor you can see the estimated GPU Memory Usage. That value should change when you modify this property.
	 */
	UPROPERTY(EditAnywhere, Category = "Morph Targets", DisplayName = "Compression Level", meta = (ClampMin = "0.01", ClampMax = "1000", EditCondition = "CanDynamicallyUpdateMorphTargets()"))
	float MorphCompressionLevel = 1.0f;

	/**
	 * The channel data that represents the delta mask multipliers.
	 * You can use this feather out influence of the ML Deformer in specific areas, such as neck line seams, where the head mesh connects with the body.
	 * The painted vertex color values will be like a weight multiplier on the ML deformer deltas applied to that vertex. You can invert the mask as well.
	 */
	UPROPERTY(EditAnywhere, Category = "Morph Targets", meta = (EditCondition = "CanDynamicallyUpdateMorphTargets()"))
	EMLDeformerMaskChannel MaskChannel = EMLDeformerMaskChannel::Disabled;

#if WITH_EDITORONLY_DATA
	/** The global vertex attribute attribute name. This is an attribute on the skeletal mesh, which can be created using the skeletal mesh editor. */
	UPROPERTY(EditAnywhere, Category = "Morph Targets", meta = (EditCondition = "CanDynamicallyUpdateMorphTargets() && MaskChannel == EMLDeformerMaskChannel::VertexAttribute", GetOptions = "GetVertexAttributeNames", NoResetToDefault))
	FName VertexAttributeName;
#endif

	/** 
	 * Enable this if you want to invert the mask channel values. For example if you painted the neck seam vertices in red, and you wish the vertices that got painted to NOT move, you have to invert the mask.
	 * On default you paint areas where the deformer should be active. If you enable the invert option, you paint areas where the deformer will not be active.
	 */
	UPROPERTY(EditAnywhere, Category = "Morph Targets", meta = (EditCondition = "CanDynamicallyUpdateMorphTargets() && MaskChannel != EMLDeformerMaskChannel::Disabled"))
	bool bInvertMaskChannel = false;

	/** The fence that let's us wait for all render commands to finish, before this instance is destroyed. */
	FRenderCommandFence RenderCommandFence;

	/** Set to true when all the morph targets got their deltas filtered out. This can happen when your global mask is all 0's for example. */
	bool bHasOnlyEmptyMorphs = true;
};

#undef UE_API
