// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerInputInfo.h"
#include "MLDeformerMorphModelInputInfo.generated.h"

#define UE_API MLDEFORMERFRAMEWORK_API

UCLASS(MinimalAPI)
class UMLDeformerMorphModelInputInfo
	: public UMLDeformerInputInfo
{
	GENERATED_BODY()

public:
	// UObject overrides.
	UE_API virtual void Serialize(FArchive& Archive) override;
	// ~END UObject overrides.

#if WITH_EDITORONLY_DATA
	UE_API TArray<float>& GetInputItemMaskBuffer();
	UE_API const TArray<float>& GetInputItemMaskBuffer() const;
	UE_API const TArrayView<const float> GetMaskForItem(int32 MaskItemIndex) const;

protected:
	/**
	 * The buffer of mask values, which contains one float per imported vertex, for all input items.
	 * An input item is an input bone or curve (or other thing).
	 * The buffer first contains all masks for all input bones followed by all curve masks.
	 * Each mask contains GetNumBaseMeshVerts() number of floats.
	 * This data is stripped at cook time, as it is not needed at runtime.
	 */
	UPROPERTY()
	TArray<float> InputItemMaskBuffer;
#endif
};

#undef UE_API
