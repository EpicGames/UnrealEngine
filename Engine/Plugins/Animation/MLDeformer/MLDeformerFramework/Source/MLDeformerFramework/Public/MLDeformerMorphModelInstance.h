// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerModelInstance.h"
#include "MLDeformerMorphModelInstance.generated.h"

#define UE_API MLDEFORMERFRAMEWORK_API

struct FExternalMorphSetWeights;
class USkeletalMeshComponent;

/**
 * The model instance for the UMLDeformerMorphModel.
 * This instance will assume the neural network outputs a set of weights, one for each morph target.
 * The weights of the morph targets in the external morph target set related to the ID of the model will
 * be set to the weights that the neural network outputs.
 * The first morph target contains the means, which need to always be added to the results. Therefore the 
 * weight of the first morph target will always be forced to a value of 1.
 */
UCLASS(MinimalAPI)
class UMLDeformerMorphModelInstance
	: public UMLDeformerModelInstance
{
	GENERATED_BODY()

public:
	// UMLDeformerModelInstance overrides.
	UE_API virtual void Init(USkeletalMeshComponent* SkelMeshComponent) override;
	UE_API virtual void PostMLDeformerComponentInit() override;
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void HandleZeroModelWeight() override;
	UE_API virtual bool IsValidForDataProvider() const override;
	UE_API virtual void PostTick(bool bExecuteCalled) override;
	// ~END UMLDeformerModelInstance overrides.

	UE_API int32 GetExternalMorphSetID() const;

	/**
	 * Find the external morph target weight data for this model instance.
	 * @param LOD The LOD level to get the weight data for.
	 * @return A pointer to the weight data, or a nullptr in case it cannot be found.
	 */
	UE_API FExternalMorphSetWeights* FindWeightData(int32 LOD) const;

protected:
	/** The next free morph target set ID. This is used to generate unique ID's for each morph model. */
	static UE_API TAtomic<int32> NextFreeMorphSetID;

	/** The ID of the external morph target set for this instance. This gets initialized during Init. */
	int32 ExternalMorphSetID = -1;
};

#undef UE_API
