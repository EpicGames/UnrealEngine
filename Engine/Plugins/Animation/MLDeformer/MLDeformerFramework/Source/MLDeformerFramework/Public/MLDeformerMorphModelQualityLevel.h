// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "MLDeformerMorphModelQualityLevel.generated.h"

/** 
 * A quality level for the Morph Model.
 * Each quality level contains the maximum number of active morph targets.
 */
USTRUCT()
struct FMLDeformerMorphModelQualityLevel
{
	GENERATED_USTRUCT_BODY()

public:
	int32 GetMaxActiveMorphs() const						{ return MaxActiveMorphs; }
	void SetMaxActiveMorphs(int32 NumMorphs)				{ MaxActiveMorphs = NumMorphs; }

public:
	UPROPERTY(EditAnywhere, Category = "Quality Level", meta = (ClampMin = "1"))
	int32 MaxActiveMorphs = 0;
};
