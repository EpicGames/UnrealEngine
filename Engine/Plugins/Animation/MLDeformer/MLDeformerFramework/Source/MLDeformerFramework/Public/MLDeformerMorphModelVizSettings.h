// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerGeomCacheVizSettings.h"
#include "MLDeformerMorphModelVizSettings.generated.h"

/**
 * The visualization settings specific to the UMLDeformerMorphModel class, or inherited classes.
 */
UCLASS(MinimalAPI)
class UMLDeformerMorphModelVizSettings
	: public UMLDeformerGeomCacheVizSettings
{
	GENERATED_BODY()

public:
	void SetMorphTargetNumber(int32 MorphIndexToPreview)	{ MorphTargetNumber = MorphIndexToPreview; }
	void SetDrawMorphTargets(bool bDraw)					{ bDrawMorphTargets = bDraw; }

	int32 GetMorphTargetNumber() const						{ return MorphTargetNumber; }
	bool GetDrawMorphTargets() const						{ return bDrawMorphTargets; }

	// Get property names.
	static FName GetMorphTargetNumberPropertyName()			{ return GET_MEMBER_NAME_CHECKED(UMLDeformerMorphModelVizSettings, MorphTargetNumber); }
	static FName GetDrawMorphTargetsPropertyName()			{ return GET_MEMBER_NAME_CHECKED(UMLDeformerMorphModelVizSettings, bDrawMorphTargets); }

protected:
	/**
	 * The morph target delta threshold. This is a preview of what deltas would be included in the selected morph target
	 * when using a delta threshold during training that is equal to this value.
	 * This only can be used after you trained, in the same editor session directly after training.
	 */
	UPROPERTY()
	float MorphTargetDeltaThreshold_DEPRECATED = 0.01f;

	/**
	 * The morph target to visualize. The first one always being the means, so not a sparse target.
	 * This only can be used after you trained, in the same editor session directly after training.
	 */
	UPROPERTY(EditAnywhere, Category = "Live Settings", meta = (ClampMin = "0"))
	int32 MorphTargetNumber = 0;

	/**
	 * Specify whether we want to debug draw the morph targets.
	 * Enabling this can help you see what the sparsity of the generated morph targets is.
	 * If the model supports input masking, it will also show the mask applied to the morph target.
	 */
	UPROPERTY(EditAnywhere, Category = "Live Settings")
	bool bDrawMorphTargets = false;
};
