// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

#define UE_API MLDEFORMERFRAMEWORK_API

namespace UE::MLDeformer
{
	// Custom serialization version for backwards compatibility during de-serialization
	struct FMLDeformerObjectVersion
	{
		FMLDeformerObjectVersion() = delete;
	
		enum Type
		{
			// Before any version changes were made.
			BeforeCustomVersionWasAdded,

			// We added LOD support.
			LODSupportAdded,

			// We deprecated the UNeuralMorphMaskInfo and should use FMLDeformerMaskInfo instead.
			MaskInfoMovedToFramework,

			// Undo support added.
			UndoSupportAdded,

			// ----- New versions can be added above this line -----
			VersionPlusOne,
			LatestVersion = VersionPlusOne - 1
		};

		// The GUID for this custom version number.
		UE_API const static FGuid GUID;
	};
}	// namespace UE::MLDeformer

#undef UE_API
