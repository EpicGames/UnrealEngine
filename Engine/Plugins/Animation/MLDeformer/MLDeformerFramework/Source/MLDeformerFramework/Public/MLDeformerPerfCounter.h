// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"

#define UE_API MLDEFORMERFRAMEWORK_API


namespace UE::MLDeformer
{
	class FMLDeformerPerfCounter
	{
	public:
		UE_API FMLDeformerPerfCounter();

		// Main methods.
		UE_API void SetHistorySize(int32 NumHistoryItems);
		UE_API void Reset();
		UE_API void BeginSample();
		UE_API void EndSample();

		// Get statistics.
		UE_API int32 GetCycles() const;
		UE_API int32 GetCyclesMin() const;
		UE_API int32 GetCyclesMax() const;
		UE_API int32 GetCyclesAverage() const;
		UE_API int32 GetNumSamples() const;
		UE_API int32 GetHistorySize() const;

	private:
		int32 StartTime = 0;
		int32 NumSamples = 0;
		int32 Cycles = 0;
		int32 CyclesMin = 0;
		int32 CyclesMax = 0;
		TArray<int32> CycleHistory;
	};
}

#undef UE_API
