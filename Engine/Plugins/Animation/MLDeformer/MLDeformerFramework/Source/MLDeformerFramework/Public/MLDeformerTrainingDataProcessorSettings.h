// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/SoftObjectPtr.h"
#include "MLDeformerTrainingDataProcessorSettings.generated.h"

#define UE_API MLDEFORMERFRAMEWORK_API

class UAnimSequence;
class USkeleton;

USTRUCT()
struct FMLDeformerTrainingDataProcessorAnim
{
	GENERATED_BODY()

public:
	/** The animation sequence that we should sample frames from. */
	UPROPERTY(EditAnywhere, Category = "", meta = (NoResetToDefault))
	TSoftObjectPtr<UAnimSequence> AnimSequence;

	/** Should we sample frames from this animation sequence? */
	UPROPERTY(EditAnywhere, Category = "")
	bool bEnabled = true;
};


USTRUCT()
struct FMLDeformerTrainingDataProcessorBoneGroup
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FString GroupName;

	UPROPERTY()
	TArray<FName> BoneNames;
};


USTRUCT()
struct FMLDeformerTrainingDataProcessorBoneGroupsList
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TArray<FMLDeformerTrainingDataProcessorBoneGroup> Groups;
};


USTRUCT()
struct FMLDeformerTrainingDataProcessorBoneList
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TArray<FName> BoneNames;
};


/**
 * The settings for the ML Deformer Training Data Processor tool.
 * These settings are also stored along with the ML Deformer asset.
 */
UCLASS(MinimalAPI)
class UMLDeformerTrainingDataProcessorSettings
	: public UObject
{
	GENERATED_BODY()

public:
	UMLDeformerTrainingDataProcessorSettings() = default;

#if WITH_EDITOR
	/**
	 * This returns a pointer to the skeleton used by the SkeletalMesh that is set in the UMLDeformerModel that this settings class is part of.
	 * If there is no skeletal mesh set on the model, a nullptr will be returned.
	 * @return A pointer to the skeleton we should be using, or nullptr if not found.
	 */
	UE_API USkeleton* FindSkeleton() const;

	/**
	 * Get the number of frames of all enabled input animations combined.
	 * This will ignore disabled input animations. It will try to load the animation sequences and get their number of frames and sum them up.
	 * @return The number of frames that would be sampled.
	 */
	UE_API int32 GetNumInputAnimationFrames() const;

	/**
	 * Check whether these settings are valid when used with a given skeleton.
	 * The skeleton we use is likely the skeleton used by the ML Deformer asset.
	 * @param Skeleton The skeleton to check against. This makes sure the input animations must be the same as this provided skeleton.
	 * @return Returns true if this configuration is valid, or false otherwise.
	 */
	UE_API bool IsValid(const USkeleton* Skeleton) const;
#endif
	
public:
	/**
	 * The list of animation sequences from which we will grab a pose for each frame.
	 * This final list of poses then will go through the pose remixing and reduction steps.
	 */
	UPROPERTY(EditAnywhere, Category = "Input Animations", meta = (NoResetToDefault))
	TArray<FMLDeformerTrainingDataProcessorAnim> AnimList;

	/**
	 * The list of bones that is used to calculate which poses have most variation.
	 * It does not influence the number of output bones in the output animation.
	 */
	UPROPERTY(EditAnywhere, Category = "Frame Reduction", meta = (NoResetToDefault))
	FMLDeformerTrainingDataProcessorBoneList BoneList;

	/**
	 * The list of bone groups used during pose remixing.
	 * Pose remixing will basically shuffle the keyframes for the group of bones as a whole.
	 * So if you put the left arm bones in a group, the frame numbers of the arm will be shuffled, but
	 * while keeping all the arm bones at the same frame. The rest of the body can be in a different frame.
	 * This might sound strange, but it can help the machine learning deformer algorithm produce better reconstructions.
	 */
	UPROPERTY(EditAnywhere, Category = "Pose Remixing", meta = (NoResetToDefault))
	FMLDeformerTrainingDataProcessorBoneGroupsList BoneGroups;

	/**
	 * The number of output frames you want the output animation sequence to have.
	 * If this number is higher than the number of frames in the input animations, it will just use the maximum
	 * number of available frames.
	 * This can be used to reduce the number of frames for training, which is useful if you want to reduce the size of your training data set.
	 * This will also change the reordering of frames, so it is NOT just going to cut off some frames at the end.
	 * It will use a greedy algorithm to find poses that are as different as possible from each other.
	 * By doing that we hope to get as much pose space coverage as possible.
	 * This setting is only used when the "Reduce Frames" option is enabled.
	 */
	UPROPERTY(EditAnywhere, Category = "Output", meta = (ClampMin="1", EditCondition = "bReduceFrames"))
	int32 NumOutputFrames = 5000;

	/**
	 * Specify whether we should reduce the number of frames that we sampled from the input animations, or not.
	 * This can be used to reduce the number of frames for training, which is useful if you want to reduce the size of your training data set.
	 * This will also change the reordering of frames, so it is NOT just going to cut off some frames at the end.
	 * It will use a greedy algorithm to find poses that are as different as possible from each other.
	 * By doing that we hope to get as much pose space coverage as possible.
	 * This setting is only used when the "Reduce Frames" option is enabled.
	 */
	UPROPERTY(EditAnywhere, Category = "Output")
	bool bReduceFrames = true;

	/**
	 * Enable this when you want to perform pose remixing.
	 * Pose remixing will basically shuffle the keyframes for the group of bones as a whole.
	 * So if you put the left arm bones in a group, the frame numbers of the arm will be shuffled, but
	 * while keeping all the arm bones at the same frame. The rest of the body can be in a different frame.
	 * This might sound strange, but it can help the machine learning deformer algorithm produce better reconstructions.
	 */
	UPROPERTY(EditAnywhere, Category = "Output")
	bool bRemixPoses = true;

	/**
	 * The random seed that is used during pose remixing.
	 * The remixing algorithm shuffles frames, which uses randomization.
	 * In order to always get the same results if we perform pose remixing on the same data set, we have to use a fixed seed.
	 * Changing the seed could result in a different order of frames.
	 */
	UPROPERTY(EditAnywhere, Category = "Output", meta = (ClampMin = "0", EditCondition = "bRemixPoses"))
	int32 RandomSeed = 777;

	/**
	 * The output animation sequence.
	 * This animation sequence will be modified and will be filled with the generated frames.
	 * Please watch out with picking existing animation sequences, as their contents will be overwritten after you press Generate.
	 */
	UPROPERTY(EditAnywhere, Category = "Output", meta = (NoResetToDefault))
	TSoftObjectPtr<UAnimSequence> OutputAnimSequence;
};

#undef UE_API
