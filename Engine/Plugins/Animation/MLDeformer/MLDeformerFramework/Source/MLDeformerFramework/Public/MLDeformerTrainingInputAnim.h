// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPtr.h"
#include "Animation/AnimSequence.h"
#include "MLDeformerTrainingInputAnim.generated.h"

/**
 * An animation that is input to the training process.
 * This is a base struct that only contains the animation sequence.
 * For example a geometry cache based model should inherit from this struct and add a geom cache member property.
 */
USTRUCT()
struct FMLDeformerTrainingInputAnim
{
	GENERATED_BODY()

public:
	virtual ~FMLDeformerTrainingInputAnim() = default;

#if WITH_EDITORONLY_DATA
	/** 
	 * Get the number of frames that we are sampling inside this animation. This can be less than the value returned by ExtractNumAnimFrames().
	 * This might be the custom frame range, so a subsection of the animation frames.
	 */
	virtual int32 GetNumFramesToSample() const					{ ensureMsgf(false, TEXT("Please overload the FMLDeformerTrainingInputAnim::GetNumFramesToSample() method.")); return 0; }

	/** Is this input animation valid? Returns true if we can sample from it. This typically checks if the pointers to the source and target animations are valid. */
	virtual bool IsValid() const								{ return GetAnimSequence() != nullptr; }

	/** How many frames does this input animation have? This is the maximum number of frames we can sample. */
	virtual int32 ExtractNumAnimFrames() const					{ const UAnimSequence* Anim = GetAnimSequence(); return Anim ? Anim->GetNumberOfSampledKeys() : 0; }

	void SetEnabled(bool bEnable)								{ bEnabled = bEnable; }
	void SetAnimSequence(TSoftObjectPtr<UAnimSequence> Seq)		{ AnimSequence = Seq; }
	void SetUseCustomRange(bool bCustomRange)					{ bUseCustomRange = bCustomRange; }
	void SetStartFrame(int32 FrameIndex)						{ StartFrame = FrameIndex; }
	void SetEndFrame(int32 FrameIndex)							{ EndFrame = FrameIndex; }
	void SetVertexMask(FName MaskName)							{ VertexMask = MaskName; }

	bool IsEnabled() const										{ return bEnabled; }
	bool GetUseCustomRange() const								{ return bUseCustomRange; }
	int32 GetStartFrame() const									{ return StartFrame; }
	int32 GetEndFrame() const									{ return EndFrame; }
	FName GetVertexMask() const									{ return VertexMask; }
	const UAnimSequence* GetAnimSequence() const				{ return AnimSequence.LoadSynchronous(); }
	UAnimSequence* GetAnimSequence()							{ return AnimSequence.LoadSynchronous(); }
	const TSoftObjectPtr<UAnimSequence>& GetAnimSequenceSoftObjectPtr() const	{ return AnimSequence; } 
	TSoftObjectPtr<UAnimSequence>& GetAnimSequenceSoftObjectPtr()				{ return AnimSequence; }

	static FName GetAnimSequencePropertyName()					{ return GET_MEMBER_NAME_CHECKED(FMLDeformerTrainingInputAnim, AnimSequence); }
	static FName GetEnabledPropertyName()						{ return GET_MEMBER_NAME_CHECKED(FMLDeformerTrainingInputAnim, bEnabled); }
	static FName GetUseCustomRangePropertyName()				{ return GET_MEMBER_NAME_CHECKED(FMLDeformerTrainingInputAnim, bUseCustomRange); }
	static FName GetStartFramePropertyName()					{ return GET_MEMBER_NAME_CHECKED(FMLDeformerTrainingInputAnim, StartFrame); }
	static FName GetEndFramePropertyName()						{ return GET_MEMBER_NAME_CHECKED(FMLDeformerTrainingInputAnim, EndFrame); }
	static FName GetVertexMaskPropertyName()					{ return GET_MEMBER_NAME_CHECKED(FMLDeformerTrainingInputAnim, VertexMask); }

private:
	/** The animation sequence. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (DisplayPriority = 0))	// Show as first property.
	TSoftObjectPtr<UAnimSequence> AnimSequence;

	/** On default all frames are included, unless we specify a custom frame include range. */
	UPROPERTY(EditAnywhere, Category = "Frame Settings")
	bool bUseCustomRange = false;

	/** The start frame number of the range of included frames. Only used when not all frames are included. */
	UPROPERTY(EditAnywhere, Category = "Frame Settings", meta = (ClampMin="0", EditCondition="bUseCustomRange"))
	int32 StartFrame = 0;

	/** The end frame number of the range of included frames. Only used when not all frames are included. */
	UPROPERTY(EditAnywhere, Category = "Frame Settings", meta = (ClampMin="0", EditCondition="bUseCustomRange"))
	int32 EndFrame = 0;

	/**
	 * The optional vertex attribute mask to use for this training data. This can be used to focus training on a specific area on the mesh.
	 * For example if you have training data for a hand, you can specify a mask that contains just the hand and maybe forearm.
	 */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FName VertexMask;

	/** Is this animation enabled? If not, it is excluded from the training process. */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (DisplayPriority = MAX_int32))	// Show as last property.
	bool bEnabled = true;
#endif	// #if WITH_EDITORONLY_DATA
};

struct FMLDeformerTrainingInputAnimName
{
	/** The name to display in the timeline combobox. */
	FString Name;

	/** The training input anim index. This is the index inside the FMLDeformerEditorModel::GetTrainingInputAnim(Index) method. */
	int32 TrainingInputAnimIndex = INDEX_NONE;
};
