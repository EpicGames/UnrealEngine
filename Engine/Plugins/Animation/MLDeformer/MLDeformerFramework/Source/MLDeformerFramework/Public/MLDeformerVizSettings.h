// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "Animation/MeshDeformer.h"
#include "Animation/AnimSequence.h"
#include "MLDeformerAsset.h"
#include "MLDeformerVizSettings.generated.h"

/** The visualization mode, which selects whether you want to view the training data, or test your already trained model. */
UENUM()
enum class EMLDeformerVizMode : uint8
{
	/** Preview the training data. */
	TrainingData = 0,

	/** Preview testing data, used on trained models. */
	TestData
};

/** The heat map mode which selects what the colors of the heatmap represent. */
UENUM()
enum class EMLDeformerHeatMapMode : uint8
{
	/** Visualize areas where the deformer is applying corrections. The color represents the size of the correction it applies. */
	Activations = 0,

	/** Visualize the error versus the ground truth model. Requires a ground truth model to be setup. */
	GroundTruth
};

/** A comparison actor. */
USTRUCT()
struct FMLDeformerCompareActor
{
	GENERATED_BODY()

	/** The name of the comparison actor, which is the label shown above it. */
	UPROPERTY(EditAnywhere, Category = "Actor Settings")
	FName Name;

	/** The ML Deformer asset that this actor should use. */
	UPROPERTY(EditAnywhere, Category = "Actor Settings")
	TSoftObjectPtr<UMLDeformerAsset> DeformerAsset;
};

/**
 * The visualization settings.
 */
UCLASS(MinimalAPI)
class UMLDeformerVizSettings
	: public UObject
{
	GENERATED_BODY()

public:
	// UObject overrides.
	virtual bool IsEditorOnly() const override				{ return true; }
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) { GetOuter()->PostEditChangeProperty(PropertyChangedEvent); } // Forward to the UMLDeformerModel.
#endif
	// ~END UObject overrides.

#if WITH_EDITORONLY_DATA
	virtual bool HasTestGroundTruth() const					{ return false; }

	void SetVisualizationMode(EMLDeformerVizMode Mode)		{ VisualizationMode = Mode; }
	void SetDeformerGraph(UMeshDeformer* InDeformerGraph)	{ DeformerGraph = InDeformerGraph; }
	void SetTrainingFrameNumber(int32 FrameNumber)			{ TrainingFrameNumber = FrameNumber; }
	void SetTestingFrameNumber(int32 FrameNumber)			{ TestingFrameNumber = FrameNumber; }
	void SetWeight(float InWeight)							{ Weight = InWeight; }
	void SetTestAnimSequence(UAnimSequence* InAnim)  		{ TestAnimSequence = InAnim; }
	void SetMeshSpacing(float Spacing)						{ MeshSpacing = Spacing; }
	void SetLabelHeight(float Height)						{ LabelHeight = Height; }
	void SetLabelScale(float Scale)							{ LabelScale = Scale; }
	void SetDrawLabels(bool bDraw)							{ bDrawLabels = bDraw; }
	void SetAnimPlaySpeed(float SpeedFactor)				{ AnimPlaySpeed = SpeedFactor; }
	void SetDrawLinearSkinnedActor(bool bDraw)				{ bDrawLinearSkinnedActor = bDraw; }
	void SetDrawMLDeformedActor(bool bDraw)					{ bDrawMLDeformedActor = bDraw; }
	void SetDrawGroundTruthActor(bool bDraw)				{ bDrawGroundTruthActor = bDraw; }
	void SetDrawMLCompareActors(bool bDraw)					{ bDrawMLCompareActors = bDraw; }
	void SetDrawDebugActorBounds(bool bDraw)				{ bDrawDebugActorBounds = bDraw; }
	void SetDrawVertexDeltas(bool bDraw)					{ bDrawDeltas = bDraw; }
	void SetXRayDeltas(bool bEnable)						{ bXRayDeltas = bEnable; }
	void SetHeatMapMax(float Value)							{ HeatMapMax = Value; }
	void SetGroundTruthLerp(float Alpha)					{ GroundTruthLerp = Alpha; }
	void SetShowHeatMap(bool bShow)							{ bShowHeatMap = bShow; }
	void SetHeatMapMode(EMLDeformerHeatMapMode Mode)		{ HeatMapMode = Mode; }
	void SetDebugBoundsColor(const FColor& Color)			{ DebugBoundsColor = Color; }
	void SetCompareActors(const TArray<FMLDeformerCompareActor>& Actors) { CompareActors = Actors; }

	FVector GetMeshSpacingOffsetVector() const				{ return FVector(MeshSpacing, 0.0f, 0.0f); }
	float GetMeshSpacing() const							{ return MeshSpacing; }
	float GetLabelHeight() const							{ return LabelHeight; }
	bool GetDrawLabels() const								{ return bDrawLabels; }
	float GetLabelScale() const								{ return LabelScale; }
	EMLDeformerVizMode GetVisualizationMode() const			{ return VisualizationMode; }
	int32 GetTrainingFrameNumber() const					{ return TrainingFrameNumber; }
	int32 GetTestingFrameNumber() const						{ return TestingFrameNumber; }
	float GetAnimPlaySpeed() const							{ return AnimPlaySpeed; }
	const UAnimSequence* GetTestAnimSequence() const		{ return TestAnimSequence.LoadSynchronous(); }
	UAnimSequence* GetTestAnimSequence()					{ return TestAnimSequence.LoadSynchronous(); }
	bool GetDrawLinearSkinnedActor() const					{ return bDrawLinearSkinnedActor; }
	bool GetDrawMLDeformedActor() const						{ return bDrawMLDeformedActor; }
	bool GetDrawGroundTruthActor() const					{ return bDrawGroundTruthActor; }
	bool GetDrawMLCompareActors() const						{ return bDrawMLCompareActors; }
	bool GetDrawDebugActorBounds() const					{ return bDrawDebugActorBounds; }
	bool GetShowHeatMap() const								{ return bShowHeatMap; }
	EMLDeformerHeatMapMode GetHeatMapMode() const			{ return HeatMapMode; }
	float GetHeatMapMax() const								{ return HeatMapMax; }
	float GetGroundTruthLerp() const						{ return HeatMapMode == EMLDeformerHeatMapMode::GroundTruth ? GroundTruthLerp : 0.0f; }
	UMeshDeformer* GetDeformerGraph() const					{ return DeformerGraph.LoadSynchronous(); }
	float GetWeight() const									{ return Weight; }
	bool GetXRayDeltas() const								{ return bXRayDeltas; }
	bool GetDrawVertexDeltas() const						{ return bDrawDeltas; }
	const TArray<FMLDeformerCompareActor>& GetCompareActors() const { return CompareActors; }
	TArray<FMLDeformerCompareActor>& GetCompareActors()		{ return CompareActors; }
	FColor GetDebugBoundsColor() const						{ return DebugBoundsColor; }

	// Get property names.
	static FName GetVisualizationModePropertyName()			{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, VisualizationMode); }
	static FName GetMeshSpacingPropertyName()				{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, MeshSpacing); }
	static FName GetDrawLabelsPropertyName()				{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, bDrawLabels); }
	static FName GetLabelHeightPropertyName()				{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, LabelHeight); }
	static FName GetLabelScalePropertyName()				{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, LabelScale); }
	static FName GetTrainingFrameNumberPropertyName()		{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, TrainingFrameNumber); }
	static FName GetTestingFrameNumberPropertyName()		{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, TestingFrameNumber); }
	static FName GetAnimPlaySpeedPropertyName()				{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, AnimPlaySpeed); }
	static FName GetTestAnimSequencePropertyName()			{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, TestAnimSequence); }
	static FName GetDrawLinearSkinnedActorPropertyName()	{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, bDrawLinearSkinnedActor); }
	static FName GetDrawMLDeformedActorPropertyName()		{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, bDrawMLDeformedActor); }
	static FName GetDrawGroundTruthActorPropertyName()		{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, bDrawGroundTruthActor); }
	static FName GetDrawMLCompareActorsPropertyName()		{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, bDrawMLCompareActors); }
	static FName GetDrawDebugActorBoundsPropertyName()		{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, bDrawDebugActorBounds); }
	static FName GetShowHeatMapPropertyName()				{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, bShowHeatMap); }
	static FName GetHeatMapModePropertyName()				{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, HeatMapMode); }
	static FName GetHeatMapMaxPropertyName()				{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, HeatMapMax); }
	static FName GetGroundTruthLerpPropertyName()			{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, GroundTruthLerp); }
	static FName GetDeformerGraphPropertyName()				{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, DeformerGraph); }
	static FName GetWeightPropertyName()					{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, Weight); }
	static FName GetXRayDeltasPropertyName()				{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, bXRayDeltas); }
	static FName GetDrawVertexDeltasPropertyName()			{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, bDrawDeltas); }
	static FName GetCompareActorsPropertyName()				{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, CompareActors); }
	static FName GetDebugBoundsColorPropertyName()			{ return GET_MEMBER_NAME_CHECKED(UMLDeformerVizSettings, DebugBoundsColor); }
#endif

protected:
#if WITH_EDITORONLY_DATA
	/** The data to visualize. */
	UPROPERTY()
	EMLDeformerVizMode VisualizationMode = EMLDeformerVizMode::TrainingData;

	/** The animation sequence to play on the skeletal mesh. */
	UPROPERTY(EditAnywhere, Category = "Test Assets")
	TSoftObjectPtr<UAnimSequence> TestAnimSequence;

	/** The deformer graph to use on the asset editor's deformed test actor. */
	UPROPERTY(EditAnywhere, Category = "Test Assets")
	TSoftObjectPtr<UMeshDeformer> DeformerGraph;

	/**
	 * The model comparison actors. Each will create a skeletal mesh with the selected ML Deformer applied to it.
	 * This is useful to compare the output of different models side by side. 
	 */
	UPROPERTY(EditAnywhere, Category = "Test Assets")
	TArray<FMLDeformerCompareActor> CompareActors;

	/** The play speed factor of the test anim sequence. */
	UPROPERTY(EditAnywhere, Category = "Live Settings", meta = (ClampMin = "0.0", ClampMax = "2.0", ForceUnits="Multiplier"))
	float AnimPlaySpeed = 1.0f;

	/** The frame number of the training data to visualize. */
	UPROPERTY(EditAnywhere, Category = "Training Meshes", meta = (ClampMin = "0"))
	int32 TrainingFrameNumber = 0;

	/** Specifies whether we should draw the labels, such as "Linear Skinned" and "ML Deformed" or not. */
	UPROPERTY(EditAnywhere, Category = "Shared Settings")
	bool bDrawLabels = true;

	/** The height in units to draw the labels at. */
	UPROPERTY(EditAnywhere, Category = "Shared Settings", meta = (EditCondition = "bDrawLabels", ForceUnits="cm"))
	float LabelHeight = 200.0f;

	/** The scale of the label text. */
	UPROPERTY(EditAnywhere, Category = "Shared Settings", meta = (ClampMin = "0.001", EditCondition = "bDrawLabels", ForceUnits="Multiplier"))
	float LabelScale = 1.0f;

	/** The spacing between meshes. */
	UPROPERTY(EditAnywhere, Category = "Shared Settings", meta = (ClampMin = "0", ForceUnits="cm"))
	float MeshSpacing = 125.0f;

	/** The frame number of the test data to visualize. */
	UPROPERTY(EditAnywhere, Category = "Live Settings", meta = (ClampMin = "0"))
	int32 TestingFrameNumber = 0;

	/** Deprecated quality level property, deprecated in 5.4. */
	UPROPERTY(Transient)
	int32 QualityLevel_DEPRECATED = 0;

	/** Specify whether the heatmap is enabled or not. */
	UPROPERTY(EditAnywhere, Category = "Live Settings")
	bool bShowHeatMap = false;

	/** Specifies what the heatmap colors should represent. */
	UPROPERTY(EditAnywhere, Category = "Live Settings", meta = (EditCondition = "bShowHeatMap"))
	EMLDeformerHeatMapMode HeatMapMode = EMLDeformerHeatMapMode::Activations;

	/**
	 * Specifies how many centimeters the most intense color of the heatmap represents.
	 * For example when set to 3, it means that everything that is red, is 3 cm or more.
	 * In Activations heat map mode this means that everything is red for all vertices with a delta longer than 3 cm, so with a corrective delta longer than 3 cm.
	 * In Ground Truth heat map mode this means that everything is red for all vertices where the difference between the ground truth and the ML Deformed
	 * model is larger than 3 cm.
	 */
	UPROPERTY(EditAnywhere, Category = "Live Settings", meta = (EditCondition = "bShowHeatMap", ClampMin = "0.001", ForceUnits="cm"))
	float HeatMapMax = 1.0f;

	/**
	 * The Lerp factor from ML deformed model to ground truth model when in heat map mode. 
	 * A value of 0 means that we look exactly the same as the ML Deformed output, while a value of 1 means
	 * the output is exactly the same as the ground truth. This is the blend factor between those two.
	 */
	UPROPERTY(EditAnywhere, Transient, Category = "Live Settings", meta = (EditCondition = "HeatMapMode==EMLDeformerHeatMapMode::GroundTruth && bShowHeatMap", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float GroundTruthLerp = 0.0f;

	/** Specifies whether we draw the linear skinned model or not. */
	UPROPERTY(EditAnywhere, Category = "Live Settings")
	bool bDrawLinearSkinnedActor = true;

	/** Specifies whether we draw the ML Deformed model or not. */
	UPROPERTY(EditAnywhere, Category = "Live Settings", DisplayName = "Draw ML Deformed Actor")
	bool bDrawMLDeformedActor = true;

	/** Specifies whether we draw the ground truth model or not. */
	UPROPERTY(EditAnywhere, Category = "Live Settings")
	bool bDrawGroundTruthActor = true;

	/** Specifies whether we draw the comparison actors or not. */
	UPROPERTY(EditAnywhere, Category = "Live Settings", DisplayName = "Draw ML Compare Actors")
	bool bDrawMLCompareActors = true;

	/** Draw the debug actor's bounds in the PIE viewport? This only renders when PIE is active. */
	UPROPERTY(EditAnywhere, Category = "Live Settings", DisplayName = "Highlight Debug Actors in PIE")
	bool bDrawDebugActorBounds = true;

	/** 
	 * The color of the bounding box rendered inside the PIE viewport.
	 * Keep in mind that the actor you are currently debugging is always rendered in green.
	 */
	UPROPERTY(EditAnywhere, Category = "Live Settings")
	FColor DebugBoundsColor = FColor::Purple;

	/** The scale factor of the ML deformer deltas being applied on top of the linear skinned results. */
	UPROPERTY(EditAnywhere, Transient, Category = "Live Settings", meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float Weight = 1.0f;

	/** Specifiy whether we want to draw the vertex deltas or not. */
	UPROPERTY(EditAnywhere, Category = "Training Meshes")
	bool bDrawDeltas = true;

	/** Specify whether the vertex deltas are rendered, even if they are behind the current mesh. */
	UPROPERTY(EditAnywhere, Category = "Training Meshes", meta = (EditCondition = "bDrawDeltas"))
	bool bXRayDeltas = true;
#endif
};
