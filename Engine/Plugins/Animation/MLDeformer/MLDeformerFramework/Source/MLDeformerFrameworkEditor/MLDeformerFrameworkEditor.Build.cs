// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class MLDeformerFrameworkEditor : ModuleRules
	{
		public MLDeformerFrameworkEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateIncludePaths.AddRange(
				new string[] 
				{
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"SequencerWidgets"
				}
			);

			PublicIncludePathModuleNames.AddRange(
				new string[]
				{
					"GeometryCache",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"InputCore",
					"EditorFramework",
					"UnrealEd",
					"CoreUObject",
					"Engine",
					"Slate",
					"SlateCore",
					"EditorStyle",
					"Persona",
					"MLDeformerFramework",
					"Projects",
					"PropertyEditor",
					"AnimationEditMode",
					"AnimGraph",
					"ToolWidgets",
					"EditorWidgets",
					"GeometryCache",
					"TimeManagement",
					"RenderCore",
					"RHI",
					"Json",
					"AssetDefinition",
					"SkeletalMeshDescription",
					"ApplicationCore",
					"PythonScriptPlugin",
					"EditorInteractiveToolsFramework",
					"InteractiveToolsFramework",
					"MeshModelingTools",
					"MeshModelingToolsEditorOnly",
					"MeshModelingToolsEditorOnlyExp",
					"MeshModelingToolsExp",
					"ModelingComponentsEditorOnly",
					"ModelingToolsEditorMode",
					"ModelingComponents",
					"SkeletalMeshEditor",
					"SkeletalMeshUtilitiesCommon",
					"WidgetRegistration",
					"StatusBar"
				}
			);

			PublicIncludePathModuleNames.AddRange(
				new string[]
				{
					"Persona",
				}
			);
		}
	}
}
