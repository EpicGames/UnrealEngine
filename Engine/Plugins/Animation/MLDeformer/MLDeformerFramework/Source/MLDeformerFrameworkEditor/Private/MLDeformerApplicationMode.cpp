// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerApplicationMode.h"
#include "MLDeformerEditorToolkit.h"
#include "MLDeformerVizSettingsTabSummoner.h"
#include "MLDeformerTimelineTabSummoner.h"
#include "PersonaModule.h"
#include "Modules/ModuleManager.h"
#include "PersonaTabs.h"

#define LOCTEXT_NAMESPACE "MLDeformerApplicationMode"

namespace UE::MLDeformer
{
	FName FMLDeformerApplicationMode::ModeName("MLDeformerAssetEditMode");
	
	FMLDeformerApplicationMode::FMLDeformerApplicationMode(
		TSharedRef<FWorkflowCentricApplication> InHostingApp,
		TSharedRef<IPersonaPreviewScene> InPreviewScene)
		: FApplicationMode(MLDeformerEditorModes::Editor)
	{
		EditorToolkit = StaticCastSharedRef<FMLDeformerEditorToolkit>(InHostingApp);
		TSharedRef<FMLDeformerEditorToolkit> MLDeformerEditor = StaticCastSharedRef<FMLDeformerEditorToolkit>(InHostingApp);

		FPersonaViewportArgs ViewportArgs(InPreviewScene);
		ViewportArgs.bAlwaysShowTransformToolbar = true;
		ViewportArgs.bShowStats = false;
		ViewportArgs.bShowTimeline = false;
		ViewportArgs.bShowLODMenu = true;
		ViewportArgs.bShowPlaySpeedMenu = false;
		ViewportArgs.bShowPhysicsMenu = false;
		ViewportArgs.ContextName = TEXT("MLDeformerEditor.Viewport");
		ViewportArgs.OnViewportCreated = FOnViewportCreated::CreateSP(MLDeformerEditor, &FMLDeformerEditorToolkit::HandleViewportCreated);

		// Hide certain property categories inside the advanced preview scene settings tab.
		FPersonaAdvancedPreviewSceneTabSettings AdvancedPreviewSceneTabSettings;
		AdvancedPreviewSceneTabSettings.bHideAdditionalMeshesCategory = true;
		AdvancedPreviewSceneTabSettings.bHideAnimationCategory = true;
		AdvancedPreviewSceneTabSettings.bHideMeshCategory = true;
		AdvancedPreviewSceneTabSettings.bHidePhysicsCategory = true;
		AdvancedPreviewSceneTabSettings.bHideAnimationBlueprintCategory = true;

		// Register Persona tabs.
		FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
		TabFactories.RegisterFactory(PersonaModule.CreatePersonaViewportTabFactory(InHostingApp, ViewportArgs));
		TabFactories.RegisterFactory(PersonaModule.CreateAdvancedPreviewSceneTabFactory(InHostingApp, InPreviewScene, AdvancedPreviewSceneTabSettings));
		TabFactories.RegisterFactory(PersonaModule.CreatePersonaToolboxTabFactory(MLDeformerEditor));
		TabFactories.RegisterFactory(PersonaModule.CreateDetailsTabFactory(InHostingApp, FOnDetailsCreated::CreateSP(&MLDeformerEditor.Get(), &FMLDeformerEditorToolkit::HandleDetailsCreated)));

		// Register custom tabs.
		TabFactories.RegisterFactory(MakeShared<FMLDeformerVizSettingsTabSummoner>(MLDeformerEditor));
		TabFactories.RegisterFactory(MakeShared<FMLDeformerTimelineTabSummoner>(MLDeformerEditor));
		for (const TUniquePtr<FToolsMenuExtender>& Extender : FMLDeformerEditorToolkit::GetToolsMenuExtenders())
		{
			if (TSharedPtr<FWorkflowTabFactory> Factory = Extender->GetTabSummoner(MLDeformerEditor); Factory.IsValid())
			{
				TabFactories.RegisterFactory(Factory.ToSharedRef());
			}
		}

		// Create tab layout.
		TabLayout = FTabManager::NewLayout("Standalone_MLDeformerEditor_Layout_v5.5.001")
			->AddArea
			(
				FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.32f)
					->SetOrientation(Orient_Horizontal)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.3f)
						->AddTab(FMLDeformerVizSettingsTabSummoner::TabID, ETabState::OpenedTab)
						->AddTab(FPersonaTabs::ToolboxID, ETabState::OpenedTab)
						->SetForegroundTab(FMLDeformerVizSettingsTabSummoner::TabID)
					)
					->Split
					(
						FTabManager::NewSplitter()
						->SetSizeCoefficient(1.0f)
						->SetOrientation(Orient_Vertical)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.7f)
							->SetHideTabWell(true)
							->AddTab(FPersonaTabs::PreviewViewportID, ETabState::OpenedTab)
						)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.03f)
							->SetHideTabWell(true)
							->AddTab(FMLDeformerTimelineTabSummoner::TabID, ETabState::OpenedTab)
						)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.3f)
						->AddTab(FPersonaTabs::DetailsID, ETabState::OpenedTab)
						->SetForegroundTab(FPersonaTabs::DetailsID)
					)
				)
			);

		PersonaModule.OnRegisterTabs().Broadcast(TabFactories, InHostingApp);
		LayoutExtender = MakeShared<FLayoutExtender>();
		PersonaModule.OnRegisterLayoutExtensions().Broadcast(*LayoutExtender.Get());
		TabLayout->ProcessExtensions(*LayoutExtender.Get());
	}

	void FMLDeformerApplicationMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
	{
		TSharedPtr<FMLDeformerEditorToolkit> Editor = EditorToolkit.Pin();
		Editor->RegisterTabSpawners(InTabManager.ToSharedRef());
		Editor->PushTabFactories(TabFactories);
	}

}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
