// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerAssetDefinitions.h"

#include "MLDeformerAsset.h"
#include "MLDeformerEditorToolkit.h"
#include "MLDeformerModule.h"
#include "IPythonScriptPlugin.h"
#include "PipInstallHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MLDeformerAssetDefinitions)

#define LOCTEXT_NAMESPACE "MLDeformer_AssetTypeActions"

FText UAssetDefinition_MLDeformer::GetAssetDisplayName() const
{ 
	return LOCTEXT("AssetTypeActions_MLDeformer", "ML Deformer");
}

FLinearColor UAssetDefinition_MLDeformer::GetAssetColor() const
{
	return FColor(255, 255, 0);
}

TSoftClassPtr<UObject> UAssetDefinition_MLDeformer::GetAssetClass() const
{ 
	return UMLDeformerAsset::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_MLDeformer::GetAssetCategories() const
{ 
	static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Animation, LOCTEXT("AssetDefinition_MLDeformerSubMenu", "Deformers"))
		};
	return Categories;
}

EAssetCommandResult UAssetDefinition_MLDeformer::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	using namespace UE::MLDeformer;
	FPythonScriptInitHelper::InitPythonAndPipInstall(FSimpleDelegate::CreateLambda([
		OpenArgs = FAssetOpenArgs(OpenArgs),
		// Taking a copy of OpenArgs.Assets as it's a TArrayView and this callback outlives the view
		OpenAssetsData = TArray<FAssetData>(OpenArgs.Assets)]()
	{
		FAssetArgs WrappedArgs(OpenAssetsData);
		for (UMLDeformerAsset* Asset : WrappedArgs.LoadObjects<UMLDeformerAsset>())
		{
			TSharedRef<FMLDeformerEditorToolkit> NewEditor(new FMLDeformerEditorToolkit());
			NewEditor->InitAssetEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Asset);
		}
	}),
	FSimpleDelegate::CreateLambda([]()
	{
		// TODO: Could alternatively open editor here and just log that python isn't available
		UE_LOGF(LogMLDeformer, Warning, "MLDeformer toolkit may not function properly without Python enabled");
	}));
	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
