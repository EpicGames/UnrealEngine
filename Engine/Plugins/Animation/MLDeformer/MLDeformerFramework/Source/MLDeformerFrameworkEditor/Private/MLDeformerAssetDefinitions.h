// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "AssetDefinitionDefault.h"
#include "MLDeformerAssetDefinitions.generated.h"

UCLASS()
class UAssetDefinition_MLDeformer
	: public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition overrides
	FText GetAssetDisplayName() const override final;
	FLinearColor GetAssetColor() const override final;
	TSoftClassPtr<UObject> GetAssetClass() const override final;
	TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override final;
	EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override final;
	// ~END UAssetDefinition overrides
};
