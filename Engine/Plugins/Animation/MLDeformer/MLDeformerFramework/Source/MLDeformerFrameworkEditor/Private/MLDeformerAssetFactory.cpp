// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerAssetFactory.h"
#include "MLDeformerAsset.h"
#include "MLDeformerEditorModule.h"
#include "MLDeformerModel.h"
#include "AssetTypeCategories.h"
#include "Modules/ModuleManager.h"
#include "MLDeformerModelRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MLDeformerAssetFactory)

#define LOCTEXT_NAMESPACE "MLDeformerAssetFactory"

UMLDeformerFactory::UMLDeformerFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UMLDeformerAsset::StaticClass();
}

UObject* UMLDeformerFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	using namespace UE::MLDeformer;

	UMLDeformerAsset* DeformerAsset = NewObject<UMLDeformerAsset>(InParent, Name, Flags | RF_Transactional);
	if (DeformerAsset)
	{
		FMLDeformerEditorModule& EditorModule = FModuleManager::GetModuleChecked<FMLDeformerEditorModule>("MLDeformerFrameworkEditor");
		FMLDeformerEditorModelRegistry& ModelRegistry = EditorModule.GetModelRegistry();

		// If we have no models registered we can early out as we cannot create a model.
		TArray<UClass*> ModelTypes;
		ModelRegistry.GetRegisteredModels().GenerateKeyArray(ModelTypes);
		if (ModelTypes.IsEmpty())
		{
			return DeformerAsset;
		}

		// Create the highest priority ML model on default and use that in our asset.
		const int32 HighestPriorityIndex = ModelRegistry.GetHighestPriorityModelIndex();
		TObjectPtr<UMLDeformerModel> Model = NewObject<UMLDeformerModel>(DeformerAsset, ModelTypes[HighestPriorityIndex], NAME_None, RF_Transactional);
		check(Model);

		Model->Init(DeformerAsset);
		DeformerAsset->SetModel(Model);
	}

	return DeformerAsset;
}

bool UMLDeformerFactory::ShouldShowInNewMenu() const
{
	return true;
}

bool UMLDeformerFactory::ConfigureProperties()
{
	return true;
}

FText UMLDeformerFactory::GetDisplayName() const
{
	return LOCTEXT("MLDeformerAsset_DisplayName", "ML Deformer");
}

uint32 UMLDeformerFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Animation;
}

FText UMLDeformerFactory::GetToolTip() const
{
	return LOCTEXT("MLDeformerAsset_Tooltip", "Machine learning based mesh deformer asset.");
}

FString UMLDeformerFactory::GetDefaultNewAssetName() const
{
	return FString(TEXT("MLD_NewDeformer"));
}

const TArray<FText>& UMLDeformerFactory::GetMenuCategorySubMenus() const
{
	static TArray<FText> SubMenus { LOCTEXT("SubMenuDeformers", "Deformers") };
	return SubMenus;
}

#undef LOCTEXT_NAMESPACE
