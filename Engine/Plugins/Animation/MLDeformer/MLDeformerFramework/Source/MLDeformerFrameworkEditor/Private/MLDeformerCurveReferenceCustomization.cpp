// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerCurveReferenceCustomization.h"
#include "MLDeformerCurveReference.h"
#include "MLDeformerModel.h"
#include "SMLDeformerCurvePickerWidget.h"

namespace UE::MLDeformer
{
	TSharedRef<IPropertyTypeCustomization> FMLDeformerCurveReferenceCustomization::MakeInstance()
	{
		return MakeShareable(new FMLDeformerCurveReferenceCustomization());
	}

	void FMLDeformerCurveReferenceCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		SetPropertyHandle(StructPropertyHandle);
		SetSkeleton(StructPropertyHandle);

		if (CurveNameProperty->IsValidHandle())
		{
			HeaderRow
			.NameContent()
			[
				StructPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.MaxDesiredWidth(0.0f)
			[
				SNew(SCurveSelectionWidget)
				.OnCurveSelectionChanged(this, &FMLDeformerCurveReferenceCustomization::OnCurveSelectionChanged)
				.OnGetSelectedCurve(this, &FMLDeformerCurveReferenceCustomization::OnGetSelectedCurve)
				.OnGetSkeletalMesh(this, &FMLDeformerCurveReferenceCustomization::OnGetSkeletalMesh)
			];
		}
	}

	void FMLDeformerCurveReferenceCustomization::SetSkeleton(TSharedRef<IPropertyHandle> StructPropertyHandle) 
	{
		TArray<UObject*> Objects;
		StructPropertyHandle->GetOuterObjects(Objects);
		check(Objects.Num() == 1);

		UMLDeformerModel* DeformerModel = Cast<UMLDeformerModel>(Objects[0]);
		check(DeformerModel);

		bool bInvalidSkeletonIsError = false;
		SkeletalMesh = DeformerModel->GetSkeletalMesh();
	}

	TSharedPtr<IPropertyHandle> FMLDeformerCurveReferenceCustomization::FindStructMemberProperty(TSharedRef<IPropertyHandle> PropertyHandle, const FName& PropertyName)
	{
		uint32 NumChildren = 0;
		PropertyHandle->GetNumChildren(NumChildren);
		for (uint32 ChildIdx = 0; ChildIdx < NumChildren; ++ChildIdx)
		{
			TSharedPtr<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildIdx);
			if (ChildHandle->GetProperty()->GetFName() == PropertyName)
			{
				return ChildHandle;
			}
		}

		return TSharedPtr<IPropertyHandle>();
	}

	void FMLDeformerCurveReferenceCustomization::SetPropertyHandle(TSharedRef<IPropertyHandle> StructPropertyHandle)
	{
		CurveNameProperty = FindStructMemberProperty(StructPropertyHandle, GET_MEMBER_NAME_CHECKED(FMLDeformerCurveReference, CurveName));
		check(CurveNameProperty->IsValidHandle());
	}

	void FMLDeformerCurveReferenceCustomization::OnCurveSelectionChanged(const FString& Name)
	{
		CurveNameProperty->SetValue(Name);
	}

	FString FMLDeformerCurveReferenceCustomization::OnGetSelectedCurve() const
	{
		FString CurveName;
		CurveNameProperty->GetValue(CurveName);
		return CurveName;
	}

	USkeletalMesh* FMLDeformerCurveReferenceCustomization::OnGetSkeletalMesh() const
	{
		return SkeletalMesh;
	}
}	// namespace UE::MLDeformer
