// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerEditorActor.h"
#include "MLDeformerModule.h"
#include "MLDeformerEditorStyle.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Components/TextRenderComponent.h"

namespace UE::MLDeformer
{
	FMLDeformerEditorActor::FMLDeformerEditorActor(const FConstructSettings& Settings)
	{
		checkf(Settings.TypeID != -1, TEXT("Expected a valid actor type ID, please set this to a valid ID."));
		check(Settings.Actor != nullptr);

		TypeID = Settings.TypeID;
		Actor = Settings.Actor;
		LabelComponent = CreateLabelComponent(Settings.Actor, Settings.LabelColor, Settings.LabelText);
		bIsTrainingActor = Settings.bIsTrainingActor;
		ActorTypeInstanceIndex = Settings.ActorTypeInstanceIndex;
	}

	FMLDeformerEditorActor::~FMLDeformerEditorActor()
	{
		if (LabelComponent)
		{
			LabelComponent->UnregisterComponent();
		}
	}

	void FMLDeformerEditorActor::SetVisibility(bool bIsVisible)
	{
		if (SkeletalMeshComponent && bIsVisible != SkeletalMeshComponent->IsVisible())
		{
			SkeletalMeshComponent->SetVisibility(bIsVisible, true);
		}
	}

	bool FMLDeformerEditorActor::IsVisible() const
	{
		if (SkeletalMeshComponent)
		{
			return SkeletalMeshComponent->IsVisible();
		}

		return true;
	}

	bool FMLDeformerEditorActor::HasVisualMesh() const
	{
		if (SkeletalMeshComponent && SkeletalMeshComponent->GetSkeletalMeshAsset())
		{
			return true;
		}

		return false;
	}

	UTextRenderComponent* FMLDeformerEditorActor::CreateLabelComponent(AActor* InActor, FLinearColor Color, const FText& Text) const
	{
		const float DefaultLabelScale = FMLDeformerEditorStyle::Get().GetFloat("MLDeformer.DefaultLabelScale");
		UTextRenderComponent* TargetLabelComponent = NewObject<UTextRenderComponent>(InActor);
		TargetLabelComponent->SetMobility(EComponentMobility::Movable);
		TargetLabelComponent->SetHorizontalAlignment(EHTA_Center);
		TargetLabelComponent->SetVerticalAlignment(EVRTA_TextCenter);
		TargetLabelComponent->SetText(Text);
		TargetLabelComponent->SetRelativeScale3D(FVector(DefaultLabelScale));
		TargetLabelComponent->SetGenerateOverlapEvents(false);
		TargetLabelComponent->SetCanEverAffectNavigation(false);
		TargetLabelComponent->SetTextRenderColor(Color.ToFColor(true));
		TargetLabelComponent->RegisterComponent();
		return TargetLabelComponent;
	}

	void FMLDeformerEditorActor::SetPlayPosition(float TimeInSeconds, bool bAutoPause)
	{
		if (SkeletalMeshComponent)
		{
			SkeletalMeshComponent->SetPosition(TimeInSeconds);
			if (bAutoPause)
			{
				SkeletalMeshComponent->bPauseAnims = true;
			}
		}
	}

	float FMLDeformerEditorActor::GetPlayPosition() const
	{
		if (SkeletalMeshComponent)
		{
			return SkeletalMeshComponent->GetPosition();
		}

		return 0.0f;
	}

	void FMLDeformerEditorActor::SetPlaySpeed(float PlaySpeed)
	{
		if (SkeletalMeshComponent)
		{
			SkeletalMeshComponent->SetPlayRate(PlaySpeed);
		}
	}

	float FMLDeformerEditorActor::GetPlaySpeed()
	{
		if (SkeletalMeshComponent)
		{
			return SkeletalMeshComponent->GetPlayRate();
		}
		return 0.0f;
	}

	void FMLDeformerEditorActor::Pause(bool bPaused)
	{
		if (SkeletalMeshComponent)
		{
			SkeletalMeshComponent->bPauseAnims = bPaused;
			SkeletalMeshComponent->RefreshBoneTransforms();
		}
	}

	bool FMLDeformerEditorActor::IsPlaying() const
	{
		if (SkeletalMeshComponent)
		{
			return !SkeletalMeshComponent->bPauseAnims;
		}

		return false;
	}

	FBox FMLDeformerEditorActor::GetBoundingBox() const
	{
		if (SkeletalMeshComponent)
		{
			return SkeletalMeshComponent->Bounds.GetBox();
		}

		FBox Box;
		Box.Init();
		return Box;
	}

	bool FMLDeformerEditorActor::IsGroundTruthActor() const
	{
		return (TypeID == ActorID_Test_GroundTruth || TypeID == ActorID_Train_GroundTruth);
	}
}	// namespace UE::MLDeformer
