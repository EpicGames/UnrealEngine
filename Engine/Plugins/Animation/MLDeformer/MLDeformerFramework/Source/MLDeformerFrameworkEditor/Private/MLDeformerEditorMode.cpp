// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerEditorMode.h"
#include "MLDeformerEditorModule.h"
#include "MLDeformerEditorToolkit.h"
#include "MLDeformerEditorModel.h"
#include "MLDeformerModel.h"
#include "MLDeformerVizSettings.h"
#include "MLDeformerEditorActor.h"
#include "AssetEditorModeManager.h"

namespace UE::MLDeformer
{
	FName FMLDeformerEditorMode::ModeName("MLDeformerAssetEditMode");

	bool FMLDeformerEditorMode::GetCameraTarget(FSphere& OutTarget) const
	{
		FMLDeformerEditorModel* ActiveModel = DeformerEditorToolkit->GetActiveModel();
		if (ActiveModel == nullptr)
		{
			return false;
		}

		// Calculate the bounding box containing the actors we're interested in.
		FBox Box;
		Box.Init();
		UMLDeformerVizSettings* VizSettings = ActiveModel->GetModel()->GetVizSettings();
		for (FMLDeformerEditorActor* EditorActor : ActiveModel->GetEditorActors())
		{
			if (EditorActor)
			{
				bool bIncludeActor = (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TrainingData) && EditorActor->IsTrainingActor() && EditorActor->IsVisible();
				bIncludeActor |= (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData) && EditorActor->IsTestActor() && EditorActor->IsVisible();

				if (bIncludeActor)
				{
					Box += EditorActor->GetBoundingBox();
				}
			}
		}
	
		if (Box.IsValid == 1)
		{
			OutTarget = FSphere(Box.GetCenter(), Box.GetExtent().X * 0.75f);
			return true;
		}

		return false;
	}

	IPersonaPreviewScene& FMLDeformerEditorMode::GetAnimPreviewScene() const
	{
		return *static_cast<IPersonaPreviewScene*>(static_cast<FAssetEditorModeManager*>(Owner)->GetPreviewScene());
	}

	void FMLDeformerEditorMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
	{
		FEdMode::Render(View, Viewport, PDI);

		FMLDeformerEditorModel* EditorModel = DeformerEditorToolkit->GetActiveModel();
		if (EditorModel)
		{
			EditorModel->Render(View, Viewport, PDI);
		}
	}

	void FMLDeformerEditorMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
	{
		FEdMode::Tick(ViewportClient, DeltaTime);

		FMLDeformerEditorModel* EditorModel = DeformerEditorToolkit->GetActiveModel();
		if (!EditorModel)
		{
			return;
		}

		EditorModel->ClampCurrentTrainingFrameIndex();
		EditorModel->Tick(ViewportClient, DeltaTime);
	}

	void FMLDeformerEditorMode::DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
	{
		FEdMode::DrawHUD(ViewportClient, Viewport, View, Canvas);
	}
}	// namespace UE::MLDeformer
