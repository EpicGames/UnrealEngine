// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerEditorModel.h"
#include "IDetailsView.h"
#include "MLDeformerModule.h"
#include "MLDeformerModel.h"
#include "MLDeformerEditorStyle.h"
#include "MLDeformerEditorModule.h"
#include "MLDeformerModelRegistry.h"
#include "MLDeformerEditorToolkit.h"
#include "MLDeformerEditorActor.h"
#include "MLDeformerAsset.h"
#include "MLDeformerComponent.h"
#include "MLDeformerInputInfo.h"
#include "MLDeformerSampler.h"
#include "MLDeformerModelInstance.h"
#include "MLDeformerTrainingInputAnim.h"
#include "MLDeformerOctree.h"
#include "MLDeformerPaintMode.h"
#include "AnimationEditorPreviewActor.h"
#include "AnimationEditorViewportClient.h"
#include "EditorModeManager.h"
#include "Selection.h"
#include "AnimPreviewInstance.h"
#include "Animation/MeshDeformer.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Animation/AnimSequence.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "IPersonaPreviewScene.h"
#include "IPersonaToolkit.h"
#include "Modules/ModuleManager.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "EditorViewportClient.h"
#include "Components/TextRenderComponent.h"
#include "Materials/Material.h"
#include "GeometryCache.h"
#include "GeometryCacheMeshData.h"
#include "GeometryCacheTrack.h"
#include "BoneContainer.h"
#include "Misc/ScopedSlowTask.h"
#include "Animation/AnimData/IAnimationDataModel.h"
#include "Engine/SkinnedAssetCommon.h"
#include "SMLDeformerInputWidget.h"
#include "SMLDeformerTimeline.h"
#include "SMLDeformerDebugSelectionWidget.h"
#include "EditorViewportClient.h"
#include "DrawDebugHelpers.h"
#include "SkeletalMeshAttributes.h"
#include "Slate/SceneViewport.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Async/ParallelFor.h"
#include "Misc/TransactionObjectEvent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MLDeformerEditorModel)

#define LOCTEXT_NAMESPACE "MLDeformerEditorModel"

namespace UE::MLDeformer
{
	FMLDeformerEditorModel::~FMLDeformerEditorModel()
	{
		if (InputObjectModifiedHandle.IsValid())
		{
			FCoreUObjectDelegates::OnObjectModified.Remove(InputObjectModifiedHandle);
		}

		if (InputObjectPropertyChangedHandle.IsValid())
		{
			FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(InputObjectPropertyChangedHandle);
		}

		if (PostEditPropertyDelegateHandle.IsValid())
		{
			Model->OnPostEditChangeProperty().Remove(PostEditPropertyDelegateHandle);
		}

		if (PreEditUndoDelegateHandle.IsValid())
		{
			Model->OnPreEditUndo().Remove(PreEditUndoDelegateHandle);
		}

		if (PostEditUndoDelegateHandle.IsValid())
		{
			Model->OnPostEditUndo().Remove(PostEditUndoDelegateHandle);
		}

		if (PostTransactedDelegateHandle.IsValid())
		{
			Model->OnPostTransacted().Remove(PostTransactedDelegateHandle);
		}

		ClearWorld();

		// Delete the samplers.
		for (int32 Index = 0; Index < Samplers.Num(); ++Index)
		{
			Samplers[Index].Reset();
		}
		Samplers.Empty();

		Editor->GetPersonaToolkit()->GetPreviewScene()->DeselectAll();

		FMLDeformerEditorModule& EditorModule = FModuleManager::LoadModuleChecked<FMLDeformerEditorModule>("MLDeformerFrameworkEditor");
		EditorModule.GetModelRegistry().RemoveEditorModelInstance(this);
	}

	void FMLDeformerEditorModel::Init(const InitSettings& Settings)
	{
		check(Settings.Editor);
		check(Settings.Model);

		Editor = Settings.Editor;
		Model = Settings.Model;

		EditorInputInfo = Model->CreateInputInfo();
		check(EditorInputInfo);

		CreateSamplers();

		ViewRange = TRange<double>(0.0, 100.0);
		WorkingRange = TRange<double>(0.0, 100.0);
		PlaybackRange = TRange<double>(0.0, 100.0);

		// Start listening to property changes and transactions to the ML Deformer model.
		PostEditPropertyDelegateHandle = Model->OnPostEditChangeProperty().AddRaw(this, &FMLDeformerEditorModel::OnPostEditChangeProperty);
		PostTransactedDelegateHandle = Model->OnPostTransacted().AddRaw(this, &FMLDeformerEditorModel::OnPostTransacted);
		PreEditUndoDelegateHandle = Model->OnPreEditUndo().AddRaw(this, &FMLDeformerEditorModel::OnPreEditUndo);
		PostEditUndoDelegateHandle = Model->OnPostEditUndo().AddRaw(this, &FMLDeformerEditorModel::OnPostEditUndo);

		// Start listening for object modified and object property changed events.
		// We use this to trigger reinits of the UI and components, when one of our input assets got modified (by property changes, or reimports, etc).
		// This listens to changes in ALL objects in the engine.
		InputObjectModifiedHandle = FCoreUObjectDelegates::OnObjectModified.AddRaw(this, &FMLDeformerEditorModel::OnObjectModified);
		InputObjectPropertyChangedHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddLambda
		(
			[this](UObject* Object, const FPropertyChangedEvent& PropertyChangedEvent)
			{
				if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
				{
					OnObjectModified(Object);
				}
			}
		);

		UpdateLODMappings();
		UpdateTrainingDeviceList();
		UpdateEditorInputInfo();
	}

	void FMLDeformerEditorModel::UpdateVertexAttributeNames()
	{
		const TArray<FName> AttributeNames = Model->GetVertexAttributeNames();

		// Now build a list of shared pointers to attribute names.
		VertexAttributeNames.Reset();
		VertexAttributeNames.Reserve(AttributeNames.Num() + 1);

		const FName DisabledName = FName(LOCTEXT("DisabledString", "Disabled").ToString());
		VertexAttributeNames.Emplace(MakeShared<FName>(DisabledName));
		for (int32 Index = 0; Index < AttributeNames.Num(); ++Index)
		{
			VertexAttributeNames.Emplace(MakeShared<FName>(AttributeNames[Index]));
		}
	}

	void FMLDeformerEditorModel::CreateSamplers()
	{
		// First get rid of the existing samplers.
		for (int32 Index = 0; Index < Samplers.Num(); ++Index)
		{
			Samplers[Index].Reset();
		}
		Samplers.Reset();

		// Create a sampler for each training input anim entry.
		const int32 NumAnims = GetNumTrainingInputAnims();
		for (int32 AnimIndex = 0; AnimIndex < NumAnims; ++AnimIndex)
		{
			TSharedPtr<FMLDeformerSampler> Sampler = CreateSamplerObject();
			check(Sampler.IsValid());
			Sampler->Init(this, AnimIndex);
			Samplers.Add(Sampler);
		}
	}

	void FMLDeformerEditorModel::OnObjectModified(UObject* Object)
	{
		if (Model->GetSkeletalMesh() == Object || Model->GetInputInfo()->GetSkeletalMesh() == Object)
		{
			UE_LOGF(LogMLDeformer, Display, "Detected a modification in skeletal mesh %ls, reinitializing inputs.", *Object->GetName());
			bNeedsAssetReinit = true;
		}

		if (Model->GetVizSettings()->GetTestAnimSequence() == Object)
		{
			UE_LOGF(LogMLDeformer, Display, "Detected a modification in test anim sequence %ls, reinitializing inputs.", *Object->GetName());
			bNeedsAssetReinit = true;
		}
	}

	void FMLDeformerEditorModel::AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObject(EditorInputInfo);
	}

	void FMLDeformerEditorModel::CopyBaseSettingsFromModel(const FMLDeformerEditorModel* SourceEditorModel)
	{
		UMLDeformerModel* SourceModel = SourceEditorModel->GetModel();
		if (SourceModel == nullptr)
		{
			return;
		}

		UMLDeformerModel* TargetModel = GetModel();
		check(TargetModel);

		// Copy model properties.
		TargetModel->SetSkeletalMesh(SourceModel->GetSkeletalMesh());
		TargetModel->SetAlignmentTransform(SourceModel->GetAlignmentTransform());
		TargetModel->SetTrainingFrameLimit(SourceModel->GetTrainingFrameLimit());
		if (TargetModel->DoesSupportBones())
		{
			TargetModel->SetBoneIncludeList(SourceModel->GetBoneIncludeList());
		}
		if (TargetModel->DoesSupportCurves())
		{
			TargetModel->SetCurveIncludeList(SourceModel->GetCurveIncludeList());
		}

		// Copy visualization properties.
		const UMLDeformerVizSettings* SourceVizSettings = SourceModel->GetVizSettings();
		UMLDeformerVizSettings* TargetVizSettings = TargetModel->GetVizSettings();
		check(TargetVizSettings);
		if (SourceVizSettings)
		{
			TargetVizSettings->SetLabelHeight(SourceVizSettings->GetLabelHeight());
			TargetVizSettings->SetVisualizationMode(SourceVizSettings->GetVisualizationMode());
			TargetVizSettings->SetMeshSpacing(SourceVizSettings->GetMeshSpacing());
			TargetVizSettings->SetLabelScale(SourceVizSettings->GetLabelScale());
			TargetVizSettings->SetDrawLabels(SourceVizSettings->GetDrawLabels());
			TargetVizSettings->SetWeight(SourceVizSettings->GetWeight());
			TargetVizSettings->SetTestAnimSequence((UAnimSequence*)SourceVizSettings->GetTestAnimSequence());
			TargetVizSettings->SetTrainingFrameNumber(SourceVizSettings->GetTrainingFrameNumber());
			TargetVizSettings->SetTestingFrameNumber(SourceVizSettings->GetTestingFrameNumber());
			TargetVizSettings->SetAnimPlaySpeed(SourceVizSettings->GetAnimPlaySpeed());
			TargetVizSettings->SetDrawLinearSkinnedActor(SourceVizSettings->GetDrawLinearSkinnedActor());
			TargetVizSettings->SetDrawMLDeformedActor(SourceVizSettings->GetDrawMLDeformedActor());
			TargetVizSettings->SetDrawGroundTruthActor(SourceVizSettings->GetDrawGroundTruthActor());
			TargetVizSettings->SetDrawMLCompareActors(SourceVizSettings->GetDrawMLCompareActors());
			TargetVizSettings->SetDrawDebugActorBounds(SourceVizSettings->GetDrawDebugActorBounds());
			TargetVizSettings->SetDrawVertexDeltas(SourceVizSettings->GetDrawVertexDeltas());
			TargetVizSettings->SetXRayDeltas(SourceVizSettings->GetXRayDeltas());
			TargetVizSettings->SetHeatMapMax(SourceVizSettings->GetHeatMapMax());
			TargetVizSettings->SetGroundTruthLerp(SourceVizSettings->GetGroundTruthLerp());
			TargetVizSettings->SetShowHeatMap(SourceVizSettings->GetShowHeatMap());
			TargetVizSettings->SetHeatMapMode(SourceVizSettings->GetHeatMapMode());
			TargetVizSettings->SetDebugBoundsColor(SourceVizSettings->GetDebugBoundsColor());
			TargetVizSettings->SetCompareActors(SourceVizSettings->GetCompareActors());

			// NOTE: We don't copy the DeformerGraph on purpose, as this model might need another one, so we let it get the default graph.
		}

		ActiveTrainingInputAnimIndex = SourceEditorModel->GetActiveTrainingInputAnimIndex();

		TargetModel->UpdateMemoryUsage();
		UpdateMemoryUsage();	
	}

	void FMLDeformerEditorModel::UpdateEditorInputInfo()
	{
		InitInputInfo(EditorInputInfo);
	}

	UWorld* FMLDeformerEditorModel::GetWorld() const
	{
		check(Editor);
		return Editor->GetPersonaToolkit()->GetPreviewScene()->GetWorld();
	}

	void FMLDeformerEditorModel::CreateTrainingLinearSkinnedActor(const TSharedRef<IPersonaPreviewScene>& InPersonaPreviewScene)
	{
		UWorld* World = InPersonaPreviewScene->GetWorld();

		// Spawn the linear skinned actor.
		FActorSpawnParameters BaseSpawnParams;
		BaseSpawnParams.Name = MakeUniqueObjectName(World, AActor::StaticClass(), "Train Base Actor");
		AActor* Actor = World->SpawnActor<AActor>(AActor::StaticClass(), BaseSpawnParams);
		Actor->SetFlags(RF_Transient);

		// Create the preview skeletal mesh component.
		const FLinearColor BaseWireColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.BaseMesh.WireframeColor");
		UDebugSkelMeshComponent* SkelMeshComponent = NewObject<UDebugSkelMeshComponent>(Actor);
		SkelMeshComponent->SetWireframeMeshOverlayColor(BaseWireColor);
		SkelMeshComponent->SetVisibility(false);
		SkelMeshComponent->MarkRenderStateDirty();

		// Set the skeletal mesh on the component.
		// NOTE: This must be done AFTER setting the AnimInstance so that the correct root anim node is loaded.
		USkeletalMesh* Mesh = Model->GetSkeletalMesh();
		SkelMeshComponent->SetSkeletalMesh(Mesh);
		SkelMeshComponent->RegisterComponent();

		// Update the persona scene.
		InPersonaPreviewScene->SetActor(Actor);
		InPersonaPreviewScene->SetPreviewMeshComponent(SkelMeshComponent);
		InPersonaPreviewScene->SetAdditionalMeshesSelectable(false);
		InPersonaPreviewScene->SetPreviewMesh(Mesh);
		if (SkelMeshComponent->GetAnimInstance())
		{
			SkelMeshComponent->GetAnimInstance()->GetRequiredBones().SetUseRAWData(true);
		}

		// Register the editor actor.
		const FLinearColor LabelColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.BaseMesh.LabelColor");
		FMLDeformerEditorActor::FConstructSettings Settings;
		Settings.Actor = Actor;
		Settings.TypeID = ActorID_Train_Base;
		Settings.LabelColor = LabelColor;
		Settings.LabelText = LOCTEXT("TrainBaseActorLabelText", "Training Base");
		Settings.bIsTrainingActor = true;
		FMLDeformerEditorActor* EditorActor = CreateEditorActor(Settings);
		EditorActor->SetSkeletalMeshComponent(SkelMeshComponent);
		EditorActor->SetMeshOffsetFactor(0.0f);
		EditorActors.Add(EditorActor);
	}

	void FMLDeformerEditorModel::CreateTestLinearSkinnedActor(UWorld* World)
	{
		FActorSpawnParameters SpawnParams;
		SpawnParams.Name = MakeUniqueObjectName(World, AActor::StaticClass(), "Test Linear Skinned Actor");
		AActor* Actor = World->SpawnActor<AActor>(AActor::StaticClass(), FTransform::Identity, SpawnParams);
		Actor->SetFlags(RF_Transient);

		const FLinearColor BaseWireColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.BaseMesh.WireframeColor");
		UDebugSkelMeshComponent* SkelMeshComponent = NewObject<UDebugSkelMeshComponent>(Actor);
		SkelMeshComponent->SetWireframeMeshOverlayColor(BaseWireColor);
		SkelMeshComponent->SetSkeletalMesh(Model->GetSkeletalMesh());
		SkelMeshComponent->RegisterComponent();
		SkelMeshComponent->SetVisibility(false);
		SkelMeshComponent->MarkRenderStateDirty();
		Actor->SetRootComponent(SkelMeshComponent);
		if (SkelMeshComponent->GetAnimInstance())
		{
			SkelMeshComponent->GetAnimInstance()->GetRequiredBones().SetUseRAWData(true);
		}

		// Register the editor actor.
		const FLinearColor LabelColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.BaseMesh.LabelColor");
		FMLDeformerEditorActor::FConstructSettings Settings;
		Settings.Actor = Actor;
		Settings.TypeID = ActorID_Test_Base;
		Settings.LabelColor = LabelColor;
		Settings.LabelText = LOCTEXT("TestBaseActorLabelText", "Linear Skinned");
		Settings.bIsTrainingActor = false;
		FMLDeformerEditorActor* EditorActor = CreateEditorActor(Settings);
		EditorActor->SetSkeletalMeshComponent(SkelMeshComponent);
		EditorActor->SetMeshOffsetFactor(0.0f);
		EditorActors.Add(EditorActor);
	}

	void FMLDeformerEditorModel::CreateTestMLDeformedActor(UWorld* World)
	{
		// Create the ML deformed actor.
		FActorSpawnParameters SpawnParams;
		SpawnParams.Name = MakeUniqueObjectName(World, AActor::StaticClass(), "Test ML Deformed");
		AActor* Actor = World->SpawnActor<AActor>(SpawnParams);
		Actor->SetFlags(RF_Transient);

		// Create the skeletal mesh component.
		const FLinearColor MLDeformedWireColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.MLDeformedMesh.WireframeColor");
		UDebugSkelMeshComponent* SkelMeshComponent = NewObject<UDebugSkelMeshComponent>(Actor);
		SkelMeshComponent->SetSkeletalMesh(Model->GetSkeletalMesh());
		Actor->SetRootComponent(SkelMeshComponent);
		SkelMeshComponent->RegisterComponent();
		SkelMeshComponent->SetWireframeMeshOverlayColor(MLDeformedWireColor);
		SkelMeshComponent->SetVisibility(false);
		SkelMeshComponent->MarkRenderStateDirty();
		if (SkelMeshComponent->GetAnimInstance())
		{
			SkelMeshComponent->GetAnimInstance()->GetRequiredBones().SetUseRAWData(true);
		}

		// Create the ML Deformer component.
		UMLDeformerAsset* DeformerAsset = Model->GetDeformerAsset();
		UMLDeformerComponent* MLDeformerComponent = NewObject<UMLDeformerComponent>(Actor);
		MLDeformerComponent->SetSuppressMeshDeformerLogWarnings(true);
		MLDeformerComponent->SetDeformerAsset(DeformerAsset);
		MLDeformerComponent->RegisterComponent();
		MLDeformerComponent->SetupComponent(DeformerAsset, SkelMeshComponent);

		// Create the editor actor.
		const FLinearColor LabelColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.MLDeformedMesh.LabelColor");
		FMLDeformerEditorActor::FConstructSettings Settings;
		Settings.Actor = Actor;
		Settings.TypeID = ActorID_Test_MLDeformed;
		Settings.LabelColor = LabelColor;
		Settings.LabelText = LOCTEXT("TestMLDeformedActorLabelText", "ML Deformed");
		Settings.bIsTrainingActor = false;
		FMLDeformerEditorActor* EditorActor = CreateEditorActor(Settings);
		EditorActor->SetSkeletalMeshComponent(SkelMeshComponent);
		EditorActor->SetMLDeformerComponent(MLDeformerComponent);
		EditorActor->SetMeshOffsetFactor(1.0f);
		EditorActors.Add(EditorActor);
	}

	void FMLDeformerEditorModel::CreateTestCompareActors(UWorld* World)
	{
		const FLinearColor MLDeformedWireColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.MLCompareMesh.WireframeColor");
		USkeletalMesh* SkelMesh = Model->GetSkeletalMesh();

		const TArray<FMLDeformerCompareActor>& CompareActors = Model->GetVizSettings()->GetCompareActors();
		for (int32 ActorIndex = 0; ActorIndex < CompareActors.Num(); ++ActorIndex)
		{
			const FMLDeformerCompareActor& CompareActor = CompareActors[ActorIndex];

			// Create the ML deformed actor.
			FActorSpawnParameters SpawnParams;
			SpawnParams.Name = MakeUniqueObjectName(World, AActor::StaticClass(), "Test Compare Actor");
			AActor* Actor = World->SpawnActor<AActor>(SpawnParams);
			Actor->SetFlags(RF_Transient);

			// Create the skeletal mesh component.
			UDebugSkelMeshComponent* SkelMeshComponent = NewObject<UDebugSkelMeshComponent>(Actor);
			UMLDeformerAsset* DeformerAsset = CompareActor.DeformerAsset.LoadSynchronous();
			if (!IsCompatibleDeformer(DeformerAsset))
			{
				SkelMesh = nullptr;
			}
			SkelMeshComponent->SetSkeletalMesh(SkelMesh);
			Actor->SetRootComponent(SkelMeshComponent);
			SkelMeshComponent->RegisterComponent();
			SkelMeshComponent->SetWireframeMeshOverlayColor(MLDeformedWireColor);
			SkelMeshComponent->SetVisibility(false);
			SkelMeshComponent->MarkRenderStateDirty();

			// Create the ML Deformer component.
			UMLDeformerAsset* FinalDeformerAsset = IsCompatibleDeformer(DeformerAsset) ? DeformerAsset : nullptr;
			UMLDeformerComponent* MLDeformerComponent = NewObject<UMLDeformerComponent>(Actor);
			MLDeformerComponent->SetSuppressMeshDeformerLogWarnings(true);
			MLDeformerComponent->SetDeformerAsset(FinalDeformerAsset);
			MLDeformerComponent->RegisterComponent();
			MLDeformerComponent->SetupComponent(FinalDeformerAsset, SkelMeshComponent);

			// Create the editor actor.
			const FLinearColor LabelColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.MLCompareMesh.LabelColor");
			FMLDeformerEditorActor::FConstructSettings Settings;
			Settings.Actor = Actor;
			Settings.TypeID = ActorID_Test_Compare;
			Settings.LabelColor = LabelColor;
			Settings.LabelText = LOCTEXT("TestCompareActorLabelText", "ML Compare");
			Settings.bIsTrainingActor = false;
			Settings.ActorTypeInstanceIndex = ActorIndex;
			FMLDeformerEditorActor* EditorActor = CreateEditorActor(Settings);
			EditorActor->SetSkeletalMeshComponent(SkelMeshComponent);
			EditorActor->SetMLDeformerComponent(MLDeformerComponent);
			EditorActor->SetMeshOffsetFactor(2.0f + static_cast<float>(ActorIndex));
			EditorActors.Add(EditorActor);
		}

		UpdateCompareActorLabels();
	}

	void FMLDeformerEditorModel::CreateActors(const TSharedRef<IPersonaPreviewScene>& InPersonaPreviewScene)
	{
		UWorld* World = InPersonaPreviewScene->GetWorld();
		CreateTrainingLinearSkinnedActor(InPersonaPreviewScene);
		CreateTestLinearSkinnedActor(World);
		CreateTestMLDeformedActor(World);
		CreateTrainingGroundTruthActor(World);
		CreateTestGroundTruthActor(World);
		CreateTestCompareActors(World);

		UpdateMeshOffsetFactors();
	}

	int32 FMLDeformerEditorModel::CalcNumValidCompareActorsPriorTo(int32 CompareActorIndex) const
	{
		int Result = 0;

		const TArray<FMLDeformerCompareActor>& CompareActors = Model->GetVizSettings()->GetCompareActors();
		for (int32 CompareInstanceIndex = 0; CompareInstanceIndex < CompareActors.Num(); ++CompareInstanceIndex)
		{
			if (CompareInstanceIndex < CompareActorIndex)
			{
				const FMLDeformerCompareActor& CompareActor = CompareActors[CompareInstanceIndex];
				UMLDeformerAsset* CompareDeformerAsset = CompareActor.DeformerAsset.LoadSynchronous();
				if (IsCompatibleDeformer(CompareDeformerAsset))
				{
					Result++;
				}
			}
		}

		return Result;
	}

	bool FMLDeformerEditorModel::IsAnimIndexValid(int32 AnimIndex) const
	{
		return FMath::IsWithinInclusive<int32>(AnimIndex, 0, GetNumTrainingInputAnims());
	}

	void FMLDeformerEditorModel::UpdateLODMappings()
	{
		LODMappings.Reset();

		const USkeletalMesh* SkelMesh = Model->GetSkeletalMesh();
		if (!SkelMesh || !SkelMesh->GetImportedModel())
		{
			return;
		}

		// Check some expectations on the number of LODS.
		const FSkeletalMeshModel* ImportedModel = SkelMesh->GetImportedModel();
		int32 NumLODLevels = SkelMesh->GetLODNum();
		if (ImportedModel->LODModels.Num() != NumLODLevels)
		{
			return;
		}
		NumLODLevels = FMath::Min(NumLODLevels, Model->GetMaxNumLODs());

		const double StartTime = FPlatformTime::Seconds();

		TArray<FSoftSkinVertex> VertexPositionsLODZero;
		ImportedModel->LODModels[0].GetVertices(VertexPositionsLODZero);

		// Build an octree that contains all our vertices.
		// We can use this to quickly get the closest vertex index for a given position.
		// This isn't perfectly accurate in some edge cases, but good enough.
		FMLDeformerOctree Octree;
		Octree.Build(VertexPositionsLODZero, /*MaxVertsPerOctreeNode*/300, /*MaxSplitHierarchyDepth*/6);

		// For each additional LOD level we are interested in.
		// We skip LOD 0 because we don't need a mapping for that one, as it's our source LOD that we map into.
		LODMappings.SetNum(NumLODLevels);
		for (int32 LodLevel = 1; LodLevel < NumLODLevels; ++LodLevel)
		{
			FLODInfo& LodInfo = LODMappings[LodLevel];

			// Get the number of vertices.
			TArray<FSoftSkinVertex> VertexPositions;
			ImportedModel->LODModels[LodLevel].GetVertices(VertexPositions);
			const int32 NumVerts = ImportedModel->LODModels[LodLevel].NumVertices;
			LodInfo.VtxMappingToLODZero.SetNum(NumVerts);

			// Process all vertices in batches, and run all batches in parallel.
			const int32 BatchSize = 100; // Number of verts per batch.
			const int32 NumBatches = (NumVerts / BatchSize) + 1;
			ParallelFor(NumBatches, [&LodInfo, &Octree, &VertexPositions, NumVerts](int32 BatchIndex)
			{
				// Make sure we don't go past the last vertex.
				const int32 StartVertex = BatchIndex * BatchSize;
				if (StartVertex >= NumVerts)
				{
					return;
				}

				// Process all vertices inside this batch.
				const int32 NumVertsInBatch = (StartVertex + BatchSize) < NumVerts ? BatchSize : FMath::Max(NumVerts - StartVertex, 0);
				for (int32 VertexIndex = StartVertex; VertexIndex < StartVertex + NumVertsInBatch; ++VertexIndex)
				{
					// Find the best vertex index in LOD 0.
					LodInfo.VtxMappingToLODZero[VertexIndex] = Octree.FindClosestMeshVertexIndex((FVector)VertexPositions[VertexIndex].Position, 0.1);
				}
			});
		} // For all LOD levels

		const double TotalTime = FPlatformTime::Seconds() - StartTime;
		UE_LOGF(LogMLDeformer, Display, "Finished Updating ML Deformer LOD mappings in %.2f seconds", TotalTime);
	}

	void FMLDeformerEditorModel::UpdateMeshOffsetFactors()
	{
		const bool bIsDebugging = (GetEditor()->GetDebugActor() != nullptr);

		// Set the default mesh translation offsets for our ground truth actors.
		for (FMLDeformerEditorActor* EditorActor : EditorActors)
		{
			if (EditorActor == nullptr)
			{
				continue;
			}

			if (EditorActor->IsGroundTruthActor())
			{			
				// The mesh offset factor basically just offsets the actor position by a given factor.
				// The amount the actor is moved from the origin is: (MeshSpacing * MeshOffsetFactor).
				// In test mode we have 3 actors (Linear, ML Deformed, Ground Truth), so it's mesh offset factor will be 2.0 for the ground truth.
				// It is 2.0 because the ground truth actor in testing mode is all the way on the right, next to the ML Deformed model.
				// In training mode we have only the Linear Skinned actor and the ground truth, so there the spacing factor is 1.0.
				// TLDR: Basically 1.0 means its the first actor next to the linear skinned actor while 2.0 means its the second character, etc.
				float MeshOffsetFactor = 1.0f;
				if (EditorActor->IsTestActor())
				{
					MeshOffsetFactor = 2.0f;

					if (Model->GetVizSettings()->GetDrawMLCompareActors() && !bIsDebugging)
					{
						const TArray<FMLDeformerCompareActor>& CompareActors = Model->GetVizSettings()->GetCompareActors();
						const int32 NumValidCompareActors = static_cast<float>(CalcNumValidCompareActorsPriorTo(CompareActors.Num()));
						MeshOffsetFactor += static_cast<float>(NumValidCompareActors);
					}
				}

				EditorActor->SetMeshOffsetFactor(MeshOffsetFactor);
			}
			else if (EditorActor->GetTypeID() == ActorID_Test_Compare)
			{
				const int32 NumValidCompareActorsBefore = static_cast<float>(CalcNumValidCompareActorsPriorTo(EditorActor->GetActorTypeInstanceIndex()));
				EditorActor->SetMeshOffsetFactor(2.0f + static_cast<float>(NumValidCompareActorsBefore));
			}
		}
	}

	void FMLDeformerEditorModel::ClearWorld()
	{
		// First remove actors from the world and add them to the destroy list.
		TArray<AActor*> ActorsToDestroy;
		ActorsToDestroy.Reserve(EditorActors.Num());
		TSharedRef<IPersonaPreviewScene> PreviewScene = Editor->GetPersonaToolkit()->GetPreviewScene();
		UWorld* World = PreviewScene->GetWorld();
		for (FMLDeformerEditorActor* EditorActor : EditorActors)
		{
			if (EditorActor)
			{				
				ActorsToDestroy.Add(EditorActor->GetActor());
			}
		}

		// Now delete the editor actors, which removes components.
		DeleteEditorActors();

		for (AActor* Actor : ActorsToDestroy)
		{
			World->EditorDestroyActor(Actor, true);
			Actor->Destroy();
		}
	}

	bool FMLDeformerEditorModel::IsCompatibleDeformer(UMLDeformerAsset* Deformer) const
	{
		if (Deformer == nullptr)
		{
			return false;
		}

		UMLDeformerModel* CompareModel = Deformer->GetModel();
		if (CompareModel->GetInputInfo()->GetSkeletalMesh() != Model->GetSkeletalMesh())
		{
			return false;
		}

		return true;
	}

	void FMLDeformerEditorModel::ClearPersonaPreviewScene()
	{
		TSharedRef<IPersonaPreviewScene> PreviewScene = Editor->GetPersonaToolkit()->GetPreviewScene();
		UWorld* World = PreviewScene->GetWorld();

		PreviewScene->DeselectAll();

		if (GetEditor()->GetEditorModeManager().GetSelectedComponents())
		{
			GetEditor()->GetEditorModeManager().GetSelectedComponents()->DeselectAll();
		}

		for (FMLDeformerEditorActor* EditorActor : EditorActors)
		{
			USkeletalMeshComponent* SkelMeshComponent = EditorActor->GetSkeletalMeshComponent();
			if (SkelMeshComponent)
			{
				PreviewScene->RemoveComponent(SkelMeshComponent);
			}
		}

		PreviewScene->SetPreviewAnimationAsset(nullptr);
		PreviewScene->SetPreviewMesh(nullptr);
		PreviewScene->SetPreviewMeshComponent(nullptr);
		PreviewScene->SetActor(nullptr);
	}

	void FMLDeformerEditorModel::ClearWorldAndPersonaPreviewScene()
	{
		ClearPersonaPreviewScene();
		ClearWorld();
	}

	FMLDeformerEditorActor* FMLDeformerEditorModel::CreateEditorActor(const FMLDeformerEditorActor::FConstructSettings& Settings) const
	{ 
		return new FMLDeformerEditorActor(Settings);
	}

	void FMLDeformerEditorModel::DeleteEditorActors()
	{
		for (FMLDeformerEditorActor* EditorActor : EditorActors)
		{
			delete EditorActor;
		}
		EditorActors.Empty();
	}

	void FMLDeformerEditorModel::SetTimelineAnimNames(const TArray<TSharedPtr<FMLDeformerTrainingInputAnimName>>& AnimNames)
	{
		SMLDeformerTimeline* Timeline = GetEditor()->GetTimeSlider();
		if (Timeline)
		{
			Timeline->SetTrainingAnimNames(AnimNames);
		}
	}

	FMLDeformerEditorActor* FMLDeformerEditorModel::FindEditorActor(int32 ActorTypeID) const
	{
		for (FMLDeformerEditorActor* EditorActor : EditorActors)
		{
			if (EditorActor && EditorActor->GetTypeID() == ActorTypeID)
			{
				return EditorActor;
			}
		}

		return nullptr;
	}

	void FMLDeformerEditorModel::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
	{
		if (bNeedsAssetReinit)
		{
			TriggerInputAssetChanged();
			UpdateLODMappings();
			Model->InitVertexMap();
			Model->InitGPUData();
			bNeedsAssetReinit = false;
		}

		// Make sure we use raw data for everything.
		for (FMLDeformerEditorActor* Actor : EditorActors)
		{
			if (Actor && Actor->GetSkeletalMeshComponent())
			{
				USkeletalMeshComponent* SkelMeshComp = Actor->GetSkeletalMeshComponent();			
				if (SkelMeshComp->GetAnimInstance())
				{
					SkelMeshComp->GetAnimInstance()->GetRequiredBones().SetUseRAWData(true);
				}
			}
		}

		UpdateMeshOffsetFactors();
		UpdateActorTransforms();
		UpdateLabels();
		UpdateActorLODs();
		ApplyDebugActorTransforms();

		// Debug draw elements inside the PIE viewport when PIE is active.
		DrawPIEDebugActors();

		// We need to tell the viewport that it needs to redraw, because otherwise while PIE is active
		// it won't redraw the asset editor unless you interact with the viewport using the mouse.
		if (GEditor->GetPIEViewport() && GetEditor()->GetViewport().IsValid())
		{
			if (Model->GetVizSettings()->GetDrawDebugActorBounds() || IsValid(GetEditor()->GetDebugActor()))
			{
				GetEditor()->GetViewport()->GetViewportClient().bNeedsRedraw = true;
			}
		}

		// Update the ML Deformer components.
		for (FMLDeformerEditorActor* EditorActor : EditorActors)
		{
			if (EditorActor && EditorActor->GetMLDeformerComponent())
			{		
				const UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
				UMLDeformerComponent* DeformerComponent = EditorActor->GetMLDeformerComponent();
				DeformerComponent->SetWeight(VizSettings->GetWeight());
			}
		}
	}

	void FMLDeformerEditorModel::UpdateLabels()
	{
		const UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
		const bool bDrawTrainingActors = (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TrainingData);
		const bool bDrawTestActors = (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData);

		for (FMLDeformerEditorActor* EditorActor : EditorActors)
		{
			if (EditorActor == nullptr)
			{
				continue;
			}

			UTextRenderComponent* LabelComponent = EditorActor->GetLabelComponent();
			if (LabelComponent == nullptr)
			{
				continue;
			}

			if (VizSettings->GetDrawLabels())
			{
				const AActor* Actor = EditorActor->GetActor();
				const FVector ActorLocation = Actor->GetActorLocation();
				const FVector AlignmentOffset = EditorActor->IsGroundTruthActor() ? Model->GetAlignmentTransform().GetTranslation() : FVector::ZeroVector;

				LabelComponent->SetRelativeLocation(ActorLocation + FVector(0.0f, 0.0f, VizSettings->GetLabelHeight()) - AlignmentOffset);
				LabelComponent->SetRelativeRotation(FQuat(FVector(0.0f, 0.0f, 1.0f), FMath::DegreesToRadians(90.0f)));
				LabelComponent->SetRelativeScale3D(FVector(VizSettings->GetLabelScale() * 0.3f));

				// Update visibility.
				const bool bLabelIsVisible = ((bDrawTrainingActors && EditorActor->IsTrainingActor()) || (bDrawTestActors && EditorActor->IsTestActor())) && EditorActor->IsVisible();
				LabelComponent->SetVisibility(bLabelIsVisible);

				// Handle test ground truth, disable its label when no ground truth asset was selected.
				if (EditorActor->GetTypeID() == ActorID_Test_GroundTruth && !VizSettings->HasTestGroundTruth())
				{
					LabelComponent->SetVisibility(false);
				}
			}
			else
			{
				LabelComponent->SetVisibility(false);
			}
		}
	}

	void FMLDeformerEditorModel::UpdateActorTransforms()
	{
		const FVector MeshSpacingVector = Model->GetVizSettings()->GetMeshSpacingOffsetVector();
		for (FMLDeformerEditorActor* EditorActor : EditorActors)
		{
			if (EditorActor)
			{
				FTransform Transform = EditorActor->IsGroundTruthActor() ? Model->GetAlignmentTransform() : FTransform::Identity;
				Transform.AddToTranslation(MeshSpacingVector * EditorActor->GetMeshOffsetFactor());
				EditorActor->GetActor()->SetActorTransform(Transform);
			}
		}
	}

	void FMLDeformerEditorModel::UpdateActorVisibility()
	{
		const UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
		const bool bShowTrainingData = (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TrainingData);
		const bool bShowTestData = (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData);
		const bool bIsDebugging = (GetEditor()->GetDebugActor() != nullptr);
		const bool bInDefaultMode = GetEditor()->IsDefaultModeActive();

		for (FMLDeformerEditorActor* EditorActor : EditorActors)
		{
			if (EditorActor)
			{
				bool bIsVisible = (EditorActor->IsTestActor() && bShowTestData) || (EditorActor->IsTrainingActor() && bShowTrainingData);
				if (EditorActor->GetTypeID() == ActorID_Test_Base)
				{
					bIsVisible &= VizSettings->GetDrawLinearSkinnedActor();
				}
				else if (EditorActor->GetTypeID() == ActorID_Test_MLDeformed)
				{
					bIsVisible &= VizSettings->GetDrawMLDeformedActor();
				}
				else if (EditorActor->GetTypeID() == ActorID_Test_GroundTruth)
				{
					bIsVisible &= VizSettings->GetDrawGroundTruthActor() && !bIsDebugging;
				}
				else if (EditorActor->GetTypeID() == ActorID_Test_Compare)
				{
					bIsVisible &= VizSettings->GetDrawMLCompareActors() && !bIsDebugging;
				}

				bIsVisible &= EditorActor->HasVisualMesh();
				bIsVisible &= bInDefaultMode;

				EditorActor->SetVisibility(bIsVisible);
			}
		}
	}

	void FMLDeformerEditorModel::ChangeSkeletalMeshOnComponent(USkeletalMeshComponent* SkelMeshComponent, USkeletalMesh* Mesh)
	{
		if (Mesh != SkelMeshComponent->GetSkeletalMeshAsset())
		{
			// Inform the skeletal mesh component about changes to the mesh.
			SkelMeshComponent->SetSkeletalMesh(Mesh);
			FPropertyChangedEvent SkelMeshChangedEvent(USkeletalMeshComponent::StaticClass()->FindPropertyByName(TEXT("SkeletalMeshAsset")));
			SkelMeshComponent->PostEditChangeProperty(SkelMeshChangedEvent);

			if (Mesh)
			{
				// Force all materials in the Skeletal Mesh Component to be the ones coming from the Mesh to avoid issues with material slots not updating.
				const TArray<FSkeletalMaterial>& Materials = Mesh->GetMaterials();
				for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); ++MaterialIndex)
				{
					SkelMeshComponent->SetMaterial(MaterialIndex, Materials[MaterialIndex].MaterialInterface);
				}
				SkelMeshComponent->MarkRenderStateDirty();
			}
		}
	}

	void FMLDeformerEditorModel::OnInputAssetsChanged()
	{
		// Update the training base actor.
		UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
		UAnimSequence* TestAnimSequence = VizSettings->GetTestAnimSequence();
		const float TestAnimSpeed = VizSettings->GetAnimPlaySpeed();

		const FMLDeformerEditorActor* TrainBaseActor = FindEditorActor(ActorID_Train_Base);
		UDebugSkelMeshComponent* SkeletalMeshComponent = TrainBaseActor ? TrainBaseActor->GetSkeletalMeshComponent() : nullptr;
		if (SkeletalMeshComponent)
		{
			ChangeSkeletalMeshOnComponent(SkeletalMeshComponent, Model->GetSkeletalMesh());
			if (GetEditor()->GetPersonaToolkitPointer())
			{
				GetEditor()->GetPersonaToolkit()->GetPreviewScene()->SetPreviewMesh(Model->GetSkeletalMesh());
			}
			UAnimSequence* TrainingAnimSequence = GetActiveTrainingInputAnimSequence();
			SkeletalMeshComponent->SetAnimationMode(EAnimationMode::AnimationSingleNode);
			const float CurrentPlayTime = SkeletalMeshComponent->GetPosition();
			SkeletalMeshComponent->SetAnimation(TrainingAnimSequence);
			SkeletalMeshComponent->SetPosition(CurrentPlayTime);
			SkeletalMeshComponent->SetPlayRate(TestAnimSpeed);
			SkeletalMeshComponent->Play(true);
			if (SkeletalMeshComponent->GetAnimInstance())
			{
				SkeletalMeshComponent->GetAnimInstance()->GetRequiredBones().SetUseRAWData(true);
			}
		}

		// Update the test base model.
		const FMLDeformerEditorActor* TestBaseActor = FindEditorActor(ActorID_Test_Base);
		SkeletalMeshComponent = TestBaseActor ? TestBaseActor->GetSkeletalMeshComponent() : nullptr;
		if (SkeletalMeshComponent)
		{
			ChangeSkeletalMeshOnComponent(SkeletalMeshComponent, Model->GetSkeletalMesh());
			SkeletalMeshComponent->SetAnimationMode(EAnimationMode::AnimationSingleNode);
			const float CurrentPlayTime = SkeletalMeshComponent->GetPosition();
			SkeletalMeshComponent->SetAnimation(TestAnimSequence);
			SkeletalMeshComponent->SetPosition(CurrentPlayTime);
			SkeletalMeshComponent->SetPlayRate(TestAnimSpeed);
			SkeletalMeshComponent->Play(true);
		}

		// Update the test ML Deformed skeletal mesh component.
		const FMLDeformerEditorActor* TestMLActor = FindEditorActor(ActorID_Test_MLDeformed);
		SkeletalMeshComponent = TestMLActor ? TestMLActor->GetSkeletalMeshComponent() : nullptr;
		if (SkeletalMeshComponent)
		{
			ChangeSkeletalMeshOnComponent(SkeletalMeshComponent, Model->GetSkeletalMesh());
			SkeletalMeshComponent->SetAnimationMode(EAnimationMode::AnimationSingleNode);
			const float CurrentPlayTime = SkeletalMeshComponent->GetPosition();
			SkeletalMeshComponent->SetAnimation(TestAnimSequence);
			SkeletalMeshComponent->SetPosition(CurrentPlayTime);
			SkeletalMeshComponent->SetPlayRate(TestAnimSpeed);
			SkeletalMeshComponent->Play(true);
			UpdateStepInterpolationMode();
		}

		// Update all compare actors.
		for (FMLDeformerEditorActor* EditorActor : EditorActors)
		{
			if (EditorActor->GetTypeID() != ActorID_Test_Compare)
			{
				continue;
			}

			SkeletalMeshComponent = EditorActor->GetSkeletalMeshComponent();
			if (SkeletalMeshComponent)
			{
				USkeletalMesh* CompareSkelMesh = Model->GetSkeletalMesh();
				const int32 InstanceIndex = EditorActor->GetActorTypeInstanceIndex();
				const FMLDeformerCompareActor& CompareActor = VizSettings->GetCompareActors()[InstanceIndex];
				UMLDeformerAsset* DeformerAsset = CompareActor.DeformerAsset.LoadSynchronous();
				if (!IsCompatibleDeformer(DeformerAsset))
				{
					CompareSkelMesh = nullptr;
				}
				SkeletalMeshComponent->SetSkeletalMesh(CompareSkelMesh);
				SkeletalMeshComponent->SetAnimationMode(EAnimationMode::AnimationSingleNode);
				const float CurrentPlayTime = SkeletalMeshComponent->GetPosition();
				SkeletalMeshComponent->SetAnimation(TestAnimSequence);
				SkeletalMeshComponent->SetPosition(CurrentPlayTime);
				SkeletalMeshComponent->SetPlayRate(TestAnimSpeed);
				SkeletalMeshComponent->Play(true);
			}
		}
	}

	void FMLDeformerEditorModel::OnPostInputAssetChanged()
	{
		CurrentTrainingFrame = -1;
		UpdateTimelineTrainingAnimList();

		Editor->UpdateTimeSliderRange();
		Model->UpdateCachedNumVertices();

		UpdateActorVisibility();
		UpdateNumTrainingFrames();
		UpdateDeformerGraph();
		RefreshMLDeformerComponents();
		
		CreateSamplers();
		UpdateEditorInputInfo();
		UpdateIsReadyForTrainingState();

		const int32 TrainingFrame = Model->GetVizSettings()->GetTrainingFrameNumber();
		SetTrainingFrame(TrainingFrame, false);

		const int32 TestFrame = Model->GetVizSettings()->GetTestingFrameNumber();
		SetTestFrame(TestFrame, false);

		CheckTrainingDataFrameChanged();
	}

	void FMLDeformerEditorModel::OnTimeSliderScrubPositionChanged(double NewScrubTime, bool bIsScrubbing)
	{
		float PlayOffset = static_cast<float>(NewScrubTime);

		UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
		bool bFrameChanged = true;
		if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TrainingData)
		{
			const int32 TargetFrame = GetTrainingFrameAtTime(NewScrubTime);
			PlayOffset = GetTrainingTimeAtFrame(TargetFrame);
			for (FMLDeformerEditorActor* EditorActor : EditorActors)
			{
				if (EditorActor && EditorActor->IsTrainingActor())
				{
					EditorActor->SetPlayPosition(PlayOffset);
				}
			}
			VizSettings->SetTrainingFrameNumber(TargetFrame);
			if (!bIsScrubbing)
			{
				CheckTrainingDataFrameChanged();
			}
		}
		else if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData)
		{
			const int32 TargetFrame = GetTestFrameAtTime(NewScrubTime);
			PlayOffset = GetTestTimeAtFrame(TargetFrame);
			for (FMLDeformerEditorActor* EditorActor : EditorActors)
			{
				if (EditorActor && EditorActor->IsTestActor())
				{
					EditorActor->SetPlayPosition(PlayOffset);
					UMLDeformerComponent* MLDComponent = EditorActor->GetMLDeformerComponent();
					if (MLDComponent)
					{
						if (MLDComponent->GetModelInstance())
						{
							MLDComponent->GetModelInstance()->SetGroundTruthFrameIndex(TargetFrame);
						}
					}
				}
			}
			VizSettings->SetTestingFrameNumber(TargetFrame);
		}
		
		bIsScrubbingTimeline = bIsScrubbing;
		UpdatePaintModePose(/*bFullUpdate=*/!bIsScrubbing);

		// If we scrubbed and we were playing before, and we are in ground truth heat map mode, then we need to possibly enable the heatmap again.
		// We disable it while playing in ground truth mode as we cannot really get correct ground truth heatmaps during playback because of interpolation and other issues.
		// If we do not do this, then while ground truth heat map is enabled and we are playing, and we scrub, then the heat map isn't enabled again.
		if (VizSettings->GetShowHeatMap() && VizSettings->GetHeatMapMode() == EMLDeformerHeatMapMode::GroundTruth)
		{
			UpdateHeatMapMaterialBasedOnMode();
		}
	}

	void FMLDeformerEditorModel::UpdatePaintModePose(bool bFullUpdate)
	{
		if (!Editor->IsInitialized() || !Editor->IsPaintModeActive())
		{
			return;
		}

		UEdMode* ActiveMode = Editor->GetEditorModeManager().GetActiveScriptableMode(UMLDeformerPaintMode::Id);
		UMLDeformerPaintMode* PaintMode = CastChecked<UMLDeformerPaintMode>(ActiveMode);
		if (!PaintMode)
		{
			return;
		}

		UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
		FMLDeformerEditorActor* EditorActor = nullptr;
		if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TrainingData)
		{
			EditorActor = FindEditorActor(ActorID_Train_Base);
		}
		else if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData)
		{
			EditorActor = FindEditorActor(ActorID_Test_MLDeformed);
		}
		else
		{
			checkf(false, TEXT("Unexpected visualization mode"));
		}

		if (EditorActor)
		{		
			PaintMode->UpdatePose(EditorActor->GetSkeletalMeshComponent(), bFullUpdate);
		}
	}

	float FMLDeformerEditorModel::CorrectedFrameTime(int32 FrameNumber, float FrameTimeForFrameNumber, FFrameRate FrameRate)
	{
		const FFrameTime PlayOffsetCurrentTime(FrameRate.AsFrameTime(FrameTimeForFrameNumber));

		if (PlayOffsetCurrentTime.FloorToFrame() != FrameNumber)
		{
			// correct the float for error
			const double CorrectedPlayOffset = FrameRate.AsInterval() * (double)(FrameNumber);
			float FloatPlayOffset = CorrectedPlayOffset;
			FloatPlayOffset = nextafterf(FloatPlayOffset, FloatPlayOffset + 1.0f);
			return FloatPlayOffset;
		}
		return FrameTimeForFrameNumber;
	}

	double FMLDeformerEditorModel::GetTrainingTimeAtFrame(int32 FrameNumber) const
	{
		UAnimSequence* AnimSequence = GetActiveTrainingInputAnimSequence();
		if (AnimSequence)
		{
			const FFrameRate FrameRate = AnimSequence->GetSamplingFrameRate();
			const float UncorrectedTime = AnimSequence->GetTimeAtFrame(FrameNumber);
			// due to floating point errors, return a double that when converted to a float cleanly maps to the value
			return CorrectedFrameTime(FrameNumber, UncorrectedTime, FrameRate);
		}
		return 0.0; 
	}

	int32 FMLDeformerEditorModel::GetTrainingFrameAtTime(double TimeInSeconds) const
	{
		const UAnimSequence* Anim = GetActiveTrainingInputAnimSequence();
		return Anim ? Anim->GetFrameAtTime(TimeInSeconds) : 0;
	}

	double FMLDeformerEditorModel::GetTestTimeAtFrame(int32 FrameNumber) const
	{
		UAnimSequence* AnimSequence = Model->GetVizSettings()->GetTestAnimSequence();
		if (AnimSequence)
		{
			const FFrameRate FrameRate = AnimSequence->GetSamplingFrameRate();
			const float UncorrectedTime = AnimSequence->GetTimeAtFrame(FrameNumber);
			// due to floating point errors, return a double that when converted to a float cleanly maps to the value
			return CorrectedFrameTime(FrameNumber, UncorrectedTime, FrameRate);
		}
		return 0.0; 
	}

	int32 FMLDeformerEditorModel::GetTestFrameAtTime(double TimeInSeconds) const
	{
		return Model->GetVizSettings()->GetTestAnimSequence() ? Model->GetVizSettings()->GetTestAnimSequence()->GetFrameAtTime(TimeInSeconds) : 0;
	}

	void FMLDeformerEditorModel::SetTrainingFrame(int32 FrameNumber, bool bIsScrubbing)
	{
		UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
		VizSettings->SetTrainingFrameNumber(FrameNumber);
		ClampCurrentTrainingFrameIndex();
		if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TrainingData)
		{
			OnTimeSliderScrubPositionChanged(GetTrainingTimeAtFrame(FrameNumber), bIsScrubbing);
		}
	}

	void FMLDeformerEditorModel::SetTestFrame(int32 FrameNumber, bool bIsScrubbing)
	{
		UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
		VizSettings->SetTestingFrameNumber(FrameNumber);
		ClampCurrentTestFrameIndex();
		if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData)
		{
			OnTimeSliderScrubPositionChanged(GetTestTimeAtFrame(FrameNumber), bIsScrubbing);
		}
	}

	void FMLDeformerEditorModel::OnPropertyChanged(FPropertyChangedEvent& PropertyChangedEvent)
	{
		const FProperty* Property = PropertyChangedEvent.Property;
		if (Property == nullptr)
		{
			return;
		}

		const bool bPropertyValueSetOrResetToDefault = (PropertyChangedEvent.ChangeType & EPropertyChangeType::ValueSet) || (PropertyChangedEvent.ChangeType & EPropertyChangeType::ResetToDefault);

		// When we change one of these properties below, restart animations etc.
		if (Property->GetFName() == UMLDeformerModel::GetSkeletalMeshPropertyName())
		{
			TriggerInputAssetChanged();
			UpdateLODMappings();
			SetResamplingInputOutputsNeeded(true);
			Model->InitVertexMap();
			Model->InitGPUData();
			SampleDeltas();
			if (GetEditor())
			{
				GetEditor()->ZoomOnActors();
			}
		}
		else if (Property->GetFName() == UMLDeformerVizSettings::GetTestAnimSequencePropertyName())
		{
			TriggerInputAssetChanged(true);
		}
		else if (Property->GetFName() == UMLDeformerVizSettings::GetCompareActorsPropertyName())
		{
			const int32 ArrayIndex = PropertyChangedEvent.GetArrayIndex(PropertyChangedEvent.Property->GetFName().ToString());

			// Make sure we're not moving elements.
			switch (PropertyChangedEvent.ChangeType)
			{
				case EPropertyChangeType::ArrayAdd:
				{
					check(ArrayIndex != INDEX_NONE);
					AddCompareActor(ArrayIndex);
					break;
				}
				case EPropertyChangeType::ArrayRemove:
				{
					check(ArrayIndex != INDEX_NONE);
					RemoveCompareActor(ArrayIndex);
					break;
				}
				case EPropertyChangeType::ArrayClear:
				{
					RemoveAllCompareActors();
					break;
				}
				case EPropertyChangeType::Duplicate:
				{
					check(ArrayIndex != INDEX_NONE);
					AddCompareActor(ArrayIndex);
					break;
				}
				default:;
			};

			TriggerInputAssetChanged(true);
			UpdateActorVisibility();
			UpdateCompareActorLabels();
		}
		else if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(FMLDeformerCompareActor, DeformerAsset))
		{
			if (bPropertyValueSetOrResetToDefault)
			{
				TriggerInputAssetChanged(true);
				UpdateActorVisibility();
				UpdateCompareActorLabels();
			}
		}
		else if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(FMLDeformerCompareActor, Name))
		{
			if (bPropertyValueSetOrResetToDefault)
			{
				UpdateCompareActorLabels();
			}
		}
		else
		if (Property->GetFName() == UMLDeformerModel::GetAlignmentTransformPropertyName())
		{
			if (bPropertyValueSetOrResetToDefault)
			{
				SetResamplingInputOutputsNeeded(true);
				SampleDeltas();
			}
		}
		else if (Property->GetFName() == UMLDeformerModel::GetMaxTrainingFramesPropertyName())
		{
			UpdateNumTrainingFrames();
			SetResamplingInputOutputsNeeded(true);
			GetEditor()->GetModelDetailsView()->ForceRefresh();
		}
		else
		if (Property->GetFName() == UMLDeformerModel::GetBoneIncludeListPropertyName() ||
			Property->GetFName() == UMLDeformerModel::GetCurveIncludeListPropertyName())
		{
			SetResamplingInputOutputsNeeded(true);
			UpdateIsReadyForTrainingState();
			RefreshInputWidget();
		}
		else if (Property->GetFName() == UMLDeformerModel::GetMaxNumLODsPropertyName())
		{
			if (bPropertyValueSetOrResetToDefault)
			{
				OnMaxNumLODsChanged();
			}
		}
		else if (Property->GetFName() == UMLDeformerVizSettings::GetAnimPlaySpeedPropertyName())
		{
			UpdateTestAnimPlaySpeed();
		}
		else if (Property->GetFName() == UMLDeformerVizSettings::GetTrainingFrameNumberPropertyName())
		{
			ClampCurrentTrainingFrameIndex();
			const int32 CurrentFrameNumber = Model->GetVizSettings()->GetTrainingFrameNumber();
			OnTimeSliderScrubPositionChanged(GetTrainingTimeAtFrame(CurrentFrameNumber), false);
		}
		else if (Property->GetFName() == UMLDeformerVizSettings::GetTestingFrameNumberPropertyName())
		{
			ClampCurrentTestFrameIndex();
			const int32 CurrentFrameNumber = Model->GetVizSettings()->GetTestingFrameNumber();
			OnTimeSliderScrubPositionChanged(GetTestTimeAtFrame(CurrentFrameNumber), false);
		}
		else if (Property->GetFName() == UMLDeformerVizSettings::GetShowHeatMapPropertyName())
		{
			SetHeatMapMaterialEnabled(Model->GetVizSettings()->GetShowHeatMap());
			UpdateDeformerGraph();
			UpdateStepInterpolationMode();
			UpdateHeatMapMaterialBasedOnMode();
		}
		else if (Property->GetFName() == UMLDeformerVizSettings::GetHeatMapModePropertyName())
		{
			UpdateStepInterpolationMode();
			UpdateHeatMapMaterialBasedOnMode();
		}
		else
		if (Property->GetFName() == UMLDeformerVizSettings::GetDrawLinearSkinnedActorPropertyName() ||
			Property->GetFName() == UMLDeformerVizSettings::GetDrawMLDeformedActorPropertyName() ||
			Property->GetFName() == UMLDeformerVizSettings::GetDrawGroundTruthActorPropertyName() ||
			Property->GetFName() == UMLDeformerVizSettings::GetDrawMLCompareActorsPropertyName())
		{
			UpdateActorVisibility();
		}
		else if (Property->GetFName() == UMLDeformerVizSettings::GetDrawVertexDeltasPropertyName())
		{
			SampleDeltas();
		} 
		else if (Property->GetFName() == UMLDeformerVizSettings::GetDeformerGraphPropertyName())
		{
			UpdateDeformerGraph();
			GetEditor()->GetVizSettingsDetailsView()->ForceRefresh();
			GetEditor()->GetModelDetailsView()->ForceRefresh();
		}
	}

	void FMLDeformerEditorModel::OnPostTransacted(const FTransactionObjectEvent& Event)
	{
		const TArray<FName>& PropertiesChanged = Event.GetChangedProperties();
		for (const FName Name : PropertiesChanged)
		{
			if (Event.GetEventType() == ETransactionObjectEventType::UndoRedo)
			{
				BroadcastPropertyChanged(Name);
			}
		}
	}

	FMLDeformerEditorActor* FMLDeformerEditorModel::GetVisualizationModeBaseActor() const
	{
		const UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
		const bool bTestMode = VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData;
		return bTestMode ? FindEditorActor(ActorID_Test_Base) : FindEditorActor(ActorID_Train_Base);
	}

	void FMLDeformerEditorModel::OnPlayPressed()
	{
		// The play button shouldn't do anything in training mode.
		const UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
		const FMLDeformerEditorActor* BaseActor = GetVisualizationModeBaseActor();
		const bool bMustPause = (BaseActor && BaseActor->IsPlaying());
		const bool bTestMode = (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData);

		// Update the current frame numbers.
		if (bMustPause)
		{
			if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TrainingData)
			{
				SetTrainingFrame(GetTrainingFrameAtTime(CalcTimelinePosition()), false);
			}
			else if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData)
			{
				SetTestFrame(GetTestFrameAtTime(CalcTimelinePosition()), false);
			}
		}

		for (FMLDeformerEditorActor* EditorActor : EditorActors)
		{
			if (EditorActor && ((EditorActor->IsTestActor() && bTestMode) || (EditorActor->IsTrainingActor() && !bTestMode)))
			{
				EditorActor->SetPlaySpeed(VizSettings->GetAnimPlaySpeed());
				EditorActor->Pause(bMustPause);
			}
		}

		if (VizSettings->GetShowHeatMap() && VizSettings->GetHeatMapMode() == EMLDeformerHeatMapMode::GroundTruth)
		{
			if (!bMustPause)
			{
				SetHeatMapMaterialEnabled(false);
			}
			else
			{
				SetHeatMapMaterialEnabled(VizSettings->GetShowHeatMap());
			}
		}
	}

	void FMLDeformerEditorModel::UpdateHeatMapMaterialBasedOnMode()
	{
		const FMLDeformerEditorActor* BaseActor = GetVisualizationModeBaseActor();
		const bool bIsPlaying = (BaseActor && BaseActor->IsPlaying());
		const UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
		const bool bInTestMode = (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData);

		if (bIsPlaying && VizSettings->GetHeatMapMode() == EMLDeformerHeatMapMode::GroundTruth)
		{
			SetHeatMapMaterialEnabled(false);
		}
		else
		{
			SetHeatMapMaterialEnabled(VizSettings->GetShowHeatMap());
		}
	}

	bool FMLDeformerEditorModel::IsPlayingAnim() const
	{
		const FMLDeformerEditorActor* BaseActor = GetVisualizationModeBaseActor();
		if (BaseActor)
		{
			UDebugSkelMeshComponent* SkeletalMeshComponent = BaseActor->GetSkeletalMeshComponent();
			if (SkeletalMeshComponent)
			{
				return !SkeletalMeshComponent->bPauseAnims;
			}
		}
		return false;
	}

	bool FMLDeformerEditorModel::IsPlayingForward() const
	{
		FMLDeformerEditorActor* BaseActor = GetVisualizationModeBaseActor();
		if (BaseActor)
		{
			return BaseActor->GetPlaySpeed() > 0.0f;
		}
		return false;
	}


	double FMLDeformerEditorModel::CalcTrainingTimelinePosition() const
	{
		FMLDeformerEditorActor* EditorActor = FindEditorActor(ActorID_Train_GroundTruth);
		if (EditorActor && EditorActor->HasVisualMesh())
		{
			return EditorActor->GetPlayPosition();
		}

		EditorActor = FindEditorActor(ActorID_Train_Base);
		if (EditorActor && EditorActor->HasVisualMesh())
		{
			return EditorActor->GetPlayPosition();
		}

		return 0.0;
	}

	double FMLDeformerEditorModel::CalcTestTimelinePosition() const
	{
		FMLDeformerEditorActor* EditorActor = FindEditorActor(ActorID_Test_GroundTruth);
		if (EditorActor && EditorActor->HasVisualMesh())
		{
			return EditorActor->GetPlayPosition();
		}

		EditorActor = FindEditorActor(ActorID_Test_Base);
		if (EditorActor && EditorActor->HasVisualMesh())
		{
			return EditorActor->GetPlayPosition();
		}

		return 0.0;
	}

	void FMLDeformerEditorModel::UpdateTestAnimPlaySpeed()
	{
		UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
		const float Speed = VizSettings->GetAnimPlaySpeed();
		for (FMLDeformerEditorActor* EditorActor : EditorActors)
		{
			if (EditorActor && EditorActor->IsTestActor()) // Only do test actors, no training actors.
			{
				EditorActor->SetPlaySpeed(Speed);
			}
		}
	}

	void FMLDeformerEditorModel::ClampCurrentTrainingFrameIndex()
	{
		UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
		const int32 ActiveAnimIndex = GetActiveTrainingInputAnimIndex();
		if (ActiveAnimIndex != INDEX_NONE)
		{
			const int32 AnimNumFrames = GetTrainingInputAnim(ActiveAnimIndex)->ExtractNumAnimFrames();
			const int32 MaxFrameIndex = (AnimNumFrames > 0) ? AnimNumFrames - 1 : 0;
			const int32 ClampedNumber = FMath::Min(VizSettings->GetTrainingFrameNumber(), MaxFrameIndex);
			VizSettings->SetTrainingFrameNumber(ClampedNumber);
		}
		else
		{
			VizSettings->SetTrainingFrameNumber(0);
		}
	}

	void FMLDeformerEditorModel::ClampCurrentTestFrameIndex()
	{
		UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
		if (GetNumTestFrames() > 0)
		{
			const int32 ClampedNumber = FMath::Min(VizSettings->GetTestingFrameNumber(), GetNumTestFrames() - 1);
			VizSettings->SetTestingFrameNumber(ClampedNumber);
		}
		else
		{
			VizSettings->SetTestingFrameNumber(0);
		}
	}

	int32 FMLDeformerEditorModel::GetNumTestFrames() const
	{
		const UAnimSequence* Sequence = Model->GetVizSettings()->GetTestAnimSequence();
		if (Sequence)
		{
			return Sequence->GetNumberOfSampledKeys();
		}
		return 0;
	}

	int32 FMLDeformerEditorModel::GetNumFramesForTraining() const 
	{
		return FMath::Min(GetNumTrainingFrames(), Model->GetTrainingFrameLimit());
	}

	FText FMLDeformerEditorModel::GetBaseAssetChangedErrorText() const
	{
		FText Result;
		if (Model->GetSkeletalMesh() && Model->GetInputInfo())
		{
			if (Model->GetNumBaseMeshVerts() != Model->GetInputInfo()->GetNumBaseMeshVertices() &&
				Model->GetNumBaseMeshVerts() > 0 && Model->GetInputInfo()->GetNumBaseMeshVertices() > 0)
			{
				Result = FText::Format(LOCTEXT("BaseMeshMismatch", "Number of vertices in base mesh has changed from {0} to {1} vertices since this ML Deformer Asset was saved! {2}"),
					Model->GetInputInfo()->GetNumBaseMeshVertices(),
					Model->GetNumBaseMeshVerts(),
					IsTrained() ? LOCTEXT("BaseMeshMismatchNN", "Model needs to be retrained!") : FText());
			}
		}
		return Result;
	}

	FText FMLDeformerEditorModel::GetVertexMapChangedErrorText() const
	{
		FText Result;
		if (Model->GetSkeletalMesh())
		{
			bool bVertexMapMatch = true;
			const FSkeletalMeshModel* ImportedModel = Model->GetSkeletalMesh()->GetImportedModel();
			if (ImportedModel)
			{
				const TArray<int32>& MeshVertexMap = ImportedModel->LODModels[0].MeshToImportVertexMap;
				const TArray<int32>& ModelVertexMap = Model->GetVertexMap();
				if (MeshVertexMap.Num() == ModelVertexMap.Num())
				{
					for (int32 Index = 0; Index < ModelVertexMap.Num(); ++Index)
					{
						if (MeshVertexMap[Index] != ModelVertexMap[Index])
						{
							bVertexMapMatch = false;
							break;
						}
					}
					
					if (!bVertexMapMatch)
					{
						Result = FText(LOCTEXT("VertexMapMismatch", "The vertex order of your Skeletal Mesh changed."));
					}
				}
			}
		}
		return Result;
	}

	FText FMLDeformerEditorModel::GetSkeletalMeshNeedsReimportErrorText() const
	{
		if (Model->GetSkeletalMesh())
		{
			constexpr int32 LODIndex = 0;
			const FMeshDescription* MeshDescription = Model->GetSkeletalMesh()->GetMeshDescription(LODIndex);
			if (!ensure(MeshDescription))
			{
				return LOCTEXT("SkelMeshIsEmpty", "Skeletal Mesh asset is empty.");
			}
			const FSkeletalMeshConstAttributes MeshAttributes(*MeshDescription);
			if (MeshAttributes.GetNumSourceGeometryParts() == 0)
			{
				return LOCTEXT("SkelMeshNeedsReimport", "Skeletal Mesh asset needs to be reimported.");
			}
		}

		return {};
	}

	FText FMLDeformerEditorModel::GetInputsErrorText() const
	{
		if (Model->GetSkeletalMesh() && GetEditorInputInfo()->IsEmpty())
		{
			FString ErrorString;
			ErrorString += FText(LOCTEXT("InputsNeededErrorText", "The training process needs inputs.\n")).ToString();

			ErrorString.RemoveFromEnd("\n");
			return FText::FromString(ErrorString);
		}

		return FText();
	}

	FText FMLDeformerEditorModel::GetIncompatibleSkeletonErrorText(const USkeletalMesh* InSkelMesh, const UAnimSequence* InAnimSeq) const
	{
		FText Result;
		if (InSkelMesh && InAnimSeq)
		{
			if (!InSkelMesh->GetSkeleton()->IsCompatibleForEditor(InAnimSeq->GetSkeleton()))
			{
				Result = LOCTEXT("SkeletonMismatch", "The base skeletal mesh and anim sequence use incompatible skeletons. The animation might not play correctly.");
			}
		}
		return Result;
	}

	FText FMLDeformerEditorModel::GetTargetAssetChangedErrorText() const
	{
		FText Result;

		UMLDeformerInputInfo* InputInfo = Model->GetInputInfo();
		if (Model->HasTrainingGroundTruth() && InputInfo)
		{
			if (Model->GetNumTargetMeshVerts() != InputInfo->GetNumTargetMeshVertices() &&
				Model->GetNumTargetMeshVerts() > 0 && InputInfo->GetNumTargetMeshVertices() > 0)
			{
				Result = FText::Format(LOCTEXT("TargetMeshMismatch", "Number of vertices in target mesh has changed from {0} to {1} vertices since this ML Deformer Asset was saved! {2}"),
					InputInfo->GetNumTargetMeshVertices(),
					Model->GetNumTargetMeshVerts(),
					IsTrained() ? LOCTEXT("BaseMeshMismatchModel", "Model needs to be retrained!") : FText());
			}
		}

		return Result;
	}

	void FMLDeformerEditorModel::AddAnimatedBonesToBonesIncludeList(const UAnimSequence* AnimSequence)
	{
		if (AnimSequence == nullptr)
		{
			UE_LOGF(LogMLDeformer, Warning, "Cannot initialize bone list as no Anim Sequence has been picked.");
			return;
		}

		const IAnimationDataModel* DataModel = AnimSequence->GetDataModel();
		if (!DataModel)
		{
			UE_LOGF(LogMLDeformer, Warning, "Anim sequence has no data model.");
			return;
		}

		check(Model->GetSkeletalMesh());
		USkeleton* Skeleton = Model->GetSkeletalMesh()->GetSkeleton();
		check(Skeleton);

		// Iterate over all bones that are both in the skeleton and the animation.
		TArray<FName> AnimatedBoneList;
		const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
		const int32 NumBones = RefSkeleton.GetNum();

		TArray<FName> TrackNames;
		DataModel->GetBoneTrackNames(TrackNames);
		TArray<FTransform> BoneTransforms;		
		for (int32 Index = 0; Index < NumBones; ++Index)
		{
			const FName BoneName = RefSkeleton.GetBoneName(Index);
			if (!TrackNames.Contains(BoneName))
			{
				continue;
			}

			// Check if there is actually animation data.
			BoneTransforms.Reset();
			DataModel->GetBoneTrackTransforms(BoneName, BoneTransforms);
			bool bIsAnimated = false;
			if (!BoneTransforms.IsEmpty())
			{
				const FQuat FirstQuat = BoneTransforms[0].GetRotation();
				for (const FTransform& KeyValue : BoneTransforms)
				{
					if (!KeyValue.GetRotation().Equals(FirstQuat))
					{
						bIsAnimated = true;
						break;
					}
				}

				if (!bIsAnimated)
				{
					UE_LOGF(LogMLDeformer, Display, "Bone '%ls' has keyframes but isn't animated.", *BoneName.ToString());
				}
			}

			if (bIsAnimated)
			{
				AnimatedBoneList.Add(BoneName);
			}
		}

		// Init the bone include list using the animated bones.
		TArray<FBoneReference>& BoneList = Model->GetBoneIncludeList();
		for (const FName BoneName : AnimatedBoneList)
		{
			BoneList.AddUnique(FBoneReference(BoneName));
		}
	}

	void FMLDeformerEditorModel::AddAnimatedBonesToBonesIncludeList()
	{
		if (!Model->GetSkeletalMesh())
		{
			UE_LOGF(LogMLDeformer, Warning, "Skeletal Mesh has not been set.");
			UpdateEditorInputInfo();
			return;
		}

		USkeleton* Skeleton = Model->GetSkeletalMesh()->GetSkeleton();
		if (!Skeleton)
		{
			UE_LOGF(LogMLDeformer, Warning, "Skeletal Mesh has no skeleton.");
			UpdateEditorInputInfo();
			return;
		}

		// Add animated bones from all animations.
		const int32 NumAnims = GetNumTrainingInputAnims();
		for (int32 Index = 0; Index < NumAnims; ++Index)
		{
			FMLDeformerTrainingInputAnim* Anim = GetTrainingInputAnim(Index);
			if (Anim && Anim->IsEnabled())
			{
				AddAnimatedBonesToBonesIncludeList(Anim->GetAnimSequence());
			}
		}

		UpdateEditorInputInfo();
	}

	void FMLDeformerEditorModel::AddAnimatedCurvesToCurvesIncludeList(const UAnimSequence* AnimSequence)
	{
		if (AnimSequence == nullptr)
		{
			UE_LOGF(LogMLDeformer, Warning, "Cannot initialize curve list as no Anim Sequence has been picked.");
			return;
		}

		const IAnimationDataModel* DataModel = AnimSequence->GetDataModel();
		if (!DataModel)
		{
			UE_LOGF(LogMLDeformer, Warning, "Anim sequence has no data model.");
			return;
		}

		check(Model->GetSkeletalMesh());

		TArray<FName> CurveList;
		UMLDeformerInputInfo::BuildCurveList(Model->GetSkeletalMesh(), CurveList);
		
		const USkeleton* Skeleton = Model->GetSkeletalMesh()->GetSkeleton();
		check(Skeleton);

		TArray<FName> AnimatedCurveList;
		for (const FName SkeletonCurveName : CurveList)
		{
			const TArray<FFloatCurve>& AnimCurves = DataModel->GetFloatCurves();
			for (const FFloatCurve& AnimCurve : AnimCurves)
			{
				if (AnimCurve.GetName() == SkeletonCurveName)
				{
					TArray<float> TimeValues;
					TArray<float> KeyValues;
					AnimCurve.GetKeys(TimeValues, KeyValues);
					if (KeyValues.Num() > 0)
					{
						const float FirstKeyValue = KeyValues[0];
						for (float CurKeyValue : KeyValues)
						{
							if (CurKeyValue != FirstKeyValue)
							{
								AnimatedCurveList.Add(SkeletonCurveName);
								break;
							}
						}
					}
					break;
				}
			}
		}

		TArray<FMLDeformerCurveReference>& CurveIncludeList = Model->GetCurveIncludeList();
		for (const FName CurveName : AnimatedCurveList)
		{
			CurveIncludeList.AddUnique(FMLDeformerCurveReference(CurveName));
		}
	}

	void FMLDeformerEditorModel::AddAnimatedCurvesToCurvesIncludeList()
	{
		if (!Model->GetSkeletalMesh())
		{
			UE_LOGF(LogMLDeformer, Warning, "Skeletal Mesh has not been set.");
			UpdateEditorInputInfo();
			return;
		}

		USkeleton* Skeleton = Model->GetSkeletalMesh()->GetSkeleton();
		if (!Skeleton)
		{
			UE_LOGF(LogMLDeformer, Warning, "Skeletal Mesh has no skeleton.");
			UpdateEditorInputInfo();
			return;
		}

		// Add animated curves from all animations.
		const int32 NumAnims = GetNumTrainingInputAnims();
		for (int32 Index = 0; Index < NumAnims; ++Index)
		{
			FMLDeformerTrainingInputAnim* Anim = GetTrainingInputAnim(Index);
			if (Anim && Anim->IsEnabled())
			{
				AddAnimatedCurvesToCurvesIncludeList(Anim->GetAnimSequence());
			}
		}

		UpdateEditorInputInfo();
	}

	void FMLDeformerEditorModel::InitInputInfo(UMLDeformerInputInfo* InputInfo)
	{
		InputInfo->Reset();

		TArray<FName>& BoneNames = InputInfo->GetBoneNames();
		TArray<FName>& CurveNames = InputInfo->GetCurveNames();

		USkeletalMesh* SkeletalMesh = Model->GetSkeletalMesh();
		InputInfo->SetSkeletalMesh(SkeletalMesh);

		BoneNames.Reset();
		CurveNames.Reset();

		InputInfo->SetNumBaseVertices(Model->GetNumBaseMeshVerts());
		InputInfo->SetNumTargetVertices(Model->GetNumTargetMeshVerts());

		const USkeleton* Skeleton = Model->GetSkeletalMesh() ? Model->GetSkeletalMesh()->GetSkeleton() : nullptr;

		// Handle bones.
		if (SkeletalMesh)
		{
			// Include all the bones when no list was provided.
			const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
			for (const FBoneReference& BoneReference : Model->GetBoneIncludeList())
			{
				if (BoneReference.BoneName.IsValid())
				{
					const FName BoneName = BoneReference.BoneName;
					if (RefSkeleton.FindBoneIndex(BoneName) == INDEX_NONE)
					{
						UE_LOGF(LogMLDeformer, Warning, "Bone '%ls' in the bones include list doesn't exist, ignoring it.", *BoneName.ToString());
						continue;
					}

					BoneNames.Add(BoneName);
				}
			}

			// Anim curves.
			if (Skeleton)
			{
				for (const FMLDeformerCurveReference& CurveReference : Model->GetCurveIncludeList())
				{
					if (CurveReference.CurveName.IsValid())
					{
						CurveNames.Add(CurveReference.CurveName);
					}
				}
			}
		}

	}

	void FMLDeformerEditorModel::InitBoneIncludeListToAnimatedBonesOnly()
	{
		TArray<FBoneReference>& BoneList = Model->GetBoneIncludeList();
		BoneList.Empty();
		AddAnimatedBonesToBonesIncludeList();
	}

	void FMLDeformerEditorModel::InitCurveIncludeListToAnimatedCurvesOnly()
	{
		TArray<FMLDeformerCurveReference>& CurveList = Model->GetCurveIncludeList();
		CurveList.Empty();
		AddAnimatedCurvesToCurvesIncludeList();
	}

	FMLDeformerSampler* FMLDeformerEditorModel::GetSamplerForActiveAnim() const
	{
		const int32 ActiveAnimIndex = GetActiveTrainingInputAnimIndex();
		return (ActiveAnimIndex != INDEX_NONE && Samplers.IsValidIndex(ActiveAnimIndex)) ? Samplers[ActiveAnimIndex].Get() : nullptr;
	}

	void FMLDeformerEditorModel::SetActiveTrainingInputAnimIndex(int32 Index)
	{
		check(IsAnimIndexValid(Index));
		ActiveTrainingInputAnimIndex = Index;
	}

	void FMLDeformerEditorModel::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
	{
		FMLDeformerSampler* Sampler = GetSamplerForActiveAnim();

		// Make sure that before we render anything, that our sampler is ready.
		if (Sampler && !Sampler->IsInitialized())
		{
			const int32 ActiveAnimIndex = GetActiveTrainingInputAnimIndex();
			Sampler->Init(this, ActiveAnimIndex); // This can still fail.
			Sampler->SetVertexDeltaSpace(EVertexDeltaSpace::PostSkinning);
			if (Sampler->IsInitialized()) // If we actually managed to initialize this frame.
			{
				SampleDeltas(); // Update the deltas.
			}
		}

		const UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
		if (Sampler && Sampler->IsInitialized() && VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TrainingData)
		{
			const TArray<float>& VertexDeltas = Sampler->GetVertexDeltas();
			const TArray<FVector3f>& LinearSkinnedPositions = Sampler->GetSkinnedVertexPositions();
			bool bDrawDeltas = VizSettings->GetDrawVertexDeltas();

			// Disable drawing deltas when we're playing the training anim sequence, as that can get too slow.
			const FMLDeformerEditorActor* BaseActor = FindEditorActor(ActorID_Train_Base);
			if (BaseActor)
			{
				bDrawDeltas &= !BaseActor->IsPlaying();
			}

			USkeletalMesh* SkelMesh = Model->GetSkeletalMesh();
			if (bDrawDeltas && Sampler->GetVertexDeltas().IsEmpty() && Model->GetSkeletalMesh() && SkelMesh->GetImportedModel())
			{
				SampleDeltas();
			}

			bDrawDeltas &= ((VertexDeltas.Num() / 3) == LinearSkinnedPositions.Num());

			// Draw the deltas for the current frame.
			if (bDrawDeltas && Model->GetSkeletalMesh() && SkelMesh->GetImportedModel() && !IsScrubbingTimeline())
			{
				const FLinearColor DeltasColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.Deltas.Color");
				const FLinearColor DebugVectorsColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.DebugVectors.Color");
				const FLinearColor DebugVectorsColor2 = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.DebugVectors.Color2");
				const uint8 DepthGroup = VizSettings->GetXRayDeltas() ? 100 : 0;

				const int32 LOD = FindEditorActor(ActorID_Train_Base)->GetSkeletalMeshComponent()->GetPredictedLODLevel();
				if (LODMappings.IsValidIndex(LOD))
				{
					const TArray<int32>& Mapping = LODMappings[LOD].VtxMappingToLODZero;	// Map from current LOD render vertex index into LOD0 render vertex index.
					const FSkeletalMeshLODModel& LODModel = SkelMesh->GetImportedModel()->LODModels[LOD];

					// Get the current LOD skinned positions. We use those as starting point for the delta lines.
					Sampler->ExtractSkinnedPositions(LOD, SkinnedPositions);

					// Render a delta for every render vertex in this LOD.
					if (!Mapping.IsEmpty()) // If we have LOD mapping data.
					{
						const TArray<int32>& MeshToImportVertexMap = SkelMesh->GetImportedModel()->LODModels[0].MeshToImportVertexMap;
						for (int32 RenderVertexIndex = 0; RenderVertexIndex < Mapping.Num(); ++RenderVertexIndex)
						{
							const int32 LOD0VertexIndex = Mapping[RenderVertexIndex]; // Get the vertex number in LOD0.
							const int32 VertexIndex = MeshToImportVertexMap[LOD0VertexIndex]; // Get the imported vertex number.
							const int32 ArrayIndex = 3 * VertexIndex; // 3 floats per delta, so multiply by 3.
							const FVector Delta(
								VertexDeltas[ArrayIndex], 
								VertexDeltas[ArrayIndex + 1], 
								VertexDeltas[ArrayIndex + 2]);
					
							// The skinned vertex positions have only the number of imported vertices number of positions, which can differ from the render vertex count.
							const FVector VertexPos = (FVector)SkinnedPositions[LODModel.MeshToImportVertexMap[RenderVertexIndex]];
							PDI->DrawLine(VertexPos, VertexPos + Delta, DeltasColor, DepthGroup);
						}
					}
					else
					{
						for (int32 Index = 0; Index < LinearSkinnedPositions.Num(); ++Index)
						{
							const int32 ArrayIndex = 3 * Index;
							const FVector Delta(
								VertexDeltas[ArrayIndex], 
								VertexDeltas[ArrayIndex + 1], 
								VertexDeltas[ArrayIndex + 2]);
							const FVector VertexPos = (FVector)LinearSkinnedPositions[Index];
							PDI->DrawLine(VertexPos, VertexPos + Delta, DeltasColor, DepthGroup);
						}
					}
				}
			}
		}
	}

	void FMLDeformerEditorModel::DrawPIEDebugActors()
	{
		// Draw all debuggable actor bounds and names.
		UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
		if (Editor && VizSettings->GetDrawDebugActorBounds() && Editor->GetDebugWidget().IsValid())
		{
			const AActor* ActiveDebugActor = Editor->GetDebugActor();

			FString Name;
			TSharedPtr<SMLDeformerDebugSelectionWidget> DebugWidget = Editor->GetDebugWidget();
			for (TSharedPtr<FMLDeformerDebugActor> DebuggableActor : DebugWidget->GetActors())
			{
				if (!DebuggableActor.IsValid() || 
					!IsValid(DebuggableActor->Actor) || 
					!DebuggableActor->Actor->GetOuter())
				{
					continue;
				}

				// Get the actor's bounding box.
				const UWorld* World = DebuggableActor->Actor->GetWorld();
				FVector Center;
				FVector Extents;
				DebuggableActor->Actor->GetActorBounds(true, Center, Extents, false);

				const FVector TextStart = DebuggableActor->Actor->GetTransform().GetTranslation();
				Name = DebuggableActor->Actor->GetName();
				FColor TextColor;
				FColor BoxColor;
				if (DebuggableActor->Actor == ActiveDebugActor)
				{
					TextColor = FColor::Yellow;
					BoxColor = FColor::Green;
				}
				else
				{
					TextColor = FColor::White;
					BoxColor = VizSettings->GetDebugBoundsColor();
				}

				DrawDebugBox(World, Center, Extents, BoxColor, false);
				DrawDebugString(World, TextStart, Name, nullptr, TextColor, 0.0f, false, 1.0f);
			}
		}
	}

	void FMLDeformerEditorModel::SampleDeltas()
	{
		UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
		ClampCurrentTrainingFrameIndex();

		FMLDeformerSampler* Sampler = GetSamplerForActiveAnim();

		// If we have no Persona toolkit yet, then it is not yet safe to init the sampler.
		if (Editor->GetPersonaToolkitPointer() != nullptr && Sampler && !Sampler->IsInitialized())
		{
			const int32 ActiveAnimIndex = GetActiveTrainingInputAnimIndex();
			Sampler->Init(this, ActiveAnimIndex);
		}

		if (Sampler && Sampler->IsInitialized())
		{
			Sampler->SetVertexDeltaSpace(EVertexDeltaSpace::PostSkinning);
			Sampler->Sample(VizSettings->GetTrainingFrameNumber());
		}
	}

	void FMLDeformerEditorModel::CheckTrainingDataFrameChanged()
	{
		UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
		ClampCurrentTrainingFrameIndex();
		if (CurrentTrainingFrame != VizSettings->GetTrainingFrameNumber())
		{
			OnTrainingDataFrameChanged();
		}
	}

	void FMLDeformerEditorModel::OnTrainingDataFrameChanged()
	{
		UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();

		// If the current frame number changed, re-sample the deltas if needed.
		if (CurrentTrainingFrame != VizSettings->GetTrainingFrameNumber())
		{
			CurrentTrainingFrame = VizSettings->GetTrainingFrameNumber();
			if (VizSettings->GetDrawVertexDeltas() && VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TrainingData)
			{
				SampleDeltas();
			}
		}
	}

	void FMLDeformerEditorModel::RefreshMLDeformerComponents()
	{
		// Update the ML Deformed actor.
		FMLDeformerEditorActor* EditorActor = FindEditorActor(ActorID_Test_MLDeformed);
		if (EditorActor && EditorActor->GetMLDeformerComponent())
		{
			USkeletalMeshComponent* SkelMeshComponent = EditorActor ? EditorActor->GetSkeletalMeshComponent() : nullptr;
			UMLDeformerAsset* DeformerAsset = GetModel()->GetDeformerAsset();
			EditorActor->GetMLDeformerComponent()->SetupComponent(DeformerAsset, SkelMeshComponent);
			UMLDeformerModelInstance* ModelInstance = EditorActor->GetMLDeformerComponent()->GetModelInstance();
			if (ModelInstance)
			{
				ModelInstance->UpdateCompatibilityStatus();
			}
		}

		// Update all compare actors.
		for (FMLDeformerEditorActor* CompareEditorActor : EditorActors)
		{
			if (CompareEditorActor->GetTypeID() != ActorID_Test_Compare)
			{
				continue;
			}

			const int32 InstanceIndex = CompareEditorActor->GetActorTypeInstanceIndex();
			const FMLDeformerCompareActor& CompareActor = Model->GetVizSettings()->GetCompareActors()[InstanceIndex];

			if (CompareEditorActor->GetMLDeformerComponent())
			{
				USkeletalMeshComponent* SkelMeshComponent = CompareEditorActor->GetSkeletalMeshComponent();
				UMLDeformerAsset* DeformerAsset = CompareActor.DeformerAsset.LoadSynchronous();
				if (!IsCompatibleDeformer(DeformerAsset))
				{
					DeformerAsset = nullptr;
				}
				CompareEditorActor->GetMLDeformerComponent()->SetupComponent(DeformerAsset, SkelMeshComponent);
			}
		}
	}

	void FMLDeformerEditorModel::CreateHeatMapMaterial()
	{
		const FString HeatMapMaterialPath = GetHeatMapMaterialPath();
		UObject* MaterialObject = StaticLoadObject(UMaterial::StaticClass(), nullptr, *HeatMapMaterialPath);
		HeatMapMaterial = Cast<UMaterial>(MaterialObject);
	}

	void FMLDeformerEditorModel::CreateHeatMapDeformerGraph()
	{
		const FString HeatMapDeformerPath = GetHeatMapDeformerGraphPath();
		if (!HeatMapDeformerPath.IsEmpty())
		{
			UObject* DeformerObject = StaticLoadObject(UMeshDeformer::StaticClass(), nullptr, *HeatMapDeformerPath);
			HeatMapDeformerGraph = Cast<UMeshDeformer>(DeformerObject);
		}
		else
		{
			HeatMapDeformerGraph = nullptr;
		}

		const FString HeatMapDeformerDualQuatPath = GetHeatMapDeformerGraphDualQuatPath();
		if (!HeatMapDeformerDualQuatPath.IsEmpty())
		{
			UObject* DeformerObjectDualQuat = StaticLoadObject(UMeshDeformer::StaticClass(), nullptr, *HeatMapDeformerDualQuatPath);
			HeatMapDeformerGraphDualQuat = Cast<UMeshDeformer>(DeformerObjectDualQuat);
		}
		else
		{
			HeatMapDeformerGraphDualQuat = nullptr;
		}
	}

	void FMLDeformerEditorModel::CreateHeatMapAssets()
	{
		CreateHeatMapMaterial();
		CreateHeatMapDeformerGraph();
	}

	UMeshDeformer* FMLDeformerEditorModel::GetActiveHeatMapDeformer() const
	{
		return HeatMapDeformerGraph;
	}

	void FMLDeformerEditorModel::SetHeatMapMaterialEnabled(bool bEnabled)
	{
		FMLDeformerEditorActor* EditorActor = FindEditorActor(ActorID_Test_MLDeformed);
		if (EditorActor == nullptr)
		{
			return;
		}

		USkeletalMeshComponent* Component = EditorActor->GetSkeletalMeshComponent();
		if (Component)
		{
			if (bEnabled)
			{
				for (int32 Index = 0; Index < Component->GetNumMaterials(); ++Index)
				{
					Component->SetMaterial(Index, HeatMapMaterial);
				}
			}
			else
			{
				Component->EmptyOverrideMaterials();
			}
		}

		UpdateDeformerGraph();
	}

	UMeshDeformer* FMLDeformerEditorModel::LoadDefaultDeformerGraph() const
	{
		const FString GraphAssetPath = Model->GetDefaultDeformerGraphAssetPath();
		if (!GraphAssetPath.IsEmpty())
		{
			UObject* Object = StaticLoadObject(UMeshDeformer::StaticClass(), nullptr, *GraphAssetPath);
			UMeshDeformer* DeformerGraph = Cast<UMeshDeformer>(Object);
			if (DeformerGraph == nullptr)
			{
				UE_LOGF(LogMLDeformer, Warning, "Failed to load default Mesh Deformer from: %ls", *GraphAssetPath);
			}
			else
			{
				UE_LOGF(LogMLDeformer, Verbose, "Loaded default Mesh Deformer from: %ls", *GraphAssetPath);
			}
			return DeformerGraph;
		}

		return nullptr;
	}

	void FMLDeformerEditorModel::SetDefaultDeformerGraphIfNeeded()
	{
		// Initialize the asset on the default plugin deformer graph.
		UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
		if (VizSettings && VizSettings->GetDeformerGraph() == nullptr)
		{
			UMeshDeformer* DefaultGraph = LoadDefaultDeformerGraph();
			VizSettings->SetDeformerGraph(DefaultGraph);
		}
	}

	FText FMLDeformerEditorModel::GetOverlayText() const
	{
		const FMLDeformerEditorActor* EditorActor = FindEditorActor(ActorID_Test_MLDeformed);
		const UMLDeformerComponent* DeformerComponent = EditorActor ? EditorActor->GetMLDeformerComponent() : nullptr;
		if (DeformerComponent)
		{
			const UMLDeformerModelInstance* ModelInstance = DeformerComponent->GetModelInstance();
			if (ModelInstance &&
				ModelInstance->GetSkeletalMeshComponent() && 
				ModelInstance->GetSkeletalMeshComponent()->GetSkeletalMeshAsset() &&
				!ModelInstance->IsCompatible() )
			{
				return FText::FromString( ModelInstance->GetCompatibilityErrorText() );
			}
		}
	 
		return FText::GetEmpty();
	}

	void FMLDeformerEditorModel::UpdateDeformerGraph()
	{
		UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();

		FMLDeformerEditorActor* EditorActor = FindEditorActor(ActorID_Test_MLDeformed);
		if (EditorActor && EditorActor->GetMLDeformerComponent())
		{
			USkeletalMeshComponent* SkelMeshComponent = EditorActor->GetSkeletalMeshComponent();
			if (SkelMeshComponent)
			{
				UMeshDeformer* MeshDeformer = IsTrained() ? VizSettings->GetDeformerGraph() : nullptr;	
				const bool bUseHeatMapDeformer = VizSettings->GetShowHeatMap();
				SkelMeshComponent->SetMeshDeformer(bUseHeatMapDeformer ? GetActiveHeatMapDeformer() : MeshDeformer);
			}
		}

		// Overwrite the comparison model's deformer graph, as that might require a different one.
		for (FMLDeformerEditorActor* CompareEditorActor : EditorActors)
		{
			if (CompareEditorActor->GetTypeID() != ActorID_Test_Compare)
			{
				continue;
			}

			if (CompareEditorActor->GetMLDeformerComponent())
			{
				const int32 InstanceIndex = CompareEditorActor->GetActorTypeInstanceIndex();
				const FMLDeformerCompareActor& CompareActor = VizSettings->GetCompareActors()[InstanceIndex];

				USkeletalMeshComponent* SkelMeshComponent = CompareEditorActor->GetSkeletalMeshComponent();
				if (SkelMeshComponent)
				{
					const UMLDeformerAsset* Asset = CompareActor.DeformerAsset.LoadSynchronous();
					if (Asset && Asset->GetModel())
					{
						UMeshDeformer* MeshDeformer = Asset->GetModel()->GetVizSettings()->GetDeformerGraph();	
						const bool bUseHeatMapDeformer = VizSettings->GetShowHeatMap();

						// NOTE: Even though we use a HeatMapDeformerGraph here, it is not entirely correct, as if we want to be fully correct
						// we should be using the active heat map deformer from an editor model that fits the compare model.
						// So right now we assume a linear skinned heatmap, but if the compare model uses dual quaternion skinning then it's heatmap would appear incorrect.
						SkelMeshComponent->SetMeshDeformer(bUseHeatMapDeformer ? HeatMapDeformerGraph.Get() : MeshDeformer);
					}
				}
			}
		}
	}

	void FMLDeformerEditorModel::OnPostTraining(ETrainingResult TrainingResult, bool bUsePartiallyTrainedWhenAborted)
	{
		FMLDeformerSampler* Sampler = GetSamplerForActiveAnim();
		if (Sampler)
		{
			Sampler->SetVertexDeltaSpace(EVertexDeltaSpace::PostSkinning);
		}
		SampleDeltas();
		Model->InitGPUData();
		UpdateMemoryUsage();
		RefreshMLDeformerComponents();
	}

	FMLDeformerEditorActor* FMLDeformerEditorModel::GetTimelineEditorActor() const
	{
		const UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
		if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TrainingData)
		{
			return FindEditorActor(ActorID_Train_GroundTruth);
		}
		else if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData)
		{
			return FindEditorActor(ActorID_Test_GroundTruth);
		}
		return nullptr;
	}

	bool FMLDeformerEditorModel::IsEditorReadyForTrainingBasicChecks()
	{
		bool bHasTrainingAnim = false;
		const int32 NumAnims = GetNumTrainingInputAnims();
		for (int32 AnimIndex = 0; AnimIndex < NumAnims; ++AnimIndex)
		{
			const FMLDeformerTrainingInputAnim* Anim = this->GetTrainingInputAnim(AnimIndex);
			if (Anim && Anim->IsEnabled())
			{
				if (Anim->IsValid())
				{
					bHasTrainingAnim = true;
					break;
				}
			}
		}

		// Make sure we have picked required assets.
		if (!Model->HasTrainingGroundTruth() ||
			!bHasTrainingAnim ||
			Model->GetSkeletalMesh() == nullptr ||
			GetNumTrainingFrames() == 0)
		{
			return false;
		}

		// Make sure we have inputs.
		if (GetEditorInputInfo()->IsEmpty())
		{
			return false;
		}

		return true;
	}

	void FMLDeformerEditorModel::UpdateMemoryUsage()
	{
		Model->InvalidateMemUsage();
	}

	UAnimSequence* FMLDeformerEditorModel::GetActiveTrainingInputAnimSequence() const
	{
		const int32 ActiveIndex = GetActiveTrainingInputAnimIndex();
		if (ActiveIndex != INDEX_NONE)
		{
			FMLDeformerTrainingInputAnim* Anim = GetTrainingInputAnim(ActiveIndex);
			if (Anim)
			{
				return Anim->GetAnimSequence();
			}
		}
		return nullptr;
	}

	void FMLDeformerEditorModel::TriggerInputAssetChanged(bool bRefreshVizSettings)
	{
		OnInputAssetsChanged();
		OnPostInputAssetChanged();
		UpdateMemoryUsage();
		
		if (Editor->GetModelDetailsView())
		{
			Editor->GetModelDetailsView()->ForceRefresh();
		}

		if (bRefreshVizSettings)
		{
			if (Editor->GetVizSettingsDetailsView())
			{
				Editor->GetVizSettingsDetailsView()->ForceRefresh();
			}
		}
	}

	int32 FMLDeformerEditorModel::GetNumCurvesOnSkeletalMesh(USkeletalMesh* SkelMesh) const
	{
		if (SkelMesh)
		{
			const USkeleton* Skeleton = SkelMesh->GetSkeleton();
			if (Skeleton)
			{
				return Skeleton->GetNumCurveMetaData();
			}
		}

		return 0;
	}

	FString FMLDeformerEditorModel::GetHeatMapMaterialPath() const
	{
		return FString(TEXT("/MLDeformerFramework/Materials/MLDeformerHeatMapMat.MLDeformerHeatMapMat"));
	}

	FString FMLDeformerEditorModel::GetHeatMapDeformerGraphPath() const
	{
		return FString(TEXT("/MLDeformerFramework/Deformers/DG_MLDeformerModel_HeatMap.DG_MLDeformerModel_HeatMap"));
	}

	FString FMLDeformerEditorModel::GetHeatMapDeformerGraphDualQuatPath() const
	{
		return FString();
	}

	void FMLDeformerEditorModel::RefreshInputWidget()
	{
		if (InputWidget.IsValid())
		{
			InputWidget->Refresh();
		}
	}

	TSharedPtr<SMLDeformerInputWidget> FMLDeformerEditorModel::CreateInputWidget()
	{
		return SNew(SMLDeformerInputWidget)
			.EditorModel(this);
	}

	const UAnimSequence* FMLDeformerEditorModel::GetAnimSequence() const
	{
		if (GetModel())
		{
			const UMLDeformerVizSettings* VizSettings = GetModel()->GetVizSettings();
			if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TrainingData)
			{
				return GetActiveTrainingInputAnimSequence();
			}
			else if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData)
			{
				return GetModel()->GetVizSettings()->GetTestAnimSequence();
			}
		}
		return nullptr;
	}

	void FMLDeformerEditorModel::UpdateRanges()
	{
		ViewRange = TRange<double>(0.0, 100.0);
		WorkingRange = TRange<double>(0.0, 100.0);
		PlaybackRange = TRange<double>(0.0, 100.0);
		const UAnimSequence* AnimSeq = GetAnimSequence();
		const double Duration = AnimSeq ? AnimSeq->GetPlayLength() : 0.0;
		ViewRange = TRange<double>(0.0, Duration);
		WorkingRange = ViewRange;
		PlaybackRange = WorkingRange;
		ClampCurrentTrainingFrameIndex();
	}

	void FMLDeformerEditorModel::HandleModelChanged()
	{
		UpdateRanges();
	}

	void FMLDeformerEditorModel::HandleVizModeChanged(EMLDeformerVizMode Mode)
	{
		UpdateRanges();

		// Update the preview scene so that the debug rendering draws the skeleton for the correct actor, depending on whether we are in training or testing mode.
		// On default we render the bones etc for the ML Deformed character.
		// Additionally trigger the input assets changed event, as that updates the animation play offsets etc. This is needed when we switch from say training back to testing mode.
		UpdatePreviewScene();
		TriggerInputAssetChanged();
	}

	void FMLDeformerEditorModel::UpdatePreviewScene()
	{
		FMLDeformerEditorActor* EditorActor = nullptr;
		UAnimationAsset* AnimAsset = nullptr;
		const EMLDeformerVizMode Mode = Model->GetVizSettings()->GetVisualizationMode();
		if (Mode == EMLDeformerVizMode::TrainingData)
		{
			EditorActor = FindEditorActor(ActorID_Train_Base);
			AnimAsset = this->GetActiveTrainingInputAnimSequence();
		}
		else if (Mode == EMLDeformerVizMode::TestData)
		{
			EditorActor = FindEditorActor(ActorID_Test_MLDeformed);
			AnimAsset = Model->GetVizSettings()->GetTestAnimSequence();
		}

		IPersonaToolkit* ToolkitPtr = GetEditor()->GetPersonaToolkitPointer();
		if (EditorActor && ToolkitPtr)
		{
			IPersonaPreviewScene& Scene = ToolkitPtr->GetPreviewScene().Get();
			Scene.SetActor(EditorActor->GetActor());
			Scene.SetSelectedActor(EditorActor->GetActor());
			UDebugSkelMeshComponent* SkelMeshComponent = EditorActor->GetSkeletalMeshComponent();
			if (SkelMeshComponent)
			{
				Scene.SetPreviewMeshComponent(SkelMeshComponent);
				Scene.SetPreviewMesh(SkelMeshComponent->GetSkeletalMeshAsset());
			}
			Scene.SetPreviewAnimationAsset(AnimAsset);
		}
	}

	/** Get the current view range */
	TRange<double> FMLDeformerEditorModel::GetViewRange() const
	{
		return ViewRange;
	}

	void FMLDeformerEditorModel::SetViewRange(TRange<double> InRange)
	{
		ViewRange = InRange;

		if (WorkingRange.HasLowerBound() && WorkingRange.HasUpperBound())
		{
			WorkingRange = TRange<double>::Hull(WorkingRange, ViewRange);
		}
		else
		{
			WorkingRange = ViewRange;
		}
	}

	TRange<double> FMLDeformerEditorModel::GetWorkingRange() const
	{
		return WorkingRange;
	}

	TRange<FFrameNumber> FMLDeformerEditorModel::GetPlaybackRange() const
	{
		const int32 Resolution = GetTickResolution();
		return TRange<FFrameNumber>(FFrameNumber(FMath::RoundToInt32(PlaybackRange.GetLowerBoundValue() * (double)Resolution)), FFrameNumber(FMath::RoundToInt32(PlaybackRange.GetUpperBoundValue() * (double)Resolution)));

	}

	int32 FMLDeformerEditorModel::GetTicksPerFrame() const
	{
		// Default to millisecond resolution
		const int32 TimelineScrubSnapValue = 1000;
		return TimelineScrubSnapValue;
	}

	FFrameNumber FMLDeformerEditorModel::GetTickResScrubPosition() const
	{
		const double Resolution = GetTickResolution();
		const UMLDeformerVizSettings* VizSettings = GetModel()->GetVizSettings();
		double TrainingFrameNumber = 0.0;
		if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TrainingData)
		{
			TrainingFrameNumber = GetTrainingFrameAtTime(CalcTimelinePosition());
		}
		else if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData)
		{
			TrainingFrameNumber = GetTestFrameAtTime(CalcTimelinePosition());
		}
		FFrameNumber FrameNumber = (int32)(TrainingFrameNumber * Resolution / (double)GetFrameRate());
		return FrameNumber;
	}

	float FMLDeformerEditorModel::GetScrubTime() const
	{
		return (float)ScrubPosition.AsDecimal() / (float)GetFrameRate();
	}

	void FMLDeformerEditorModel::SetScrubPosition(FFrameTime NewScrubPosition, bool bIsScrubbing)
	{
		const double FrameNumber = NewScrubPosition.AsDecimal() * (double)GetFrameRate() / (double)GetTickResolution();
		ScrubPosition.FrameNumber = (int32)FrameNumber;

		const UMLDeformerVizSettings* VizSettings = GetModel()->GetVizSettings();
		if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TrainingData)
		{
			SetTrainingFrame(ScrubPosition.FrameNumber.Value, bIsScrubbing);
		}
		else if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData)
		{
			SetTestFrame(ScrubPosition.FrameNumber.Value, bIsScrubbing);
		}
	}

	void FMLDeformerEditorModel::SetScrubPosition(FFrameNumber NewScrubPosition, bool bIsScrubbing)
	{
		const double Resolution = GetTickResolution();
		const double ClampedValue = FMath::Clamp((double)NewScrubPosition.Value, PlaybackRange.GetLowerBoundValue() * Resolution, PlaybackRange.GetUpperBoundValue() * Resolution);
		const int32 FrameValue = ClampedValue * (double)GetFrameRate() / Resolution;

		const UMLDeformerVizSettings* VizSettings = GetModel()->GetVizSettings();
		if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TrainingData)
		{
			SetTrainingFrame(FrameValue, bIsScrubbing);
		}
		else if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData)
		{
			SetTestFrame(FrameValue, bIsScrubbing);
		}
	}

	void FMLDeformerEditorModel::SetDisplayFrames(bool bInDisplayFrames)
	{
		bDisplayFrames = bInDisplayFrames;
	}

	bool FMLDeformerEditorModel::IsDisplayingFrames() const
	{
		return bDisplayFrames;
	}

	void FMLDeformerEditorModel::HandleViewRangeChanged(TRange<double> InRange)
	{
		SetViewRange(InRange);
	}

	void FMLDeformerEditorModel::HandleWorkingRangeChanged(TRange<double> InRange)
	{
		WorkingRange = InRange;
	}

	double FMLDeformerEditorModel::CalcTimelinePosition() const
	{
		const UMLDeformerVizSettings* VizSettings = GetModel()->GetVizSettings();
		if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TrainingData)
		{
			return CalcTrainingTimelinePosition();
		}
		else if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData)
		{
			return CalcTestTimelinePosition();
		}
		return 0.0;
	}

	double FMLDeformerEditorModel::GetFrameRate() const
	{
		const UAnimSequence* AnimSequence = GetAnimSequence();
		if (AnimSequence)
		{
			return AnimSequence->GetSamplingFrameRate().AsDecimal();
		}
		else
		{
			return 30.0;
		}
	}

	int32 FMLDeformerEditorModel::GetTickResolution() const
	{
		return FMath::RoundToInt32((double)GetTicksPerFrame() * GetFrameRate());
	}

	void FMLDeformerEditorModel::OnPostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
	{
		const FProperty* Property = PropertyChangedEvent.Property;
		if (Property == nullptr)
		{
			return;
		}

		OnPropertyChanged(PropertyChangedEvent);
	}

	FString FMLDeformerEditorModel::GetTrainedNetworkOnnxFile() const
	{
		const FString ClassName = Model->GetClass()->GetFName().ToString();

		// This returns something like:
		// "<intermediatedir>/YourModel/YourModel.onnx"
		return FString::Printf(TEXT("%s%s/%s.onnx"), *FPaths::ProjectIntermediateDir(), *ClassName, *ClassName);
	}

	ETrainingResult FMLDeformerEditorModel::Train()
	{ 
		UE_LOGF(LogMLDeformer, Warning, "Please override the FMLDeformerEditorModel::Train() method inside your derived editor model class.");
		return ETrainingResult::Success;
	}

	int32 FMLDeformerEditorModel::GetNumTrainingFrames() const
	{ 
		return NumTrainingFrames;
	}

	void FMLDeformerEditorModel::UpdateNumTrainingFrames()
	{
		NumTrainingFrames = 0;
		UE_LOGF(LogMLDeformer, Warning, "Please override the FMLDeformerEditorModel::UpdateNumTrainingFrames() inside your model.");
	}

	UMLDeformerComponent* FMLDeformerEditorModel::FindMLDeformerComponent(int32 ActorID) const
	{
		const FMLDeformerEditorActor* EditorActor = FindEditorActor(ActorID);
		if (EditorActor)
		{
			return EditorActor->GetMLDeformerComponent();
		}
		return nullptr;
	}

	void FMLDeformerEditorModel::AddCompareActor(int32 ArrayIndex)
	{
		// Renumber the instance index values of other editor compare actors.
		for (FMLDeformerEditorActor* EditorActor : EditorActors)
		{
			if (EditorActor->GetTypeID() == ActorID_Test_Compare)
			{
				if (EditorActor->GetActorTypeInstanceIndex() >= ArrayIndex)
				{
					EditorActor->SetActorTypeInstanceIndex(EditorActor->GetActorTypeInstanceIndex() + 1);
				}
			}
		}

		const FLinearColor MLDeformedWireColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.MLCompareMesh.WireframeColor");
		USkeletalMesh* SkelMesh = Model->GetSkeletalMesh();

		TArray<FMLDeformerCompareActor>& CompareActors = Model->GetVizSettings()->GetCompareActors();
		FMLDeformerCompareActor& CompareActor = CompareActors[ArrayIndex];

		// Create the ML deformed actor.
		FActorSpawnParameters SpawnParams;
		UWorld* World = GetWorld();
		SpawnParams.Name = MakeUniqueObjectName(World, AActor::StaticClass(), "Test Compare Actor");
		AActor* Actor = World->SpawnActor<AActor>(SpawnParams);
		Actor->SetFlags(RF_Transient);

		// Create the skeletal mesh component.
		UDebugSkelMeshComponent* SkelMeshComponent = NewObject<UDebugSkelMeshComponent>(Actor);
		UMLDeformerAsset* DeformerAsset = CompareActor.DeformerAsset.LoadSynchronous();
		if (!IsCompatibleDeformer(DeformerAsset))
		{
			SkelMesh = nullptr;
		}
		SkelMeshComponent->SetSkeletalMesh(SkelMesh);
		Actor->SetRootComponent(SkelMeshComponent);
		SkelMeshComponent->RegisterComponent();
		SkelMeshComponent->SetWireframeMeshOverlayColor(MLDeformedWireColor);
		SkelMeshComponent->SetVisibility(false);
		SkelMeshComponent->MarkRenderStateDirty();
		if (SkelMeshComponent->GetAnimInstance())
		{
			SkelMeshComponent->GetAnimInstance()->GetRequiredBones().SetUseRAWData(true);
		}

		// Create the ML Deformer component.
		UMLDeformerAsset* FinalDeformerAsset = IsCompatibleDeformer(DeformerAsset) ? DeformerAsset : nullptr;
		UMLDeformerComponent* MLDeformerComponent = NewObject<UMLDeformerComponent>(Actor);
		MLDeformerComponent->SetSuppressMeshDeformerLogWarnings(true);
		MLDeformerComponent->SetDeformerAsset(FinalDeformerAsset);
		MLDeformerComponent->RegisterComponent();
		MLDeformerComponent->SetupComponent(FinalDeformerAsset, SkelMeshComponent);

		// Create the editor actor.
		const FLinearColor LabelColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.MLCompareMesh.LabelColor");
		FMLDeformerEditorActor::FConstructSettings Settings;
		Settings.Actor = Actor;
		Settings.TypeID = ActorID_Test_Compare;
		Settings.LabelColor = LabelColor;
		Settings.LabelText = LOCTEXT("TestCompareActorLabelText", "ML Compare");
		Settings.bIsTrainingActor = false;
		Settings.ActorTypeInstanceIndex = ArrayIndex;
		FMLDeformerEditorActor* EditorActor = CreateEditorActor(Settings);
		EditorActor->SetSkeletalMeshComponent(SkelMeshComponent);
		EditorActor->SetMLDeformerComponent(MLDeformerComponent);
		EditorActor->SetMeshOffsetFactor(2.0f + static_cast<float>(ArrayIndex));
		EditorActors.Add(EditorActor);
	}

	void FMLDeformerEditorModel::RemoveCompareActor(int32 ArrayIndex)
	{
		UWorld* World = GetWorld();

		FMLDeformerEditorActor* EditorActorToDestroy = nullptr;
		AActor* ActorToDestroy = nullptr;
		for (FMLDeformerEditorActor* EditorActor : EditorActors)
		{
			if (EditorActor && EditorActor->GetTypeID() == ActorID_Test_Compare && EditorActor->GetActorTypeInstanceIndex() == ArrayIndex)
			{
				EditorActorToDestroy = EditorActor;
				ActorToDestroy = EditorActor->GetActor();
				World->DestroyActor(EditorActor->GetActor(), true);
				EditorActors.Remove(EditorActor);
				delete EditorActor;
				break;
			}
		}

		// Renumber other editor actors that come after.
		for (FMLDeformerEditorActor* EditorActor : EditorActors)
		{
			if (EditorActor && EditorActor->GetTypeID() == ActorID_Test_Compare && EditorActor->GetActorTypeInstanceIndex() > ArrayIndex)
			{
				EditorActor->SetActorTypeInstanceIndex(EditorActor->GetActorTypeInstanceIndex() - 1);
			}
		}

		if (ActorToDestroy)
		{
			ActorToDestroy->Destroy();
		}
	}

	void FMLDeformerEditorModel::RemoveAllCompareActors()
	{
		UWorld* World = GetWorld();

		TArray<AActor*> ActorsToDestroy;
		TArray<FMLDeformerEditorActor*> EditorActorsToDestroy;
		for (FMLDeformerEditorActor* EditorActor : EditorActors)
		{
			if (EditorActor && EditorActor->GetTypeID() == ActorID_Test_Compare)
			{				
				ActorsToDestroy.Add(EditorActor->GetActor());
				World->DestroyActor(EditorActor->GetActor(), true);
				EditorActorsToDestroy.Add(EditorActor);
			}
		}

		// Now delete the editor actors, which removes components.
		for (FMLDeformerEditorActor* EditorActor : EditorActorsToDestroy)
		{
			EditorActors.Remove(EditorActor);
			delete EditorActor;
		}

		for (AActor* Actor : ActorsToDestroy)
		{
			if (Actor)
			{
				Actor->Destroy();
			}
		}
	}

	void FMLDeformerEditorModel::UpdateCompareActorLabels()
	{
		for (FMLDeformerEditorActor* EditorActor : EditorActors)
		{
			if (EditorActor && EditorActor->GetTypeID() == ActorID_Test_Compare)
			{
				UTextRenderComponent* LabelComponent = EditorActor->GetLabelComponent();
				if (LabelComponent)
				{
					const int32 InstanceIndex = EditorActor->GetActorTypeInstanceIndex();
					const FMLDeformerCompareActor& CompareActor = Model->GetVizSettings()->GetCompareActors()[InstanceIndex];
					FText LabelText;				
					if (!CompareActor.Name.IsNone() && CompareActor.Name.IsValid())
					{
						LabelText = FText::FromName(CompareActor.Name);
					}
					else
					{
						UMLDeformerAsset* DeformerAsset = CompareActor.DeformerAsset.LoadSynchronous();
						LabelText = DeformerAsset ? FText::FromString(DeformerAsset->GetName()) : FText::Format(LOCTEXT("DefaultCompareActorDeformerName", "ML Compare #{0}"), InstanceIndex);
					}
					const FLinearColor LabelColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.MLCompareMesh.LabelColor");
					LabelComponent->SetText(LabelText);
					LabelComponent->SetTextRenderColor(LabelColor.ToFColor(true));
				}
			}
		}
	}

	int32 FMLDeformerEditorModel::GetActiveTrainingInputAnimIndex() const
	{
		return ActiveTrainingInputAnimIndex;
	}

	TSharedPtr<FMLDeformerSampler> FMLDeformerEditorModel::CreateSamplerObject() const
	{
		return MakeShared<FMLDeformerSampler>();
	}

	int32 FMLDeformerEditorModel::GetNumTrainingInputAnims() const
	{
		UE_LOGF(LogMLDeformer, Warning, "Please override the FMLDeformerEditorModel::GetNumTrainingInputAnims() method inside your derived editor model class.");
		return 0;
	}

	FMLDeformerTrainingInputAnim* FMLDeformerEditorModel::GetTrainingInputAnim(int32 Index) const
	{
		UE_LOGF(LogMLDeformer, Warning, "Please override the FMLDeformerEditorModel::GetTrainingInputAnim(int32) method inside your derived editor model class.");
		return nullptr;
	}

	void FMLDeformerEditorModel::UpdateTimelineTrainingAnimList()
	{
		TArray<TSharedPtr<FMLDeformerTrainingInputAnimName>> NameList;

		// Build the list of names, based on the training inputs.
		const int32 NumAnims = GetNumTrainingInputAnims();
		for (int32 AnimIndex = 0; AnimIndex < NumAnims; ++AnimIndex)
		{
			const FMLDeformerTrainingInputAnim* Anim = GetTrainingInputAnim(AnimIndex);
			if (Anim)
			{
				if (Anim->IsValid())
				{
					TSharedPtr<FMLDeformerTrainingInputAnimName> AnimName = MakeShared<FMLDeformerTrainingInputAnimName>();
					AnimName->TrainingInputAnimIndex = AnimIndex;
					AnimName->Name = FString::Printf(TEXT("[#%d] %s"), AnimIndex, *Anim->GetAnimSequence()->GetName());
					NameList.Emplace(AnimName);
				}
			}
		}

		SetTimelineAnimNames(NameList);
	}

	void FMLDeformerEditorModel::InvalidateDeltas()
	{
		// Set the current training frame to -1, which will cause the deltas to get updated next frame.
		CurrentTrainingFrame = -1;
	}

	void FMLDeformerEditorModel::UpdateActorLODs()
	{
		FMLDeformerEditorActor* MLDeformedActor = FindEditorActor(ActorID_Test_MLDeformed);
		if (MLDeformedActor)
		{		
			USkeletalMeshComponent* BaseSkelMeshComponent = MLDeformedActor->GetSkeletalMeshComponent();		
			if (BaseSkelMeshComponent)
			{
				const int32 LOD = BaseSkelMeshComponent->GetPredictedLODLevel();
				for (FMLDeformerEditorActor* EditorActor : EditorActors)
				{
					if (EditorActor->GetTypeID() == ActorID_Test_Compare || EditorActor->GetTypeID() == ActorID_Test_Base)
					{
						if (EditorActor->GetSkeletalMeshComponent())
						{
							EditorActor->GetSkeletalMeshComponent()->SetPredictedLODLevel(LOD);
						}
					}
				}
			}
		}
	}

	void FMLDeformerEditorModel::ApplyDebugActorTransforms()
	{
		AActor* DebugActor = GetEditor()->GetDebugActor();
		if (DebugActor && IsValid(DebugActor))
		{
			const TArray<FTransform> DebugActorComponentSpaceTransforms = GetEditor()->GetDebugActorComponentSpaceTransforms();
			if (!DebugActorComponentSpaceTransforms.IsEmpty())
			{
				ApplyDebugActorTransforms(DebugActorComponentSpaceTransforms);
			}
		}
	}

	void FMLDeformerEditorModel::OnDebugActorChanged(TObjectPtr<AActor> DebugActor)
	{
		// Mark the test ML Deformed character as having debugging enabled.
		FMLDeformerEditorActor* TestMLDeformedActor = FindEditorActor(ActorID_Test_MLDeformed);
		if (TestMLDeformedActor)
		{
			UMLDeformerComponent* MLDComponent = TestMLDeformedActor->GetMLDeformerComponent();
			if (MLDComponent)
			{
				MLDComponent->SetDebugActor(DebugActor);
			}
		}

		// Update visibility, mesh offsets and transforms.
		// This is needed as when we enable or disable debugging we can show/hide compare actors, ground truth actors, etc.
		UpdateActorVisibility();
		UpdateMeshOffsetFactors();
		UpdateActorTransforms();

		// Trigger a redraw of the MLD asset editor viewport.
		if (GetEditor()->GetViewport().IsValid())
		{
			GetEditor()->GetViewport()->GetViewportClient().bNeedsRedraw = true;
		}
	}

	void FMLDeformerEditorModel::ApplyDebugActorTransforms(const TArray<FTransform>& DebugActorComponentSpaceTransforms)
	{
		for (FMLDeformerEditorActor* EditorActor : EditorActors)
		{
			// Only overwrite the transforms for the test ML Deformed character and the linear skinned character, leave the rest as-is.
			// It doesn't make sense to overwrite transforms for the training actors for example.
			// We also shouldn't apply to the compare actors, as the morph weights aren't copied over, as they might have a different number of morphs etc.
			if (EditorActor == nullptr || 
				(EditorActor->GetTypeID() != ActorID_Test_MLDeformed && EditorActor->GetTypeID() != ActorID_Test_Base))
			{
				continue;
			}

			// Copy the transforms to all editor actors that have a skeletal mesh component with valid skeletal mesh setup.
			UDebugSkelMeshComponent* SkelMeshComponent = EditorActor->GetSkeletalMeshComponent();
			if (SkelMeshComponent && SkelMeshComponent->GetSkeletalMeshAsset())
			{
				TArray<FTransform>& EditableTransforms = SkelMeshComponent->GetEditableComponentSpaceTransforms();
				check(EditableTransforms.Num() == DebugActorComponentSpaceTransforms.Num());
				EditableTransforms = DebugActorComponentSpaceTransforms;
				SkelMeshComponent->ApplyEditedComponentSpaceTransforms();
			}
		}
	}

	void FMLDeformerEditorModel::UpdateStepInterpolationMode()
	{
		// Force step interpolation when viewing the heatmap in ground truth mode.
		UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
		FMLDeformerEditorActor* MLDeformerEditorActor = static_cast<FMLDeformerEditorActor*>(FindEditorActor(ActorID_Test_MLDeformed));
		if (MLDeformerEditorActor)
		{
			USkeletalMeshComponent* SkelMeshComponent = MLDeformerEditorActor->GetSkeletalMeshComponent();
			if (SkelMeshComponent)
			{
				UAnimSingleNodeInstance* AnimInstance = Cast<UAnimSingleNodeInstance>(SkelMeshComponent->GetAnimInstance());
				if (AnimInstance)
				{
					const bool bForceStepInterpolation = VizSettings->GetShowHeatMap() && (VizSettings->GetHeatMapMode() == EMLDeformerHeatMapMode::GroundTruth) && VizSettings->HasTestGroundTruth();
					const TOptional<EAnimInterpolationType> InterpolationOverride = bForceStepInterpolation ? EAnimInterpolationType::Step : TOptional<EAnimInterpolationType>();
					AnimInstance->SetInterpolationOverride(InterpolationOverride);				
				}
			}
		}
	}

	TVertexAttributesConstRef<float> FMLDeformerEditorModel::FindVertexAttributes(FName AttributeName) const
	{
		const USkeletalMesh* SkelMesh = Model->GetSkeletalMesh();
		if (SkelMesh)
		{
			const int32 LodLevel = 0;
			const FMeshDescription* MeshDescription = SkelMesh->GetMeshDescription(LodLevel);
			if (MeshDescription)
			{
				return MeshDescription->VertexAttributes().GetAttributesRef<float>(AttributeName);
			}
		}
		return TVertexAttributesConstRef<float>();
	}

	void FMLDeformerEditorModel::BroadcastPropertyChanged(const FName PropertyName)
	{
		FProperty* Property = Model->GetClass()->FindPropertyByName(PropertyName);
		if (Property == nullptr)
		{
			UE_LOGF(LogMLDeformer, Error, "Failed to find property '%ls' in class '%ls'", *PropertyName.ToString(), *Model->GetName());
			return;
		}

		FPropertyChangedEvent Event(Property, EPropertyChangeType::ValueSet);
		Model->PostEditChangeProperty(Event);
	}
}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
