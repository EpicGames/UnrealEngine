// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerEditorModule.h"
#include "MLDeformerEditorMode.h"
#include "MLDeformerModule.h"
#include "MLDeformerEditorModel.h"
#include "MLDeformerModel.h"
#include "MLDeformerCurveReferenceCustomization.h"
#include "MLDeformerGeomCacheTrainingInputAnimCustomize.h"
#include "SMLDeformerInputWidget.h"
#include "Modules/ModuleManager.h"
#include "EditorModeRegistry.h"
#include "PropertyEditorDelegates.h"
#include "Tools/TrainingDataProcessor/TrainingDataProcessorTool.h"
#include "Tools/TrainingDataProcessor/AnimCustomization.h"
#include "Tools/TrainingDataProcessor/BoneListCustomization.h"
#include "Tools/TrainingDataProcessor/BoneGroupsListCustomization.h"
#include "Tools/TrainingDataProcessor/SBoneListWidget.h"
#include "Tools/TrainingDataProcessor/SBoneGroupsListWidget.h"
#include "Tools/TrainingDataProcessor/TrainingDataProcessorSettingsDetailCustomization.h"

#define LOCTEXT_NAMESPACE "MLDeformerEditorModule"

IMPLEMENT_MODULE(UE::MLDeformer::FMLDeformerEditorModule, MLDeformerFrameworkEditor)

namespace UE::MLDeformer
{
	void FMLDeformerEditorModule::StartupModule()
	{
		FEditorModeRegistry::Get().RegisterMode<FMLDeformerEditorMode>(FMLDeformerEditorMode::ModeName, LOCTEXT("MLDeformerEditorMode", "MLDeformer"), FSlateIcon(), false);

		// Register detail customizations.
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomPropertyTypeLayout("MLDeformerCurveReference", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMLDeformerCurveReferenceCustomization::MakeInstance) );
		PropertyModule.RegisterCustomPropertyTypeLayout("MLDeformerGeomCacheTrainingInputAnim", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMLDeformerGeomCacheTrainingInputAnimCustomization::MakeInstance) );
		PropertyModule.RegisterCustomPropertyTypeLayout("MLDeformerTrainingDataProcessorAnim", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&TrainingDataProcessor::FAnimCustomization::MakeInstance) );
		PropertyModule.RegisterCustomPropertyTypeLayout("MLDeformerTrainingDataProcessorBoneList", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&TrainingDataProcessor::FBoneListCustomization::MakeInstance) );
		PropertyModule.RegisterCustomPropertyTypeLayout("MLDeformerTrainingDataProcessorBoneGroupsList", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&TrainingDataProcessor::FBoneGroupsListCustomization::MakeInstance) );
		PropertyModule.RegisterCustomClassLayout("MLDeformerTrainingDataProcessorSettings", FOnGetDetailCustomizationInstance::CreateStatic(&TrainingDataProcessor::FTrainingDataProcessorSettingsDetailCustomization::MakeInstance));
		PropertyModule.NotifyCustomizationModuleChanged();

		TrainingDataProcessor::RegisterTool();

		SMLDeformerInputWidget::RegisterCommands();
		TrainingDataProcessor::FBoneListWidgetCommands::Register();
		TrainingDataProcessor::FBoneGroupsListWidgetCommands::Register();
	}

	void FMLDeformerEditorModule::ShutdownModule()
	{
		FEditorModeRegistry::Get().UnregisterMode(FMLDeformerEditorMode::ModeName);
		
		// Unregister detail customizations.
		if (FModuleManager::Get().IsModuleLoaded(TEXT("PropertyEditor")))
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
			PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("MLDeformerCurveReference"));
			PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("MLDeformerGeomCacheTrainingInputAnim"));
			PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("MLDeformerTrainingDataProcessorAnim"));
			PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("MLDeformerTrainingDataProcessorBoneList"));
			PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("MLDeformerTrainingDataProcessorBoneGroupsList"));
			PropertyModule.UnregisterCustomClassLayout(TEXT("MLDeformerTrainingDataProcessorSettings"));
			PropertyModule.NotifyCustomizationModuleChanged();
		}

		SMLDeformerInputWidget::UnregisterCommands();
		TrainingDataProcessor::FBoneListWidgetCommands::Unregister();
		TrainingDataProcessor::FBoneGroupsListWidgetCommands::Unregister();
	}
}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
