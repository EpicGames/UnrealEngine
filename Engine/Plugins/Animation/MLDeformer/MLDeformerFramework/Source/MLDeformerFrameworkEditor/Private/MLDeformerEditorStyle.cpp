// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerEditorStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateTypes.h"
#include "Math/Color.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"

namespace UE::MLDeformer
{
	#define MLDEFORMER_IMAGE_BRUSH(RelativePath, ...) FSlateImageBrush(RootToContentDir(RelativePath, TEXT(".png")), __VA_ARGS__)

	FMLDeformerEditorStyle::FMLDeformerEditorStyle() : FSlateStyleSet("MLDeformerEditorStyle")
	{
		const FString ResourceDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("MLDeformerFramework"))->GetBaseDir(), TEXT("Resources"));
		SetContentRoot(ResourceDir);

		const FVector2D Icon16x16(16.0f, 16.0f);
		const FVector2D Icon20x20(20.0f, 20.0f);
		const FVector2D Icon40x40(40.0f, 40.0f);

		// Icons.
		Set("MLDeformer.Timeline.PlayIcon", new MLDEFORMER_IMAGE_BRUSH("Icons/Play_40x", Icon20x20));
		Set("MLDeformer.Timeline.PauseIcon", new MLDEFORMER_IMAGE_BRUSH("Icons/Pause_40x", Icon20x20));
		Set("MLDeformer.Timeline.TabIcon", new MLDEFORMER_IMAGE_BRUSH("Icons/TimelineTab_40x", Icon16x16));
		Set("MLDeformer.VizSettings.TabIcon", new MLDEFORMER_IMAGE_BRUSH("Icons/VizSettingsTab_40x", Icon40x40));
		Set("MLDeformer.Debug.RefreshIcon", new MLDEFORMER_IMAGE_BRUSH("Icons/Refresh_40x", Icon40x40));

		// Colors and sizes.
		Set("MLDeformer.BaseMesh.WireframeColor", FLinearColor(0.0f, 1.0f, 0.0f));
		Set("MLDeformer.BaseMesh.LabelColor", FLinearColor(0.0f, 1.0f, 0.0f));

		Set("MLDeformer.TargetMesh.WireframeColor", FLinearColor(0.0f, 0.0f, 1.0f));
		Set("MLDeformer.TargetMesh.LabelColor", FLinearColor(0.0f, 0.0f, 1.0f));

		Set("MLDeformer.MLDeformedMesh.WireframeColor", FLinearColor(1.0f, 0.0f, 0.0f));
		Set("MLDeformer.MLDeformedMesh.LabelColor", FLinearColor(1.0f, 0.0f, 0.0f));

		Set("MLDeformer.MLCompareMesh.WireframeColor", FLinearColor(1.0f, 0.4f, 0.0f));
		Set("MLDeformer.MLCompareMesh.LabelColor", FLinearColor(1.0f, 0.4f, 0.0f));

		Set("MLDeformer.GroundTruth.WireframeColor", FLinearColor(0.0f, 0.0f, 1.0f));
		Set("MLDeformer.GroundTruth.LabelColor", FLinearColor(0.0f, 0.0f, 1.0f));

		Set("MLDeformer.Deltas.Color", FLinearColor(0.0f, 1.0f, 0.0f));
		Set("MLDeformer.Vertex.Color", FLinearColor(1.0f, 0.0f, 1.0f));
		Set("MLDeformer.Vertex.Size", 5.0f);

		Set("MLDeformer.DebugVectors.Color", FLinearColor(1.0f, 0.0f, 0.0f));
		Set("MLDeformer.DebugVectors.Color2", FLinearColor(1.0f, 1.0f, 0.0f));

		Set("MLDeformer.Morphs.IncludedVertexColor", FLinearColor(1.0f, 0.0f, 1.0f));
		Set("MLDeformer.Morphs.ExcludedVertexColor", FLinearColor(0.1f, 0.1f, 0.1f));

		Set("MLDeformer.Morphs.MaskIncludedVertexColor", FLinearColor(0.0f, 1.0f, 1.0f));
		Set("MLDeformer.Morphs.MaskIncludedVertexColorPainted", FLinearColor(0.0f, 0.5f, 1.0f));
		Set("MLDeformer.Morphs.MaskExcludedVertexColor", FLinearColor(0.1f, 0.1f, 0.1f));

		Set("MLDeformer.Statistics.Performance", FLinearColor(0.0f, 1.0f, 0.0f));
		Set("MLDeformer.DefaultLabelScale", 0.65f);

		Set("MLDeformer.InputsWidget.HighlightColor", FLinearColor(0.0f, 1.0f, 0.0f));
		Set("MLDeformer.InputsWidget.ErrorColor", FLinearColor(1.0f, 0.5f, 0.0f));

		Set("MLDeformer.Debug.ActiveDebugColor", FLinearColor(0.0f, 1.0f, 0.0f));
		Set("MLDeformer.Debug.EditorSelectedMarkColor", FLinearColor(1.0f, 1.0f, 0.0f));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	FMLDeformerEditorStyle::~FMLDeformerEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	FMLDeformerEditorStyle& FMLDeformerEditorStyle::Get()
	{
		static FMLDeformerEditorStyle Inst;
		return Inst;
	}
}	// namespace UE::MLDeformer

#undef MLDEFORMER_IMAGE_BRUSH
