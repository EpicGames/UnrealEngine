// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerEditorToolkit.h"
#include "IDetailsView.h"
#include "MLDeformerModule.h"
#include "MLDeformerAsset.h"
#include "MLDeformerModel.h"
#include "MLDeformerApplicationMode.h"
#include "MLDeformerEditorMode.h"
#include "MLDeformerPaintMode.h"
#include "MLDeformerPaintModeToolkit.h"
#include "MLDeformerEditorModule.h"
#include "MLDeformerEditorModel.h"
#include "MLDeformerVizSettingsTabSummoner.h"
#include "MLDeformerTimelineTabSummoner.h"
#include "MLDeformerInputInfo.h"
#include "MLDeformerEditorStyle.h"
#include "MLDeformerVizSettings.h"
#include "MLDeformerSampler.h"
#include "SMLDeformerTimeline.h"
#include "SMLDeformerDebugSelectionWidget.h"
#include "AnimationEditorViewportClient.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "Modules/ModuleManager.h"
#include "PersonaModule.h"
#include "PersonaTabs.h"
#include "IPersonaToolkit.h"
#include "IAssetFamily.h"
#include "IPersonaViewport.h"
#include "Preferences/PersonaOptions.h"
#include "UObject/Object.h"
#include "SSimpleTimeSlider.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Misc/MessageDialog.h"

#define LOCTEXT_NAMESPACE "MLDeformerEditorToolkit"

namespace UE::MLDeformer
{
	const FName MLDeformerEditorModes::Editor("MLDeformerEditorMode");
	const FName MLDeformerEditorAppName = FName(TEXT("MLDeformerEditorApp"));

	TArray<TUniquePtr<FToolsMenuExtender>> FMLDeformerEditorToolkit::ToolsMenuExtenders;
	FCriticalSection FMLDeformerEditorToolkit::ExtendersMutex;

	FMLDeformerEditorToolkit::~FMLDeformerEditorToolkit()
	{
		GEditor->UnregisterForUndo(this);
		ActiveModel.Reset();
	}

	void FMLDeformerEditorToolkit::InitAssetEditor(
		const EToolkitMode::Type Mode,
		const TSharedPtr<IToolkitHost>& InitToolkitHost,
		UMLDeformerAsset* InDeformerAsset)
	{
		DeformerAsset = InDeformerAsset;
		bIsInitialized = false;

		FPersonaToolkitArgs PersonaToolkitArgs;
		PersonaToolkitArgs.OnPreviewSceneCreated = FOnPreviewSceneCreated::FDelegate::CreateSP(this, &FMLDeformerEditorToolkit::HandlePreviewSceneCreated);

		FMLDeformerEditorModule& EditorModule = FModuleManager::LoadModuleChecked<FMLDeformerEditorModule>("MLDeformerFrameworkEditor");
		FMLDeformerEditorModelRegistry& ModelRegistry = EditorModule.GetModelRegistry();	

		// Create a new model if none was selected yet.
		UMLDeformerModel* DeformerModel = DeformerAsset->GetModel();
		if (DeformerModel)
		{
			DeformerModel->Init(DeformerAsset);
			FMLDeformerEditorModel* EditorModel = ModelRegistry.CreateEditorModel(DeformerModel);
			FMLDeformerEditorModel::InitSettings InitSettings;
			InitSettings.Editor = this;
			InitSettings.Model = DeformerModel;
			if (EditorModel)
			{
				EditorModel->Init(InitSettings);
			}
			ActiveModel = TSharedPtr<FMLDeformerEditorModel>(EditorModel);
		}

		FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
		PersonaToolkit = PersonaModule.CreatePersonaToolkit(DeformerAsset, PersonaToolkitArgs);

		const bool bCreateDefaultStandaloneMenu = true;
		const bool bCreateDefaultToolbar = true;
		FAssetEditorToolkit::InitAssetEditor(
			Mode,
			InitToolkitHost,
			MLDeformerEditorAppName,
			FTabManager::FLayout::NullLayout,
			bCreateDefaultStandaloneMenu,
			bCreateDefaultToolbar,
			DeformerAsset);

		// Create and set the application mode.
		ApplicationMode = new FMLDeformerApplicationMode(SharedThis(this), PersonaToolkit->GetPreviewScene());
		AddApplicationMode(MLDeformerEditorModes::Editor, MakeShareable(ApplicationMode));
		SetCurrentMode(MLDeformerEditorModes::Editor);

		// Activate the editor mode.
		GetEditorModeManager().SetDefaultMode(FMLDeformerEditorMode::ModeName);
		GetEditorModeManager().ActivateMode(FMLDeformerEditorMode::ModeName);

		FMLDeformerEditorMode* EditorMode = static_cast<FMLDeformerEditorMode*>(GetEditorModeManager().GetActiveMode(FMLDeformerEditorMode::ModeName));
		if (EditorMode)
		{
			EditorMode->SetEditorToolkit(this);
		}

		SAssignNew(DebugWidget, SMLDeformerDebugSelectionWidget)
			.MLDeformerEditor(this);

		ExtendToolbar();
		RegenerateMenusAndToolbars();

		OnSwitchedVisualizationMode();
		if (ActiveModel)
		{
			ActiveModel->UpdateIsReadyForTrainingState();
			ActiveModel->SetTrainingFrame(ActiveModel->GetModel()->GetVizSettings()->GetTrainingFrameNumber(), false);
			ActiveModel->SetTestFrame(ActiveModel->GetModel()->GetVizSettings()->GetTestingFrameNumber(), false);
			ActiveModel->InvalidateDeltas();
			ActiveModel->GetModel()->GetVizSettings()->SetFlags(RF_Transactional);
		}

		if (DeformerModel)
		{
			GetModelDetailsView()->SetObject(DeformerModel);
			GetVizSettingsDetailsView()->SetObject(DeformerModel->GetVizSettings());
		}

		if (DebugWidget.IsValid())
		{
			DebugWidget->Refresh();
		}

		GetModelDetailsView()->ForceRefresh();
		GetVizSettingsDetailsView()->ForceRefresh();

		ShowNoModelsWarningIfNeeded();

		GEditor->RegisterForUndo(this);

		bIsInitialized = true;
	}

	void FMLDeformerEditorToolkit::ShowNoModelsWarningIfNeeded()
	{
		FMLDeformerEditorModule& EditorModule = FModuleManager::LoadModuleChecked<FMLDeformerEditorModule>("MLDeformerFrameworkEditor");
		const FMLDeformerEditorModelRegistry& ModelRegistry = EditorModule.GetModelRegistry();	
		if (ModelRegistry.GetNumRegisteredModels() == 0)
		{
			const EAppReturnType::Type ReturnType = FMessageDialog::Open(
				EAppMsgType::Ok, 
				FText(LOCTEXT("NoModelsFoundWarningMessage", "No ML Deformer models have been registered.\nPlease load a model plugin.\n\nThere is nothing to do inside this editor, until you load a model plugin.")),
				LOCTEXT("NoModelsFoundWarningTitle", "No ML Deformer Models Found"));
		}
	}

	TSharedRef<IPersonaToolkit> FMLDeformerEditorToolkit::GetPersonaToolkit() const 
	{
		return PersonaToolkit.ToSharedRef();
	}

	void FMLDeformerEditorToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_MLDeformerEditor", "ML Deformer Editor"));
		auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

		FAssetEditorToolkit::RegisterTabSpawners(InTabManager);
	}

	void FMLDeformerEditorToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
	{
		FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
	}

	void FMLDeformerEditorToolkit::ExtendToolbar()
	{
		TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

		AddToolbarExtender(ToolbarExtender);

		ToolbarExtender->AddToolBarExtension(
			"Asset",
			EExtensionHook::After,
			GetToolkitCommands(),
			FToolBarExtensionDelegate::CreateSP(this, &FMLDeformerEditorToolkit::FillToolbar)
		);
	}

	TSharedRef<SWidget> FMLDeformerEditorToolkit::GenerateVizModeButtonContents(TSharedRef<FUICommandList> InCommandList)
	{
		const bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InCommandList);

		FUIAction TrainItemAction(FExecuteAction::CreateSP(this, &FMLDeformerEditorToolkit::SwitchVizMode, EMLDeformerVizMode::TrainingData));
		MenuBuilder.AddMenuEntry(GetVizModeName(EMLDeformerVizMode::TrainingData), TAttribute<FText>(), FSlateIcon(), TrainItemAction);

		FUIAction TestItemAction(FExecuteAction::CreateSP(this, &FMLDeformerEditorToolkit::SwitchVizMode, EMLDeformerVizMode::TestData));
		MenuBuilder.AddMenuEntry(GetVizModeName(EMLDeformerVizMode::TestData), TAttribute<FText>(), FSlateIcon(), TestItemAction);

		return MenuBuilder.MakeWidget();
	}

	TSharedRef<SWidget> FMLDeformerEditorToolkit::GenerateModelButtonContents(TSharedRef<FUICommandList> InCommandList)
	{
		const bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InCommandList);

		FMLDeformerEditorModule& EditorModule = FModuleManager::LoadModuleChecked<FMLDeformerEditorModule>("MLDeformerFrameworkEditor");
		const FMLDeformerEditorModelRegistry& ModelRegistry = EditorModule.GetModelRegistry();

		TArray<UClass*> ModelTypes;
		ModelRegistry.GetRegisteredModels().GenerateKeyArray(ModelTypes);

		// Generate a list of model names and sort them.
		TArray<FString> ModelNames;
		ModelNames.Reserve(ModelTypes.Num());
		for (int32 ModelIndex = 0; ModelIndex < ModelTypes.Num(); ++ModelIndex)
		{
			const UClass* Model = ModelTypes[ModelIndex];
			const UMLDeformerModel* DefaultModel = Cast<UMLDeformerModel>(Model->GetDefaultObject());
			check(DefaultModel);
			ModelNames.Add(DefaultModel->GetDisplayName());
		}
		ModelNames.Sort();

		// Iterate over all registered models and add them to the list.
		for (int32 ModelIndex = 0; ModelIndex < ModelNames.Num(); ++ModelIndex)
		{
			// Find the model index as in the registry, so before sorting.
			int32 UnsortedModelIndex = -1;
			for (int32 Index = 0; Index < ModelTypes.Num(); ++Index)
			{
				const UClass* Model = ModelTypes[Index];
				const UMLDeformerModel* DefaultModel = Cast<UMLDeformerModel>(Model->GetDefaultObject());
				if (DefaultModel->GetDisplayName() == ModelNames[ModelIndex])
				{
					UnsortedModelIndex = Index;
					break;
				}
			}

			FUIAction ItemAction(FExecuteAction::CreateSP(this, &FMLDeformerEditorToolkit::OnModelChanged, UnsortedModelIndex, false));
			MenuBuilder.AddMenuEntry(FText::FromString(ModelNames[ModelIndex]), TAttribute<FText>(), FSlateIcon(), ItemAction);
		}

		return MenuBuilder.MakeWidget();
	}

	TSharedRef<SWidget> FMLDeformerEditorToolkit::GenerateToolsMenuContents(TSharedRef<FUICommandList> InCommandList)
	{
		const bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, InCommandList);

		// Sort the tools menu items before we add them.
		ToolsMenuExtenders.Sort([this](const TUniquePtr<FToolsMenuExtender>& A, const TUniquePtr<FToolsMenuExtender>& B)
			{
				return A->GetMenuEntry(*this).LabelOverride.Get().ToString() < B->GetMenuEntry(*this).LabelOverride.Get().ToString();
			} );

		for (const TUniquePtr<FToolsMenuExtender>& Extender : ToolsMenuExtenders)
		{
			MenuBuilder.AddMenuEntry(Extender->GetMenuEntry(*this));
		}

		return MenuBuilder.MakeWidget();
	}

	void FMLDeformerEditorToolkit::SwitchVizMode(EMLDeformerVizMode Mode)
	{
		if (!ActiveModel.IsValid())
		{
			return;
		}

		ActiveModel->GetModel()->GetVizSettings()->SetVisualizationMode(Mode);
		OnSwitchedVisualizationMode();
	}

	void FMLDeformerEditorToolkit::OnModelChanged(int Index, bool bForceChange)
	{
		if (ActiveModel && !bForceChange)
		{
			const EAppReturnType::Type ReturnType = FMessageDialog::Open(
				EAppMsgType::YesNo, 
				LOCTEXT("SwitchModelConfirmMessage", "Are you sure you want to switch the current model?\nYou will lose your current setup.\n"),
				LOCTEXT("SwitchModelConfirmTitle", "Switch current model?"));

			if (ReturnType == EAppReturnType::No)
			{
				return;
			}
		}

		FMLDeformerEditorModule& EditorModule = FModuleManager::LoadModuleChecked<FMLDeformerEditorModule>("MLDeformerFrameworkEditor");
		FMLDeformerEditorModelRegistry& ModelRegistry = EditorModule.GetModelRegistry();

		// Remove existing actors etc.
		if (ActiveModel)
		{
			ActiveModel->ClearWorldAndPersonaPreviewScene();

			if (ModelDetailsView)
			{
				ModelDetailsView->SetObject(nullptr);
				ModelDetailsView->ForceRefresh();
			}

			if (VizSettingsDetailsView)
			{
				VizSettingsDetailsView->SetObject(nullptr);
				VizSettingsDetailsView->ForceRefresh();
			}
		}

		// Get the runtime model type based on the index, and create an instance of it.
		TArray<UClass*> ModelTypes;
		ModelRegistry.GetRegisteredModels().GenerateKeyArray(ModelTypes);
		TObjectPtr<UMLDeformerModel> Model = NewObject<UMLDeformerModel>(DeformerAsset, ModelTypes[Index], NAME_None, RF_Transactional);
		check(Model);
		Model->Init(DeformerAsset);
		DeformerAsset->SetModel(Model);
	
		// Create a new editor model that is related to this runtime model.
		FMLDeformerEditorModel* EditorModel = ModelRegistry.CreateEditorModel(Model);
		FMLDeformerEditorModel::InitSettings InitSettings;	
		InitSettings.Editor = this;
		InitSettings.Model = Model;
		EditorModel->Init(InitSettings);

		// Tell the editor we use this model now.
		TSharedPtr<FMLDeformerEditorModel> NewModel = TSharedPtr<FMLDeformerEditorModel>(EditorModel);

		if (ActiveModel)
		{
			NewModel->CopyBaseSettingsFromModel(ActiveModel.Get());
		}

		NewModel->GetModel()->InitVertexMap();
		NewModel->GetModel()->InitGPUData();

		ActiveModel.Reset();
		ActiveModel = NewModel;

		// Create the new scene for this model.
		if (PersonaToolkit)
		{
			HandlePreviewSceneCreated(PersonaToolkit->GetPreviewScene());
		}

		OnSwitchedVisualizationMode();
		ActiveModel->UpdateIsReadyForTrainingState();

		if (ModelDetailsView)
		{
			ModelDetailsView->SetObject(Model);
			ModelDetailsView->ForceRefresh();
		}

		if (VizSettingsDetailsView)
		{
			VizSettingsDetailsView->SetObject(Model->GetVizSettings());
			VizSettingsDetailsView->ForceRefresh();
		}

		if (TimeSlider)
		{
			TWeakPtr<FMLDeformerEditorModel> WeakModel = ActiveModel; 
			TimeSlider->SetModel(WeakModel);
		}

		DeformerAsset->Modify();
	}

	FText FMLDeformerEditorToolkit::GetVizModeName(EMLDeformerVizMode Mode) const
	{
		if (Mode == EMLDeformerVizMode::TrainingData)
		{
			return LOCTEXT("VizMode_Training", "Training");
		}
		else if (Mode == EMLDeformerVizMode::TestData)
		{
			return LOCTEXT("VizMode_Testing", "Testing");
		}

		check(false);
		return FText();
	}

	FText FMLDeformerEditorToolkit::GetCurrentVizModeName() const
	{
		if (!ActiveModel.IsValid())
		{
			return GetVizModeName(EMLDeformerVizMode::TrainingData);
		}

		const EMLDeformerVizMode VizMode = ActiveModel->GetModel()->GetVizSettings()->GetVisualizationMode();
		return GetVizModeName(VizMode);
	}

	FText FMLDeformerEditorToolkit::GetActiveModelName() const
	{	
		if (ActiveModel.IsValid())
		{
			return FText::FromString(ActiveModel->GetModel()->GetDisplayName());
		}

		return LOCTEXT("SelectModel", "<Select a Model>");
	}

	bool FMLDeformerEditorToolkit::IsTraining() const
	{
		return bIsTraining;
	}

	AActor* FMLDeformerEditorToolkit::GetDebugActor() const
	{
		return DebugWidget.IsValid() ? DebugWidget->GetDebugActor() : nullptr;
	}

	bool FMLDeformerEditorToolkit::Train(bool bSuppressDialogs)
	{
		check(ActiveModel);
		bIsTraining = true;

		// Ask if we want to retrain the network if we already have something trained.
		UMLDeformerModel* Model = ActiveModel->GetModel();
		if (ActiveModel->IsTrained() && !bSuppressDialogs)
		{
			const EAppReturnType::Type ConfirmReturnType = FMessageDialog::Open(
				EAppMsgType::YesNo, 
				LOCTEXT("RetrainConfirmationMessage", "This asset already has been trained.\n\nAre you sure you would like to re-train the network with your current settings?\n"),
				LOCTEXT("RetrainConfirmationTitle", "Re-train the network?"));

			if (ConfirmReturnType == EAppReturnType::No || ConfirmReturnType == EAppReturnType::Cancel)
			{
				bIsTraining = false;
				return true;
			}
		}

		ShowNotification(LOCTEXT("StartTraining", "Starting training process"), SNotificationItem::ECompletionState::CS_Pending, true);

		ActiveModel->OnPreTraining();

		// Initialize the training inputs.
		ActiveModel->UpdateEditorInputInfo();

		// Make sure we have something to train on.
		// If this triggers, the train button most likely was enabled while it shouldn't be.
		check(!ActiveModel->GetEditorInputInfo()->IsEmpty());

		// Train the model, which executes the Python code's "train" function.
		const double StartTime = FPlatformTime::Seconds();
		const ETrainingResult TrainingResult = ActiveModel->Train();
		const double TrainingDuration = FPlatformTime::Seconds() - StartTime;
		bool bUsePartiallyTrained = true;	// Will get modified by the HandleTrainingResult function below.
		bool bSuccess = true;
		const bool bMarkDirty = HandleTrainingResult(TrainingResult, TrainingDuration, bUsePartiallyTrained, bSuppressDialogs, bSuccess);
		ActiveModel->OnPostTraining(TrainingResult, bUsePartiallyTrained);

		const bool NeedsResamplingBackup = ActiveModel->GetResamplingInputOutputsNeeded();
		ActiveModel->SetResamplingInputOutputsNeeded(NeedsResamplingBackup);
		ActiveModel->RefreshMLDeformerComponents();
		ActiveModel->SetHeatMapMaterialEnabled(Model->GetVizSettings()->GetShowHeatMap());

		if (bMarkDirty)
		{
			DeformerAsset->Modify();
		}

		GetModelDetailsView()->ForceRefresh();
		GetVizSettingsDetailsView()->ForceRefresh();

		bIsTraining = false;
		return bSuccess;
	}

	bool FMLDeformerEditorToolkit::IsTrainButtonEnabled() const
	{
		return ActiveModel.Get() ? ActiveModel->IsReadyForTraining() : false;
	}
	
	FSlateIcon FMLDeformerEditorToolkit::GetTrainingStatusIcon() const
	{
		static const FName CompileStatusBackground("AssetEditor.CompileStatus.Background");
		static const FName CompileStatusUnknown("AssetEditor.CompileStatus.Overlay.Unknown");
		static const FName CompileStatusWarning("AssetEditor.CompileStatus.Overlay.Warning");
		
		if (!ActiveModel.IsValid())
		{
			return FSlateIcon(FAppStyle::Get().GetStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusUnknown); 
		}

		if (ActiveModel->IsReadyForTraining())
		{
			return FSlateIcon(FAppStyle::Get().GetStyleSetName(), CompileStatusBackground, NAME_None);
		}

		return FSlateIcon(FAppStyle::Get().GetStyleSetName(), CompileStatusBackground, NAME_None, CompileStatusWarning);
	}
	
	void FMLDeformerEditorToolkit::FillToolbar(FToolBarBuilder& ToolbarBuilder)
	{
		// Training button and model selection.
		ToolbarBuilder.BeginSection("Main");
		{
			ToolbarBuilder.AddToolBarButton(
				FUIAction
				(
					FExecuteAction::CreateLambda([this]() { Train(false); }),
					FCanExecuteAction::CreateLambda([this]() { return IsTrainButtonEnabled() && !IsPaintModeActive(); })
				),
				NAME_None,
				LOCTEXT("TrainModel", "Train Model"),
				LOCTEXT("TrainModelTooltip", "Train the machine learning model.\nIf this button is disabled you first need to configure your inputs in the details panel."),
				TAttribute<FSlateIcon>(this, &FMLDeformerEditorToolkit::GetTrainingStatusIcon),
				EUserInterfaceActionType::ToggleButton
			);

			ToolbarBuilder.AddSeparator();

			// Model selection.
			TSharedPtr<FUICommandList> CommandList = GetToolkitCommands();
			ToolbarBuilder.AddComboButton(
				FUIAction
				(
					FExecuteAction(),
					FCanExecuteAction::CreateLambda([this]() { return !IsPaintModeActive(); })						
				),
				FOnGetContent::CreateRaw(this, &FMLDeformerEditorToolkit::GenerateModelButtonContents, CommandList.ToSharedRef()),
				TAttribute<FText>::CreateRaw(this, &FMLDeformerEditorToolkit::GetActiveModelName),
				LOCTEXT("ActiveModelTooltip", "The currently active ML Deformer Model."),
				FSlateIcon(FMLDeformerEditorStyle::Get().GetStyleSetName(), "MLDeformer.VizSettings.TabIcon")
			);

			// Visualization mode.
			ToolbarBuilder.AddComboButton(
				FUIAction
				(
					FExecuteAction(),
					FCanExecuteAction::CreateLambda([this]() { return !IsPaintModeActive(); })						
				),
				FOnGetContent::CreateRaw(this, &FMLDeformerEditorToolkit::GenerateVizModeButtonContents, CommandList.ToSharedRef()),
				TAttribute<FText>::CreateRaw(this, &FMLDeformerEditorToolkit::GetCurrentVizModeName),			
				LOCTEXT("VizModeModeTooltip", "The visualization mode, specifying whether you are working on training or testing."),
				FSlateIcon(FMLDeformerEditorStyle::Get().GetStyleSetName(), "MLDeformer.VizSettings.TabIcon")
			);
		}
		ToolbarBuilder.EndSection();

		// Tools.
		if (!ToolsMenuExtenders.IsEmpty())
		{
			ToolbarBuilder.BeginSection("Tools");
			{
				TSharedPtr<FUICommandList> CommandList = GetToolkitCommands();
				ToolbarBuilder.AddComboButton(
					FUIAction
					(
						FExecuteAction(),
						FCanExecuteAction::CreateLambda([this]() { return !IsPaintModeActive(); })						
					),
					FOnGetContent::CreateRaw(this, &FMLDeformerEditorToolkit::GenerateToolsMenuContents, CommandList.ToSharedRef()),
					TAttribute<FText>::CreateLambda(
						[this]()
						{
							return LOCTEXT("ToolsMenu", "Tools");
						}
					),
					LOCTEXT("ToolsMenuTooltip", "Tools"),
					FSlateIcon(FMLDeformerEditorStyle::Get().GetStyleSetName(), "MLDeformer.VizSettings.TabIcon")
				);
			}
			ToolbarBuilder.EndSection();
		}

		// Debugging.
		ToolbarBuilder.BeginSection("Debugging");
		{
			DebugWidget->SetEnabled(TAttribute<bool>::CreateLambda([this]()
			{
				return !IsPaintModeActive() && GetDebuggingVisibility() == EVisibility::Visible;
			}));

			ToolbarBuilder.AddWidget(DebugWidget.ToSharedRef());
			ToolbarBuilder.AddToolBarButton
			(
				FUIAction
				(
					FExecuteAction::CreateLambda([this]() { if (DebugWidget.IsValid()) { DebugWidget->Refresh(); } }),
					FCanExecuteAction::CreateLambda([this]() { return !IsPaintModeActive() && GetDebuggingVisibility() == EVisibility::Visible; }),
					FGetActionCheckState::CreateLambda([]() { return ECheckBoxState::Checked; }),
					FIsActionButtonVisible::CreateLambda([this]() { return true;/*return GetDebuggingVisibility() == EVisibility::Visible;*/ })
				),
				NAME_None,
				FText(),
				LOCTEXT("RefreshDebugTooltip", "Refresh the list of debuggable actors."),
				FSlateIcon(FMLDeformerEditorStyle::Get().GetStyleSetName(), "MLDeformer.Debug.RefreshIcon"),
				EUserInterfaceActionType::Button
			);
		}
		ToolbarBuilder.EndSection();

		ToolbarBuilder.BeginSection("EditorModes");
		{
			ToolbarBuilder.AddToolBarButton(
				FUIAction
				(
					FExecuteAction::CreateLambda
					(
						[this]()
						{
							if (!GetEditorModeManager().IsModeActive(UMLDeformerPaintMode::Id))
							{
								EnablePaintMode();
							}
							else
							{
								DisablePaintMode();
							}							
						}
					),
					FCanExecuteAction::CreateLambda([this]()
					{
						if (ActiveModel.IsValid() && ActiveModel->GetModel() && ActiveModel->GetModel()->GetSkeletalMesh())
						{
							return true;
						}
						return false; 
					}),
					FIsActionChecked::CreateLambda([this] { return IsPaintModeActive(); })
				),
				NAME_None,
				LOCTEXT("PaintMode", "Paint Mode"),
				LOCTEXT("PaintModeTooltip", "Enable or disable vertex attribute painting."),
				FSlateIcon(),
				EUserInterfaceActionType::ToggleButton
			);
		}
	}

	EVisibility FMLDeformerEditorToolkit::GetDebuggingVisibility() const
	{
		if (ActiveModel && ActiveModel->GetModel() && ActiveModel->GetModel()->GetVizSettings())
		{
			if (ActiveModel->GetModel()->GetVizSettings()->GetVisualizationMode() == EMLDeformerVizMode::TestData)
			{
				return EVisibility::Visible;
			}
			else
			{
				return EVisibility::Collapsed;
			}
		}
		return EVisibility::Visible;
	}

	bool FMLDeformerEditorToolkit::HandleTrainingResult(ETrainingResult TrainingResult, double TrainingDuration, bool& bOutUsePartiallyTrained, bool bSuppressDialogs, bool& bOutSuccess)
	{
		bOutSuccess = false;
		bOutUsePartiallyTrained = true;

		FText WindowMessage;

		// Calculate hours, minutes and seconds.
		const int32 Hours = static_cast<int32>(TrainingDuration / 3600);
		TrainingDuration -= Hours * 3600.0;
		const int32 Minutes = static_cast<int32>(TrainingDuration / 60);
		TrainingDuration -= Minutes * 60.0;
		const int32 Seconds = static_cast<int32>(TrainingDuration);

		// Format the results in some HH:MM::SS format.
		FFormatNamedArguments Args;
		FNumberFormattingOptions NumberOptions;
		NumberOptions.SetMinimumIntegralDigits(2);
		NumberOptions.SetUseGrouping(false);
		Args.Add(TEXT("Hours"), FText::AsNumber(Hours, &NumberOptions));
		Args.Add(TEXT("Minutes"), FText::AsNumber(Minutes, &NumberOptions));
		Args.Add(TEXT("Seconds"), FText::AsNumber(Seconds, &NumberOptions));
		const FText FormatString = LOCTEXT("TrainingDurationFormat", "{Hours}:{Minutes}:{Seconds} (HH:MM:SS)");
		const FText TrainingDurationText = FText::Format(FormatString, Args);
		UE_LOGF(LogMLDeformer, Display, "Training duration: %ls", *TrainingDurationText.ToString());

		bool bMarkDirty = false;
		switch (TrainingResult)
		{
			// Training fully finished.
			case ETrainingResult::Success:
			{
				ActiveModel->SetResamplingInputOutputsNeeded(false);
				if (!ActiveModel->LoadTrainedNetwork())
				{
					GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/Notifications/CompileFailed_Cue.CompileFailed_Cue"));
					WindowMessage = LOCTEXT("TrainingOnnxLoadFailed", "Training completed but resulting network couldn't be loaded!\n");
				}
				else
				{
					GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/Notifications/CompileSuccess_Cue.CompileSuccess_Cue"));
					FFormatNamedArguments SuccessArgs;
					SuccessArgs.Add(TEXT("Duration"), TrainingDurationText);
					WindowMessage = FText::Format(LOCTEXT("TrainingSuccess", "Training completed successfully!\n\nTraining time: {Duration}\n"), SuccessArgs);
					ActiveModel->InitInputInfo(ActiveModel->GetModel()->GetInputInfo());
					bMarkDirty = true;
					bOutSuccess = true;
				}
			}
			break;

			// User aborted the training process. Ask whether they want to use the partially trained results or not.
			case ETrainingResult::Aborted:
			{
				ActiveModel->SetResamplingInputOutputsNeeded(false);

				EAppReturnType::Type ReturnType = EAppReturnType::No;	// When we suppress dialogs, do not use the the partially trained network on abort.
				if (!bSuppressDialogs)
				{
					ReturnType = FMessageDialog::Open(
						EAppMsgType::YesNo, 
						LOCTEXT("TrainingAbortedMessage", "Training has been aborted.\nThe neural network has only been partially trained.\nWould you like to use this partially trained network?\n"),
						LOCTEXT("TrainingAbortedMessageTitle", "Use partially trained network?"));
				}

				if (ReturnType == EAppReturnType::Yes)
				{
					if (!ActiveModel->LoadTrainedNetwork())
					{
						ShowNotification(LOCTEXT("TrainingOnnxLoadFailedPartial", "Training partially completed, but resulting network couldn't be loaded!"), SNotificationItem::ECompletionState::CS_Fail, true);
					}
					else
					{
						ShowNotification(LOCTEXT("PartialTrainingSuccess", "Training partially completed!"), SNotificationItem::ECompletionState::CS_Success, true);
						ActiveModel->InitInputInfo(ActiveModel->GetModel()->GetInputInfo());
						bMarkDirty = true;
					}
				}
				else
				{
					bOutUsePartiallyTrained = false;
					ShowNotification(LOCTEXT("TrainingAborted", "Training aborted!"), SNotificationItem::ECompletionState::CS_None, true);
				}

				bOutSuccess = true;
			}
			break;

			// Training aborted but we cannot use the current network.
			case ETrainingResult::AbortedCantUse:
			{
				ShowNotification(LOCTEXT("TrainingAborted", "Training aborted!"), SNotificationItem::ECompletionState::CS_None, true);
				WindowMessage = LOCTEXT("TrainingAbortedCantUse", "Training aborted by user.\n");
			}
			break;

			// Training data had issues.
			case ETrainingResult::FailOnData:
			{
				GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/Notifications/CompileFailed_Cue.CompileFailed_Cue"));
				WindowMessage = LOCTEXT("TrainingFailedOnData", "Training failed!\nCheck input parameters or sequence length.\n");
			}
			break;

			// There is an error in the python script.
			case ETrainingResult::FailPythonError:
			{
				GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/Notifications/CompileFailed_Cue.CompileFailed_Cue"));
				WindowMessage = LOCTEXT("TrainingPythonError", "Training failed!\nThere is a python error, please check the output log.\n");
			}
			break;

			// There is an error another error, some exception got thrown while training.
			case ETrainingResult::Other:
			{
				GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/Notifications/CompileFailed_Cue.CompileFailed_Cue"));
				WindowMessage = LOCTEXT("TrainingOtherError", "Training failed!\nPlease check the output log.\n");
			}
			break;

			// Unhandled error codes.
			default:
				checkf(false, TEXT("Unknown error code"));
		}

		// Show a message window.
		if (!WindowMessage.IsEmpty() && !bSuppressDialogs)
		{
			FMessageDialog::Open(EAppMsgType::Ok, WindowMessage, LOCTEXT("TrainingResultsWindowTitle", "Training Results"));
		}

		ActiveModel->UpdateDeformerGraph();
		return bMarkDirty;
	}

	FName FMLDeformerEditorToolkit::GetToolkitFName() const
	{
		return FName("MLDeformerFrameworkEditor");
	}

	FText FMLDeformerEditorToolkit::GetBaseToolkitName() const
	{
		return LOCTEXT("MLDeformerFrameworkEditorAppLabel", "ML Deformer Editor");
	}

	void FMLDeformerEditorToolkit::ShowNotification(const FText& Message, SNotificationItem::ECompletionState State, bool PlaySound) const
	{
		FNotificationInfo Info(Message);
		Info.FadeInDuration = 0.1f;
		Info.FadeOutDuration = 0.5f;
		Info.ExpireDuration = 3.5f;
		Info.bUseThrobber = false;
		Info.bUseSuccessFailIcons = true;
		Info.bUseLargeFont = true;
		Info.bFireAndForget = true;
		Info.bAllowThrottleWhenFrameRateIsLow = false;
		auto NotificationItem = FSlateNotificationManager::Get().AddNotification(Info);
		NotificationItem->SetCompletionState(State);
		NotificationItem->ExpireAndFadeout();

		if (PlaySound)
		{
			switch (State)
			{
				case SNotificationItem::ECompletionState::CS_Success: GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/Notifications/CompileSuccess_Cue.CompileSuccess_Cue")); break;
				case SNotificationItem::ECompletionState::CS_Fail: GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/Notifications/CompileFailed_Cue.CompileFailed_Cue")); break;
				case SNotificationItem::ECompletionState::CS_Pending: GEditor->PlayEditorSound(TEXT("/Engine/EditorSounds/Notifications/CompileStart_Cue.CompileStart_Cue")); break;
				default:;
			};
		}
	}

	FText FMLDeformerEditorToolkit::GetToolkitName() const
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("AssetName"), FText::FromString(DeformerAsset->GetName()));
		return FText::Format(LOCTEXT("MLDeformerEditorToolkitName", "{AssetName}"), Args);
	}

	FLinearColor FMLDeformerEditorToolkit::GetWorldCentricTabColorScale() const
	{
		return FLinearColor::White;
	}

	FString FMLDeformerEditorToolkit::GetWorldCentricTabPrefix() const
	{
		return TEXT("MLDeformerFrameworkEditor");
	}

	void FMLDeformerEditorToolkit::AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObject(DeformerAsset);
	}

	TStatId FMLDeformerEditorToolkit::GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FMLDeformerEditorToolkit, STATGROUP_Tickables);
	}

	void FMLDeformerEditorToolkit::PostUndo(bool bSuccess)
	{
		if (ActiveModel && !IsPaintModeActive())
		{
			ActiveModel->TriggerInputAssetChanged(true);
		}
	}

	void FMLDeformerEditorToolkit::PostRedo(bool bSuccess)
	{
		if (ActiveModel && !IsPaintModeActive())
		{
			ActiveModel->TriggerInputAssetChanged(true);
		}
	}

	void FMLDeformerEditorToolkit::HandlePreviewSceneCreated(const TSharedRef<IPersonaPreviewScene>& InPersonaPreviewScene)
	{
		if (ActiveModel)
		{
			ActiveModel->SetDefaultDeformerGraphIfNeeded();
			ActiveModel->CreateActors(InPersonaPreviewScene);
			ActiveModel->UpdateActorVisibility();

			if (GetModelDetailsView())
			{
				GetModelDetailsView()->SetObject(nullptr, true);
			}

			if (GetVizSettingsDetailsView())
			{
				GetVizSettingsDetailsView()->SetObject(nullptr, true);
			}
			ActiveModel->CreateHeatMapAssets();
			ActiveModel->SetHeatMapMaterialEnabled(ActiveModel->GetModel()->GetVizSettings()->GetShowHeatMap());
			ActiveModel->SetResamplingInputOutputsNeeded(true);
		}
	}

	void FMLDeformerEditorToolkit::HandleDetailsCreated(const TSharedRef<class IDetailsView>& InDetailsView)
	{
		ModelDetailsView = InDetailsView;
		InDetailsView->OnFinishedChangingProperties().AddSP(this, &FMLDeformerEditorToolkit::OnFinishedChangingDetails);
	}

	void FMLDeformerEditorToolkit::OnFinishedChangingDetails(const FPropertyChangedEvent& PropertyChangedEvent)
	{
		const FProperty* Property = PropertyChangedEvent.Property;
		if (Property == nullptr)
		{
			return;
		}

		if (Property->GetFName() == UMLDeformerVizSettings::GetVisualizationModePropertyName())
		{
			OnSwitchedVisualizationMode();
		}
	}

	void FMLDeformerEditorToolkit::OnSwitchedVisualizationMode()
	{
		// Make sure the time slider is updated to reflect the right animation range.
		UpdateTimeSliderRange();

		if (ActiveModel)
		{
			UMLDeformerVizSettings* VizSettings = ActiveModel->GetModel()->GetVizSettings();
			if (VizSettings)
			{
				ActiveModel->UpdateActorVisibility();

				if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TrainingData)
				{
					ActiveModel->ClampCurrentTrainingFrameIndex();
					const int32 CurrentFrameNumber = ActiveModel->GetModel()->GetVizSettings()->GetTrainingFrameNumber();
					ActiveModel->SetTrainingFrame(CurrentFrameNumber, false);
					ActiveModel->SampleDeltas();
				}
				else if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData)
				{
					ActiveModel->ClampCurrentTestFrameIndex();
					const int32 CurrentFrameNumber = ActiveModel->GetModel()->GetVizSettings()->GetTestingFrameNumber();
					ActiveModel->SetTestFrame(CurrentFrameNumber, false);
				}

				if (GetVizSettingsDetailsView())
				{
					GetVizSettingsDetailsView()->ForceRefresh();
				}

				ActiveModel->SetHeatMapMaterialEnabled(VizSettings->GetShowHeatMap());
				ActiveModel->UpdateDeformerGraph();
				ActiveModel->HandleVizModeChanged(VizSettings->GetVisualizationMode());
			}
		}
		ZoomOnActors();
	}

	TArray<FTransform> FMLDeformerEditorToolkit::GetDebugActorComponentSpaceTransforms() const
	{
		const FMLDeformerEditorModel* EditorModel = GetActiveModel();
		if (EditorModel)
		{
			const AActor* DebugActor = GetDebugActor();
			if (DebugActor && EditorModel->GetModel())
			{				
				const USkeletalMesh* ModelSkelMesh = EditorModel->GetModel()->GetSkeletalMesh();
				if (ModelSkelMesh)
				{
					for (const UActorComponent* Component : DebugActor->GetComponents())
					{
						const USkeletalMeshComponent* DebugActorSkelMeshComponent = Cast<USkeletalMeshComponent>(Component);
						if (DebugActorSkelMeshComponent)
						{
							if (DebugActorSkelMeshComponent->GetSkeletalMeshAsset() != ModelSkelMesh)
							{
								continue;
							}

							const USkinnedMeshComponent* LeaderPoseComponent = DebugActorSkelMeshComponent->LeaderPoseComponent.Get();
							if (LeaderPoseComponent && !LeaderPoseComponent->GetComponentSpaceTransforms().IsEmpty())
							{
								const FReferenceSkeleton& RefSkel = ModelSkelMesh->GetRefSkeleton();
								const int32 NumBones = RefSkel.GetNum();

								TArray<FTransform> OutTransforms;	// TODO: Create some cached reusable buffer?
								OutTransforms.SetNumUninitialized(NumBones);

								const TArray<FTransform>& FollowerComponentTransforms = DebugActorSkelMeshComponent->GetComponentSpaceTransforms();
								const TArray<FTransform>& LeaderComponentTransforms = LeaderPoseComponent->GetComponentSpaceTransforms();

								const TArray<int32>& BoneMap = DebugActorSkelMeshComponent->GetLeaderBoneMap();
								for (int32 Index = 0; Index < NumBones; ++Index)
								{									
									const int32 LeaderTransformIndex = BoneMap[Index];
									if (LeaderTransformIndex != INDEX_NONE)
									{
										OutTransforms[Index] = LeaderComponentTransforms[LeaderTransformIndex];
									}
									else
									{
										OutTransforms[Index] = FollowerComponentTransforms[Index];
									}
								}

								return MoveTemp(OutTransforms);
							}
							else
							{
								return DebugActorSkelMeshComponent->GetComponentSpaceTransforms();
							}
						}
					}
				}
			}
		}

		return TArray<FTransform>();
	}

	void FMLDeformerEditorToolkit::ZoomOnActors()
	{
		// Zoom in on the actors.
		if (bIsInitialized)
		{
			FMLDeformerEditorMode* EditorMode = static_cast<FMLDeformerEditorMode*>(GetEditorModeManager().GetActiveMode(FMLDeformerEditorMode::ModeName));
			if (EditorMode && PersonaViewport.IsValid())
			{
				FSphere Sphere;
				if (EditorMode->GetCameraTarget(Sphere))
				{
					const FBox Box(Sphere.Center - FVector(Sphere.W, 0.0f, 0.0f), Sphere.Center + FVector(Sphere.W, 0.0f, 0.0f));
					PersonaViewport->GetViewportClient().FocusViewportOnBox(Box);
				}
			}
		}
	}

	void FMLDeformerEditorToolkit::HandleViewportCreated(const TSharedRef<IPersonaViewport>& InPersonaViewport)
	{
		PersonaViewport = InPersonaViewport;

		InPersonaViewport->AddOverlayWidget(
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.Padding(FMargin(5.0f, 40.0f))
			[
				SNew(STextBlock)
				.Text(this, &FMLDeformerEditorToolkit::GetOverlayText)
				.Visibility(EVisibility::HitTestInvisible)
				.ColorAndOpacity(FLinearColor(1.0f, 0.5f, 0.0f, 1.0f))
			]
		);
	}

	FText FMLDeformerEditorToolkit::GetOverlayText() const
	{
		if (!ActiveModel)
		{
			return FText::GetEmpty();
		}

		return ActiveModel->GetOverlayText();
	}

	IDetailsView* FMLDeformerEditorToolkit::GetModelDetailsView() const
	{
		return ModelDetailsView.Get();
	}

	IDetailsView* FMLDeformerEditorToolkit::GetVizSettingsDetailsView() const
	{
		return VizSettingsDetailsView.Get();
	}

	void FMLDeformerEditorToolkit::SetTimeSlider(TSharedPtr<SMLDeformerTimeline> InTimeSlider)
	{
		TimeSlider = InTimeSlider;
	}

	SMLDeformerTimeline* FMLDeformerEditorToolkit::GetTimeSlider() const
	{
		return TimeSlider.Get();
	}

	double FMLDeformerEditorToolkit::CalcTimelinePosition() const
	{
		if (ActiveModel)
		{
			const UMLDeformerVizSettings* VizSettings = ActiveModel->GetModel()->GetVizSettings();
			if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TrainingData)
			{
				return ActiveModel->CalcTrainingTimelinePosition();
			}
			else if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData)
			{
				return ActiveModel->CalcTestTimelinePosition();
			}
		}

		return 0.0;
	}

	void FMLDeformerEditorToolkit::OnTimeSliderScrubPositionChanged(double NewScrubTime, bool bIsScrubbing)
	{
		if (ActiveModel)
		{
			ActiveModel->OnTimeSliderScrubPositionChanged(NewScrubTime, bIsScrubbing);
		}
	}

	void FMLDeformerEditorToolkit::UpdateTimeSliderRange()
	{
		if (!ActiveModel.IsValid())
		{
			SetTimeSliderRange(0.0, 1.0);
			return;
		}

		const UMLDeformerVizSettings* VizSettings = ActiveModel->GetModel()->GetVizSettings();
		if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TrainingData)
		{
			const UAnimSequence* AnimSeq = ActiveModel.Get() ? ActiveModel->GetActiveTrainingInputAnimSequence() : nullptr;
			const double Duration = AnimSeq ? AnimSeq->GetPlayLength() : 0.0;
			SetTimeSliderRange(0.0, Duration);
		}
		else
		if (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData)
		{
			const UAnimSequence* AnimSeq = ActiveModel.Get() ? ActiveModel->GetModel()->GetVizSettings()->GetTestAnimSequence() : nullptr;
			const double Duration = AnimSeq ? AnimSeq->GetPlayLength() : 0.0;
			SetTimeSliderRange(0.0, Duration);
		}

		ActiveModel->ClampCurrentTrainingFrameIndex();
	}

	void FMLDeformerEditorToolkit::SetTimeSliderRange(double StartTime, double EndTime)
	{
		TRange<double> ViewRange(StartTime, EndTime); 
		if (ActiveModel.IsValid())
		{
			ActiveModel->SetViewRange(ViewRange);
		}
	}

	bool FMLDeformerEditorToolkit::SwitchModelType(UClass* ModelType, bool bForceChange)
	{
		FMLDeformerEditorModule& EditorModule = FModuleManager::GetModuleChecked<FMLDeformerEditorModule>("MLDeformerFrameworkEditor");
		FMLDeformerEditorModelRegistry& ModelRegistry = EditorModule.GetModelRegistry();

		// Get the list of all registered model types.
		TArray<UClass*> ModelTypes;
		ModelRegistry.GetRegisteredModels().GenerateKeyArray(ModelTypes);

		// Find the model index for this type.
		const int32 Index = ModelTypes.Find(ModelType);
		if (Index == INDEX_NONE)
		{
			return false;
		}

		OnModelChanged(Index, bForceChange);
		return true;
	}

	void FMLDeformerEditorToolkit::AddToolsMenuExtender(TUniquePtr<FToolsMenuExtender> Extender)
	{
		FScopeLock Lock(&ExtendersMutex);
		ToolsMenuExtenders.Emplace(MoveTemp(Extender));
	}

	TConstArrayView<TUniquePtr<FToolsMenuExtender>> FMLDeformerEditorToolkit::GetToolsMenuExtenders()
	{
		FScopeLock Lock(&ExtendersMutex);
		return ToolsMenuExtenders;
	}

	void FMLDeformerEditorToolkit::EnablePaintMode()
	{
		GetEditorModeManager().DeactivateMode(FMLDeformerEditorMode::ModeName);
		GetEditorModeManager().ActivateMode(UMLDeformerPaintMode::Id);

		UEdMode* ActiveMode = GetEditorModeManager().GetActiveScriptableMode(UMLDeformerPaintMode::Id);
		if (UMLDeformerPaintMode* PaintMode = CastChecked<UMLDeformerPaintMode>(ActiveMode))
		{
			PaintMode->SetMLDeformerEditor(this);

			if (ActiveModel.IsValid())
			{
				ActiveModel->UpdatePaintModePose();
			}

			// Disable the viz settings,details and timeline tabs, as we don't want to be able to modify those settings while in Paint mode.
			TSharedPtr<SDockTab> VizTab = TabManager->FindExistingLiveTab(FMLDeformerVizSettingsTabSummoner::TabID);
			if (VizTab.IsValid())
			{
				VizTab->SetEnabled(false);
				VizTab->GetContent()->SetEnabled(false);
			}

			TSharedPtr<SDockTab> DetailsTab = TabManager->FindExistingLiveTab(FPersonaTabs::DetailsID);
			if (DetailsTab.IsValid())
			{
				DetailsTab->SetEnabled(false);
				DetailsTab->GetContent()->SetEnabled(false);
			}
		}

		if (ActiveModel.IsValid())
		{
			ActiveModel->UpdateActorVisibility();
			ActiveModel->UpdateLabels();
		}
	}

	void FMLDeformerEditorToolkit::DisablePaintMode()
	{
		GetEditorModeManager().DeactivateMode(UMLDeformerPaintMode::Id);
		GetEditorModeManager().ActivateMode(FMLDeformerEditorMode::ModeName);

		// Enable some tabs again after Paint mode ended.
		TSharedPtr<SDockTab> VizTab = TabManager->FindExistingLiveTab(FMLDeformerVizSettingsTabSummoner::TabID);
		if (VizTab.IsValid())
		{
			VizTab->SetEnabled(true);
			VizTab->GetContent()->SetEnabled(true);
		}

		TSharedPtr<SDockTab> DetailsTab = TabManager->FindExistingLiveTab(FPersonaTabs::DetailsID);
		if (DetailsTab.IsValid())
		{
			DetailsTab->SetEnabled(true);
			DetailsTab->GetContent()->SetEnabled(true);
		}

		// Update the actor visibility as the mode changes that.
		if (ActiveModel.IsValid())
		{
			ActiveModel->UpdateLabels();
			ActiveModel->RequestTriggerInputAssetsChangedNextTick();
		}
	}

	bool FMLDeformerEditorToolkit::IsDefaultModeActive() const
	{
		if (GetPersonaToolkitPointer() == nullptr)
		{
			return true;
		}
		return GetEditorModeManager().IsModeActive(FMLDeformerEditorMode::ModeName);
	}

	bool FMLDeformerEditorToolkit::IsPaintModeActive() const
	{
		if (GetPersonaToolkitPointer() == nullptr)
		{
			return true;
		}
		return GetEditorModeManager().IsModeActive(UMLDeformerPaintMode::Id);
	}

	void FMLDeformerEditorToolkit::AddViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget, int32 ZOrder)
	{
		if (GetViewport().IsValid())
		{
			GetViewport()->AddOverlayWidget(InViewportOverlayWidget, ZOrder);
		}	
	}

	void FMLDeformerEditorToolkit::RemoveViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget)
	{
		if (GetViewport().IsValid())
		{
			GetViewport()->RemoveOverlayWidget(InViewportOverlayWidget);
		}	
	}

	void FMLDeformerEditorToolkit::Tick(float DeltaTime)
	{
		if (bNeedsPaintModeDisable)
		{
			DisablePaintMode();
			bNeedsPaintModeDisable = false;
		}		
	}

}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
