// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerGeomCacheActor.h"
#include "GeometryCacheComponent.h"
#include "Animation/DebugSkelMeshComponent.h"

namespace UE::MLDeformer
{
	FMLDeformerGeomCacheActor::FMLDeformerGeomCacheActor(const FConstructSettings& Settings)
		: FMLDeformerEditorActor(Settings)
	{
	}

	void FMLDeformerGeomCacheActor::SetVisibility(bool bIsVisible)
	{
		FMLDeformerEditorActor::SetVisibility(bIsVisible);

		if (GeomCacheComponent && bIsVisible != GeomCacheComponent->IsVisible())
		{
			GeomCacheComponent->SetVisibility(bIsVisible, true);
		}
	}

	bool FMLDeformerGeomCacheActor::IsVisible() const
	{
		if (SkeletalMeshComponent)
		{
			return SkeletalMeshComponent->IsVisible();
		}

		if (GeomCacheComponent)
		{
			return GeomCacheComponent->IsVisible();
		}

		return true;
	}

	bool FMLDeformerGeomCacheActor::HasVisualMesh() const
	{
		if (SkeletalMeshComponent && SkeletalMeshComponent->GetSkeletalMeshAsset())
		{
			return true;
		}

		if (GeomCacheComponent && GeomCacheComponent->GetGeometryCache())
		{
			return true;
		}

		return false;
	}

	void FMLDeformerGeomCacheActor::SetPlayPosition(float TimeInSeconds, bool bAutoPause)
	{
		if (SkeletalMeshComponent)
		{
			SkeletalMeshComponent->SetPosition(TimeInSeconds);
			if (bAutoPause)
			{
				SkeletalMeshComponent->bPauseAnims = true;
			}
		}

		if (GeomCacheComponent)
		{
			GeomCacheComponent->SetManualTick(true);
			GeomCacheComponent->TickAtThisTime(TimeInSeconds, false, false, false);
		}
	}

	float FMLDeformerGeomCacheActor::GetPlayPosition() const
	{
		if (SkeletalMeshComponent)
		{
			return SkeletalMeshComponent->GetPosition();
		}

		if (GeomCacheComponent)
		{
			float Duration = GeomCacheComponent->GetDuration();
			float AnimTime = GeomCacheComponent->GetAnimationTime();
			float StartTime = GeomCacheComponent->GetStartTimeOffset();
			float DeltaTime = AnimTime - StartTime;
			return DeltaTime > Duration ? FMath::Fmod(DeltaTime, Duration) : DeltaTime;
		}

		return 0.0f;
	}

	void FMLDeformerGeomCacheActor::SetPlaySpeed(float PlaySpeed)
	{
		if (SkeletalMeshComponent)
		{
			SkeletalMeshComponent->SetPlayRate(PlaySpeed);
		}

		if (GeomCacheComponent)
		{
			GeomCacheComponent->SetPlaybackSpeed(PlaySpeed);
		}
	}

	void FMLDeformerGeomCacheActor::Pause(bool bPaused)
	{
		if (SkeletalMeshComponent)
		{
			SkeletalMeshComponent->bPauseAnims = bPaused;
		}

		if (GeomCacheComponent)
		{
			GeomCacheComponent->SetManualTick(bPaused);
		}
	}

	bool FMLDeformerGeomCacheActor::IsPlaying() const
	{
		if (SkeletalMeshComponent)
		{
			return !SkeletalMeshComponent->bPauseAnims;
		}

		if (GeomCacheComponent)
		{
			return GeomCacheComponent->IsPlaying();
		}

		return false;
	}

	FBox FMLDeformerGeomCacheActor::GetBoundingBox() const
	{
		if (SkeletalMeshComponent)
		{
			return SkeletalMeshComponent->Bounds.GetBox();
		}

		if (GeomCacheComponent)
		{
			return GeomCacheComponent->Bounds.GetBox();
		}

		FBox Box;
		Box.Init();
		return Box;
	}
}	// namespace UE::MLDeformer
