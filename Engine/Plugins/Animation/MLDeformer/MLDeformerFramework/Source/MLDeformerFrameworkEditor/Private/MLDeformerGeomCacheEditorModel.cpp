// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerGeomCacheEditorModel.h"
#include "MLDeformerGeomCacheActor.h"
#include "MLDeformerEditorToolkit.h"
#include "MLDeformerEditorStyle.h"
#include "MLDeformerGeomCacheModel.h"
#include "MLDeformerGeomCacheVizSettings.h"
#include "MLDeformerInputInfo.h"
#include "MLDeformerGeomCacheSampler.h"
#include "MLDeformerGeomCacheHelpers.h"
#include "MLDeformerGeomCacheModel.h"
#include "SMLDeformerTimeline.h"
#include "GeometryCache.h"
#include "GeometryCacheComponent.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Animation/MorphTarget.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Engine/SkeletalMesh.h"

#define LOCTEXT_NAMESPACE "MLDeformerGeomCacheEditorModel"

namespace UE::MLDeformer
{
	void FMLDeformerGeomCacheEditorModel::Init(const InitSettings& Settings)
	{
		FMLDeformerEditorModel::Init(Settings);

		// Create an initial training input animation entry.
		if (GetGeomCacheModel()->GetTrainingInputAnims().IsEmpty())
		{
			GetGeomCacheModel()->GetTrainingInputAnims().AddDefaulted();
			CreateSamplers();
		}
	}

	void FMLDeformerGeomCacheEditorModel::CopyBaseSettingsFromModel(const FMLDeformerEditorModel* SourceEditorModel)
	{
		UMLDeformerGeomCacheModel* SourceGeomCacheModel = Cast<UMLDeformerGeomCacheModel>(SourceEditorModel->GetModel());
		if (SourceGeomCacheModel) // If we have a model derived from the UMLDeformerGeomCacheModel
		{
			// Copy the training input animations.
			UMLDeformerGeomCacheModel* TargetModel = Cast<UMLDeformerGeomCacheModel>(GetModel());
			TArray<FMLDeformerGeomCacheTrainingInputAnim>& TargetInputAnims = TargetModel->GetTrainingInputAnims();
			TargetInputAnims = SourceGeomCacheModel->GetTrainingInputAnims();

			// Copy the ground truth in the visualization settings.
			UMLDeformerGeomCacheVizSettings* SourceVizSettings = Cast<UMLDeformerGeomCacheVizSettings>(SourceGeomCacheModel->GetVizSettings());
			if (SourceVizSettings)
			{
				UMLDeformerGeomCacheVizSettings* TargetVizSettings = Cast<UMLDeformerGeomCacheVizSettings>(TargetModel->GetVizSettings());
				check(TargetVizSettings)
				TargetVizSettings->SetTestGroundTruth(SourceVizSettings->GetTestGroundTruth());
			}
		}

		// Copy the settings of the base class.
		FMLDeformerEditorModel::CopyBaseSettingsFromModel(SourceEditorModel);
	}

	TSharedPtr<FMLDeformerSampler> FMLDeformerGeomCacheEditorModel::CreateSamplerObject() const
	{
		return MakeShared<FMLDeformerGeomCacheSampler>();
	}

	FMLDeformerEditorActor* FMLDeformerGeomCacheEditorModel::CreateEditorActor(const FMLDeformerEditorActor::FConstructSettings& Settings) const
	{
		return new FMLDeformerGeomCacheActor(Settings);
	}

	UMLDeformerGeomCacheVizSettings* FMLDeformerGeomCacheEditorModel::GetGeomCacheVizSettings() const 
	{ 
		return Cast<UMLDeformerGeomCacheVizSettings>(Model->GetVizSettings());
	}

	UMLDeformerGeomCacheModel* FMLDeformerGeomCacheEditorModel::GetGeomCacheModel() const
	{ 
		return Cast<UMLDeformerGeomCacheModel>(Model);
	}

	FMLDeformerGeomCacheActor* FMLDeformerGeomCacheEditorModel::FindGeomCacheEditorActor(int32 ID) const
	{
		return static_cast<FMLDeformerGeomCacheActor*>(FindEditorActor(ID));
	}

	UGeometryCache* FMLDeformerGeomCacheEditorModel::GetActiveGeometryCache() const
	{
		const int32 ActiveAnimIndex = GetActiveTrainingInputAnimIndex();
		if (ActiveAnimIndex == INDEX_NONE)
		{
			return nullptr;
		}

		FMLDeformerGeomCacheTrainingInputAnim* ActiveAnim = static_cast<FMLDeformerGeomCacheTrainingInputAnim*>(GetTrainingInputAnim(ActiveAnimIndex));
		return ActiveAnim ? ActiveAnim->GetGeometryCache() : nullptr;
	}

	void FMLDeformerGeomCacheEditorModel::UpdateNumTrainingFrames()
	{
		NumTrainingFrames = 0;

		const TArray<FMLDeformerGeomCacheTrainingInputAnim>& TrainingInputAnims = GetGeomCacheModel()->GetTrainingInputAnims();
		for (const FMLDeformerGeomCacheTrainingInputAnim& InputAnim : TrainingInputAnims)
		{
			if (InputAnim.IsEnabled() && InputAnim.IsValid())
			{
				NumTrainingFrames += InputAnim.GetNumFramesToSample();
			}
		}
	}

	void FMLDeformerGeomCacheEditorModel::UpdateIsReadyForTrainingState()
	{
		bIsReadyForTraining = false;
		
		// Do some basic checks first, like if there is a skeletal mesh, ground truth, anim sequence, and if there are frames.
		if (!IsEditorReadyForTrainingBasicChecks())
		{
			return;
		}
	
		// Now make sure the geom caches have no errors.
		UMLDeformerGeomCacheModel* GeomCacheModel = GetGeomCacheModel();
		USkeletalMesh* SkeletalMesh = GeomCacheModel->GetSkeletalMesh();
		const int32 NumAnims = GetNumTrainingInputAnims();
		for (int32 AnimIndex = 0; AnimIndex < NumAnims; ++AnimIndex)
		{
			FMLDeformerGeomCacheTrainingInputAnim* Anim = static_cast<FMLDeformerGeomCacheTrainingInputAnim*>(GetTrainingInputAnim(AnimIndex));
			if (Anim && Anim->IsEnabled())
			{
				UGeometryCache* GeomCache = Anim->GetGeometryCache();
				if (!GetGeomCacheErrorText(SkeletalMesh, GeomCache).IsEmpty())
				{
					return;
				}
			}
		}

		// Check if we have any mappings at all.
		for (int32 AnimIndex = 0; AnimIndex < NumAnims; ++AnimIndex)
		{
			FMLDeformerGeomCacheTrainingInputAnim* Anim = static_cast<FMLDeformerGeomCacheTrainingInputAnim*>(GetTrainingInputAnim(AnimIndex));
			if (!Anim->IsEnabled())
			{
				continue;
			}

			FMLDeformerGeomCacheSampler* GeomCacheSampler = static_cast<FMLDeformerGeomCacheSampler*>(GetSamplerForTrainingAnim(AnimIndex));
			if (GeomCacheSampler == nullptr)
			{
				return;
			}

			if (!GeomCacheSampler->IsInitialized())
			{
				GeomCacheSampler->Init(this, AnimIndex);
			}

			if (GeomCacheSampler->GetMeshMappings().IsEmpty())
			{
				return;
			}
		}

		bIsReadyForTraining = true;
	}

	void FMLDeformerGeomCacheEditorModel::CreateTrainingGroundTruthActor(UWorld* World)
	{
		const int32 ActiveAnimIndex = GetActiveTrainingInputAnimIndex();
		UGeometryCache* GeomCache = nullptr;
		if (ActiveAnimIndex != INDEX_NONE)
		{
			FMLDeformerGeomCacheTrainingInputAnim* Anim = static_cast<FMLDeformerGeomCacheTrainingInputAnim*>(GetTrainingInputAnim(ActiveAnimIndex));
			GeomCache = Anim ? Anim->GetGeometryCache() : nullptr;
		}

		const FLinearColor LabelColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.TargetMesh.LabelColor");
		const FLinearColor WireframeColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.TargetMesh.WireframeColor");
		CreateGeomCacheActor(
			World,
			ActorID_Train_GroundTruth,
			"Train GroundTruth",
			GeomCache,
			LabelColor,
			WireframeColor,
			LOCTEXT("TrainGroundTruthActorLabelText", "Target Mesh"),
			true);
	}

	void FMLDeformerGeomCacheEditorModel::CreateTestGroundTruthActor(UWorld* World)
	{
		UGeometryCache* GeomCache = GetGeomCacheVizSettings()->GetTestGroundTruth();
		const FLinearColor LabelColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.GroundTruth.LabelColor");
		const FLinearColor WireframeColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.GroundTruth.WireframeColor");
		CreateGeomCacheActor(
			World,
			ActorID_Test_GroundTruth,
			"Test GroundTruth",
			GeomCache,
			LabelColor,
			WireframeColor,
			LOCTEXT("TestGroundTruthActorLabelText", "Ground Truth"),
			false);
	}

	void FMLDeformerGeomCacheEditorModel::CreateGeomCacheActor(UWorld* World, int32 ActorID, const FName& Name, UGeometryCache* GeomCache, FLinearColor LabelColor, FLinearColor WireframeColor, const FText& LabelText, bool bIsTrainingActor)
	{
		FActorSpawnParameters SpawnParams;
		SpawnParams.Name = MakeUniqueObjectName(World, AActor::StaticClass(), Name);
		AActor* Actor = World->SpawnActor<AActor>(SpawnParams);
		Actor->SetFlags(RF_Transient);

		// Create the Geometry Cache Component.
		UGeometryCacheComponent* GeomCacheComponent = NewObject<UGeometryCacheComponent>(Actor);
		GeomCacheComponent->SetGeometryCache(GeomCache);
		GeomCacheComponent->RegisterComponent();
		GeomCacheComponent->SetOverrideWireframeColor(true);
		GeomCacheComponent->SetWireframeOverrideColor(WireframeColor);
		GeomCacheComponent->MarkRenderStateDirty();
		GeomCacheComponent->SetVisibility(false);
		Actor->SetRootComponent(GeomCacheComponent);

		// Create the editor actor.
		FMLDeformerEditorActor::FConstructSettings Settings;
		Settings.Actor = Actor;
		Settings.TypeID = ActorID;
		Settings.LabelColor = LabelColor;
		Settings.LabelText = LabelText;
		Settings.bIsTrainingActor = bIsTrainingActor;
		FMLDeformerGeomCacheActor* EditorActor = static_cast<FMLDeformerGeomCacheActor*>(CreateEditorActor(Settings));
		EditorActor->SetGeometryCacheComponent(GeomCacheComponent);
		EditorActors.Add(EditorActor);
	}

	ETrainingResult FMLDeformerGeomCacheEditorModel::Train()
	{
		UE_LOGF(LogMLDeformer, Error, "Please implement your Train method.");
		return ETrainingResult::Success;
	}

	void FMLDeformerGeomCacheEditorModel::OnPropertyChanged(FPropertyChangedEvent& PropertyChangedEvent)
	{
		const FProperty* Property = PropertyChangedEvent.Property;
		if (Property == nullptr)
		{
			return;
		}

		// Handle base class property changes.
		FMLDeformerEditorModel::OnPropertyChanged(PropertyChangedEvent);

		// Start frame changed.
		if (PropertyChangedEvent.GetMemberPropertyName() == TEXT("TrainingInputAnims") && Property->GetFName() == FMLDeformerTrainingInputAnim::GetStartFramePropertyName())
		{
			for (FMLDeformerGeomCacheTrainingInputAnim& InputAnim : GetGeomCacheModel()->GetTrainingInputAnims())
			{
				if (InputAnim.GetStartFrame() > InputAnim.GetEndFrame())
				{
					InputAnim.SetEndFrame(InputAnim.GetStartFrame());
				}
			}
			SetResamplingInputOutputsNeeded(true);
			TriggerInputAssetChanged(true);
		}
		else if (PropertyChangedEvent.GetMemberPropertyName() == TEXT("TrainingInputAnims") && Property->GetFName() == FMLDeformerTrainingInputAnim::GetEndFramePropertyName())
		{
			for (FMLDeformerGeomCacheTrainingInputAnim& InputAnim : GetGeomCacheModel()->GetTrainingInputAnims())
			{
				if (InputAnim.GetEndFrame() < InputAnim.GetStartFrame())
				{
					InputAnim.SetStartFrame(InputAnim.GetEndFrame());
				}
			}
			SetResamplingInputOutputsNeeded(true);
			TriggerInputAssetChanged(true);
		}
		else if (Property->GetFName() == UMLDeformerGeomCacheModel::GetTrainingInputAnimsPropertyName() || PropertyChangedEvent.GetMemberPropertyName() == TEXT("TrainingInputAnims"))
		{
			if (GetEditor() && GetEditor()->GetTimeSlider())
			{
				if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayRemove)
				{
					const int32 ArrayIndex = PropertyChangedEvent.GetArrayIndex(PropertyChangedEvent.GetMemberPropertyName().ToString());
					GetEditor()->GetTimeSlider()->OnDeletedTrainingInputAnim(ArrayIndex);
				}
				else if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayClear)
				{
					ActiveTrainingInputAnimIndex = INDEX_NONE;
				}
			}
			SetResamplingInputOutputsNeeded(true);
			TriggerInputAssetChanged(true);
		}
		else if (Property->GetFName() == UMLDeformerGeomCacheVizSettings::GetTestGroundTruthPropertyName())
		{
			TriggerInputAssetChanged(true);
		}
	}

	void FMLDeformerGeomCacheEditorModel::OnInputAssetsChanged()
	{
		// Update the skeletal mesh components of the training, test base, and ml deformed actor.
		FMLDeformerEditorModel::OnInputAssetsChanged();

		UMLDeformerGeomCacheVizSettings* VizSettings = GetGeomCacheVizSettings();
		check(VizSettings);
		const float TestAnimSpeed = VizSettings->GetAnimPlaySpeed();
		UAnimSequence* TestAnimSequence = VizSettings->GetTestAnimSequence();

		// Update the training geometry cache.
		const FMLDeformerGeomCacheActor* TrainGroundTruthActor = FindGeomCacheEditorActor(ActorID_Train_GroundTruth);
		UGeometryCacheComponent* GeometryCacheComponent = TrainGroundTruthActor ? TrainGroundTruthActor->GetGeometryCacheComponent() : nullptr;
		if (GeometryCacheComponent)
		{
			GeometryCacheComponent->SetGeometryCache(GetActiveGeometryCache());
			GeometryCacheComponent->SetLooping(true);
			GeometryCacheComponent->SetManualTick(true);
			GeometryCacheComponent->SetPlaybackSpeed(TestAnimSpeed);
			GeometryCacheComponent->Play();
		}

		// Update the test geometry cache (ground truth) component.
		const FMLDeformerGeomCacheActor* TestGroundTruthActor = FindGeomCacheEditorActor(ActorID_Test_GroundTruth);
		GeometryCacheComponent = TestGroundTruthActor ? TestGroundTruthActor->GetGeometryCacheComponent() : nullptr;
		if (GeometryCacheComponent)
		{
			GeometryCacheComponent->SetGeometryCache(VizSettings->GetTestGroundTruth());
			GeometryCacheComponent->SetLooping(true);
			GeometryCacheComponent->SetManualTick(true);
			GeometryCacheComponent->SetPlaybackSpeed(TestAnimSpeed);
			GeometryCacheComponent->Play();
		}

		GetGeomCacheModel()->GetGeomCacheMeshMappings().Reset();
	}

	void FMLDeformerGeomCacheEditorModel::OnObjectModified(UObject* Object)
	{
		// Handle changes for the skeletal mesh and anim sequence.
		FMLDeformerEditorModel::OnObjectModified(Object);

		const int32 NumAnims = GetGeomCacheModel()->GetTrainingInputAnims().Num();
		for (int32 AnimIndex = 0; AnimIndex < NumAnims; ++AnimIndex)
		{
			const FMLDeformerGeomCacheTrainingInputAnim& Anim = GetGeomCacheModel()->GetTrainingInputAnims()[AnimIndex];
			if (Anim.GetAnimSequence() == Object)
			{
				UE_LOGF(LogMLDeformer, Display, "Detected a modification in training anim sequence %ls, reinitializing inputs.", *Object->GetName());
				bNeedsAssetReinit = true;
			}

			if (Anim.GetGeometryCache() == Object)
			{
				UE_LOGF(LogMLDeformer, Display, "Detected a modification in training geometry cache %ls, reinitializing inputs.", *Object->GetName());
				bNeedsAssetReinit = true;
			}
		}

		// Check if it is the ground truth test geom cache.
		if (GetGeomCacheModel()->GetGeomCacheVizSettings()->GetTestGroundTruth() == Object)
		{
			UE_LOGF(LogMLDeformer, Display, "Detected a modification in test ground truth geometry cache %ls, reinitializing inputs.", *Object->GetName());
			bNeedsAssetReinit = true;
		}
	}

	int32 FMLDeformerGeomCacheEditorModel::GetNumTrainingInputAnims() const
	{
		return GetGeomCacheModel()->GetTrainingInputAnims().Num();
	}

	FMLDeformerTrainingInputAnim* FMLDeformerGeomCacheEditorModel::GetTrainingInputAnim(int32 Index) const
	{
		if (GetGeomCacheModel()->GetTrainingInputAnims().IsValidIndex(Index))
		{
			return &GetGeomCacheModel()->GetTrainingInputAnims()[Index];
		}
		
		return nullptr;
	}
}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
