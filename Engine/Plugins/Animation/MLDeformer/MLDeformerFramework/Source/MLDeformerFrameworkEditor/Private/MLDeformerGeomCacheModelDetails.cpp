// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerGeomCacheModelDetails.h"
#include "MLDeformerGeomCacheEditorModel.h"
#include "MLDeformerGeomCacheModel.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/Layout/SBox.h"
#include "SWarningOrErrorBox.h"

#define LOCTEXT_NAMESPACE "MLDeformerGeomCacheModelDetails"

namespace UE::MLDeformer
{
	bool FMLDeformerGeomCacheModelDetails::UpdateMemberPointers(const TArray<TWeakObjectPtr<UObject>>& Objects)
	{
		if (!FMLDeformerModelDetails::UpdateMemberPointers(Objects))
		{
			return false;
		}

		GeomCacheModel = Cast<UMLDeformerGeomCacheModel>(Model);
		check(GeomCacheModel);
		GeomCacheEditorModel = static_cast<FMLDeformerGeomCacheEditorModel*>(EditorModel);

		return (GeomCacheModel != nullptr && GeomCacheEditorModel != nullptr);
	}

	void FMLDeformerGeomCacheModelDetails::AddTrainingInputAnims()
	{
		InputOutputCategoryBuilder->AddProperty(UMLDeformerGeomCacheModel::GetTrainingInputAnimsPropertyName(), UMLDeformerGeomCacheModel::StaticClass())
			.DisplayName( FText::Format(LOCTEXT("TrainingInputAnimsString", "Training Input Anims ({0} Frames)"), EditorModel->GetNumTrainingFrames()) );
	}
}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
