// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerGeomCacheSampler.h"
#include "MLDeformerModel.h"
#include "MLDeformerEditorModel.h"
#include "MLDeformerInputInfo.h"
#include "MLDeformerEditorToolkit.h"
#include "MLDeformerGeomCacheHelpers.h"
#include "MLDeformerGeomCacheModel.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "GeometryCacheComponent.h"
#include "GeometryCache.h"
#include "GeometryCacheMeshData.h"
#include "GeometryCacheTrack.h"
#include "SkeletalMeshAttributes.h"
#include "Async/ParallelFor.h"

namespace UE::MLDeformer
{
	void FMLDeformerGeomCacheSampler::Init(FMLDeformerEditorModel* InModel, int32 InAnimIndex)
	{
		FMLDeformerSampler::Init(InModel, InAnimIndex);

		if (!TargetMeshActor)
		{
			return;
		}

		// Create the geometry cache component.
		if (GeometryCacheComponent.Get() == nullptr)
		{
			GeometryCacheComponent = NewObject<UGeometryCacheComponent>(TargetMeshActor);
			GeometryCacheComponent->RegisterComponent();
			TargetMeshActor->SetRootComponent(GeometryCacheComponent);
		}

		UMLDeformerGeomCacheModel* GeomCacheModel = Cast<UMLDeformerGeomCacheModel>(InModel->GetModel());
		check(GeomCacheModel);

		FMLDeformerGeomCacheTrainingInputAnim* GeomCacheAnim = static_cast<FMLDeformerGeomCacheTrainingInputAnim*>(EditorModel->GetTrainingInputAnim(InAnimIndex));
		UGeometryCache* GeomCache = GeomCacheAnim->GetGeometryCache();
		GeometryCacheComponent->SetGeometryCache(GeomCache);
		GeometryCacheComponent->SetManualTick(true);
		GeometryCacheComponent->SetVisibility(false);

		// Generate mappings between the meshes in the SkeletalMesh and the geometry cache tracks.
		FailedImportedMeshNames.Reset();
		MeshMappings.Reset();
		GenerateGeomCacheMeshMappings(
			Model->GetSkeletalMesh(), 
			GeomCache, 
			MeshMappings, 
			FailedImportedMeshNames,
			VertexCountMisMatchNames);

		GeomCacheMeshDatas.Reset(MeshMappings.Num());
		GeomCacheMeshDatas.AddDefaulted(MeshMappings.Num());
	}

	void FMLDeformerGeomCacheSampler::Sample(int32 InAnimFrameIndex)
	{
		// Call this first to update bone and curve values.
		// This will also calculate the skinned positions if the delta space is set to PostSkinning.
		FMLDeformerSampler::Sample(InAnimFrameIndex);
		
		const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent.Get() ? SkeletalMeshComponent->GetSkeletalMeshAsset() : nullptr;
		UGeometryCache* GeometryCache = GeometryCacheComponent->GetGeometryCache();
		if (SkeletalMeshComponent && SkeletalMesh && GeometryCacheComponent && GeometryCache)
		{
			const FTransform& AlignmentTransform = Model->GetAlignmentTransform();

			// For all mesh mappings we found.
			constexpr int32 LODIndex = 0;
			const FMeshDescription* MeshDescription = SkeletalMesh->GetMeshDescription(LODIndex);
			const FSkeletalMeshConstAttributes MeshAttributes(*MeshDescription);
			const FSkeletalMeshAttributes::FSourceGeometryPartVertexOffsetAndCountConstRef GeoPartOffsetAndCounts = MeshAttributes.GetSourceGeometryPartVertexOffsetAndCounts();
			
			for (int32 MeshMappingIndex = 0; MeshMappingIndex < MeshMappings.Num(); ++MeshMappingIndex)
			{
				const FMLDeformerGeomCacheMeshMapping& MeshMapping = MeshMappings[MeshMappingIndex];
				TArrayView<const int32> GeoPartInfo = GeoPartOffsetAndCounts.Get(MeshMapping.MeshIndex);
				const int32 StartImportedVertex = GeoPartInfo[0];
				const int32 NumVertices = GeoPartInfo[1];
				
				UGeometryCacheTrack* Track = GeometryCache->Tracks[MeshMapping.TrackIndex];

				// Sample the mesh data of the geom cache.
				FGeometryCacheMeshData& GeomCacheMeshData = GeomCacheMeshDatas[MeshMappingIndex];
				if (!Track->GetMeshDataAtSampleIndex(InAnimFrameIndex, GeomCacheMeshData))
				{
					continue;
				}

				// Calculate the vertex deltas.
				const FSkeletalMeshLODRenderData& SkelMeshLODData = SkeletalMesh->GetResourceForRendering()->LODRenderData[LODIndex];
				const FSkinWeightVertexBuffer& SkinWeightBuffer = *SkeletalMeshComponent->GetSkinWeightBuffer(LODIndex);

				ParallelFor(TEXT("GeomCacheSampler_Sample"), NumVertices, 250, [&](int32 VertexIndex)
				{
					const int32 SkinnedVertexIndex = StartImportedVertex + VertexIndex;
					const int32 GeomCacheVertexIndex = MeshMapping.SkelMeshToTrackVertexMap[VertexIndex];
					if (GeomCacheVertexIndex != INDEX_NONE && GeomCacheMeshData.Positions.IsValidIndex(GeomCacheVertexIndex))
					{
						const int32 RenderVertexIndex = MeshMapping.ImportedVertexToRenderVertexMap[VertexIndex];
						if (RenderVertexIndex == INDEX_NONE)
						{
							return;
						}

						// Calculate the pre-skinning delta.
						FVector3f Delta = FVector3f::ZeroVector;
						if (VertexDeltaSpace == EVertexDeltaSpace::PreSkinning)
						{
							if (SkinningMode == EMLDeformerSkinningMode::Linear)
							{
								// Calculate the inverse skinning transform for this vertex.
								const FMatrix44f InvSkinningTransform = CalcInverseSkinningTransform(RenderVertexIndex, SkelMeshLODData, SkinWeightBuffer);

								// Calculate the pre-skinning data.
								const FSkeletalMeshLODRenderData& LODData = SkeletalMesh->GetResourceForRendering()->LODRenderData[0];
								const FVector3f UnskinnedPosition = LODData.StaticVertexBuffers.PositionVertexBuffer.VertexPosition(RenderVertexIndex);
								const FVector3f GeomCacheVertexPos = (FVector3f)AlignmentTransform.TransformPosition((FVector)GeomCacheMeshData.Positions[GeomCacheVertexIndex]);
								const FVector3f PreSkinningTargetPos = InvSkinningTransform.TransformPosition(GeomCacheVertexPos);
								Delta = PreSkinningTargetPos - UnskinnedPosition;
							}
							else // Dual Quaternion Skinning.
							{
								const FVector3f SkinnedVertexPos = SkinnedVertexPositions[SkinnedVertexIndex];
								const FVector3f GeomCacheVertexPos = (FVector3f)AlignmentTransform.TransformPosition((FVector)GeomCacheMeshData.Positions[GeomCacheVertexIndex]);
								const FVector3f WorldDelta = GeomCacheVertexPos - SkinnedVertexPos;
								const FSkeletalMeshLODRenderData& LODData = SkeletalMesh->GetResourceForRendering()->LODRenderData[0];
								Delta = CalcDualQuaternionDelta(RenderVertexIndex, WorldDelta, LODData, SkinWeightBuffer);
							}
						}
						else if (VertexDeltaSpace == EVertexDeltaSpace::PostSkinning)
						{
							const FVector3f SkinnedVertexPos = SkinnedVertexPositions[SkinnedVertexIndex];
							const FVector3f GeomCacheVertexPos = (FVector3f)AlignmentTransform.TransformPosition((FVector)GeomCacheMeshData.Positions[GeomCacheVertexIndex]);
							Delta = GeomCacheVertexPos - SkinnedVertexPos;
						}

						const int32 ArrayIndex = 3 * SkinnedVertexIndex;
						VertexDeltas[ArrayIndex + 0] = Delta.X;
						VertexDeltas[ArrayIndex + 1] = Delta.Y;
						VertexDeltas[ArrayIndex + 2] = Delta.Z;
					}
				});	// ParallelFor
			}
		}
		else
		{
			VertexDeltas.Reset();
		}
	}

	float FMLDeformerGeomCacheSampler::GetTimeAtFrame(int32 InAnimFrameIndex) const
	{
		// the animation instance must drive the time update
		if (SkeletalMeshComponent)
		{
			UAnimSingleNodeInstance* SingleNodeInstance = Cast<UAnimSingleNodeInstance>(SkeletalMeshComponent->GetAnimInstance());
			if (SingleNodeInstance)
			{
				const UAnimSequenceBase* SequenceBase = Cast<UAnimSequenceBase>(SingleNodeInstance->CurrentAsset);
				if (SequenceBase)
				{
					const FFrameRate FrameRate = SequenceBase->GetSamplingFrameRate();
					const float UncorrectedTime = SequenceBase->GetTimeAtFrame(InAnimFrameIndex);
					const FFrameTime PlayOffsetCurrentTime(FrameRate.AsFrameTime(UncorrectedTime));
					return FMLDeformerEditorModel::CorrectedFrameTime(InAnimFrameIndex, UncorrectedTime, FrameRate);
				}
			}
		}

		if (GeometryCacheComponent.Get())
		{
			return GeometryCacheComponent->GetTimeAtFrame(InAnimFrameIndex);
		}
		return 0.0f;
	}
}	// namespace UE::MLDeformer
