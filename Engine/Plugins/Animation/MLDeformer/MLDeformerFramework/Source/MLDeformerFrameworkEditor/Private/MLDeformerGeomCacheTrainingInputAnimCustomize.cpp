// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerGeomCacheTrainingInputAnimCustomize.h"
#include "MLDeformerGeomCacheTrainingInputAnim.h"
#include "MLDeformerGeomCacheModel.h"
#include "MLDeformerGeomCacheHelpers.h"
#include "MLDeformerEditorModel.h"
#include "MLDeformerEditorModule.h"
#include "Animation/Skeleton.h"
#include "PropertyCustomizationHelpers.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SComboBox.h"
#include "SWarningOrErrorBox.h"
#include "DetailWidgetRow.h"
#include "Engine/SkeletalMesh.h"

#define LOCTEXT_NAMESPACE "MLDeformerGeomCacheTrainingInputAnimCustomize"

namespace UE::MLDeformer
{
	TSharedRef<IPropertyTypeCustomization> FMLDeformerGeomCacheTrainingInputAnimCustomization::MakeInstance()
	{
		return MakeShareable(new FMLDeformerGeomCacheTrainingInputAnimCustomization());
	}

	void FMLDeformerGeomCacheTrainingInputAnimCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		const int32 ArrayIndex = StructPropertyHandle->GetIndexInArray();
		check(ArrayIndex != INDEX_NONE);

		UMLDeformerGeomCacheModel* Model = FindMLDeformerModel(StructPropertyHandle);
		check(Model);
		int32 NumAnimFrames = 0;
		FMLDeformerGeomCacheTrainingInputAnim& InputAnim = Model->GetTrainingInputAnims()[ArrayIndex];
		const UGeometryCache* GeomCache = InputAnim.GetGeometryCache();
		if (GeomCache)
		{
			NumAnimFrames = GeomCache->GetEndFrame() - GeomCache->GetStartFrame() + 1;
		}
		const int32 NumFramesToSample = InputAnim.GetNumFramesToSample();

		HeaderRow
		.NameContent()
		.VAlign(EVerticalAlignment::VAlign_Center)
		.HAlign(EHorizontalAlignment::HAlign_Left)
		[
			SNew(STextBlock)
			.Text(FText::Format(LOCTEXT("HeaderTextFmt", "Animation #{0}"), ArrayIndex))
			.Font(StructCustomizationUtils.GetRegularFont())
			.ColorAndOpacity(InputAnim.IsEnabled() ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground())
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)	// Required to work around a text alignment issue. Otherwise the text block will not center align.
			+SHorizontalBox::Slot()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.HAlign(EHorizontalAlignment::HAlign_Left)
			[
				SNew(STextBlock)
				.Text(FText::Format(LOCTEXT("HeaderTextValue", "Sample {0} / {1} Frames"), NumFramesToSample, NumAnimFrames))
				.Font(StructCustomizationUtils.GetRegularFont())
				.ColorAndOpacity(InputAnim.IsEnabled() ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground())
			]
		];
	}

	void FMLDeformerGeomCacheTrainingInputAnimCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		// Get all the child properties.
		uint32 NumChildren;
		StructPropertyHandle->GetNumChildren(NumChildren);
		TMap<FName, TSharedPtr<IPropertyHandle>> PropertyHandles;
		PropertyHandles.Reserve(NumChildren);
		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
			const FName PropertyName = ChildHandle->GetProperty()->GetFName();
			PropertyHandles.Add(PropertyName, ChildHandle);
		}

		const int32 ArrayIndex = StructPropertyHandle->GetIndexInArray();
		check(ArrayIndex != INDEX_NONE);

		TSharedPtr<IPropertyHandle> AnimSequenceHandle		= PropertyHandles.FindChecked(FMLDeformerGeomCacheTrainingInputAnim::GetAnimSequencePropertyName());
		TSharedPtr<IPropertyHandle> GeomCacheHandle			= PropertyHandles.FindChecked(FMLDeformerGeomCacheTrainingInputAnim::GetGeomCachePropertyName());
		TSharedPtr<IPropertyHandle> EnabledHandle			= PropertyHandles.FindChecked(FMLDeformerGeomCacheTrainingInputAnim::GetEnabledPropertyName());
		TSharedPtr<IPropertyHandle> UseCustomRangeHandle	= PropertyHandles.FindChecked(FMLDeformerGeomCacheTrainingInputAnim::GetUseCustomRangePropertyName());
		TSharedPtr<IPropertyHandle> StartFrameHandle		= PropertyHandles.FindChecked(FMLDeformerGeomCacheTrainingInputAnim::GetStartFramePropertyName());
		TSharedPtr<IPropertyHandle> EndFrameHandle			= PropertyHandles.FindChecked(FMLDeformerGeomCacheTrainingInputAnim::GetEndFramePropertyName());
		TSharedPtr<IPropertyHandle> VertexMapHandle			= PropertyHandles.FindChecked(FMLDeformerGeomCacheTrainingInputAnim::GetVertexMaskPropertyName());
		check(AnimSequenceHandle.IsValid());
		check(GeomCacheHandle.IsValid());
		check(EnabledHandle.IsValid());
		check(UseCustomRangeHandle.IsValid());
		check(StartFrameHandle.IsValid());
		check(EndFrameHandle.IsValid());
		check(VertexMapHandle.IsValid());

		UMLDeformerGeomCacheModel* Model = FindMLDeformerModel(StructPropertyHandle);
		check(Model);
		FMLDeformerGeomCacheTrainingInputAnim& InputAnim = Model->GetTrainingInputAnims()[ArrayIndex];
		UAnimSequence* AnimSequence = InputAnim.GetAnimSequence();
		UGeometryCache* GeomCache = InputAnim.GetGeometryCache();

		USkeleton* Skeleton = nullptr;
		FMLDeformerEditorModel* EditorModel = nullptr;
		if (Model->GetSkeletalMesh())
		{
			Skeleton = Model->GetSkeletalMesh()->GetSkeleton();
		}

		// Get the editor model for this runtime model.
		FMLDeformerEditorModule& EditorModule = FModuleManager::LoadModuleChecked<FMLDeformerEditorModule>("MLDeformerFrameworkEditor");
		EditorModel = EditorModule.GetModelRegistry().GetEditorModel(Model);

		// Add the properties
		IDetailPropertyRow& AnimRow = ChildBuilder.AddProperty(AnimSequenceHandle.ToSharedRef());
		AnimRow.EditCondition(InputAnim.IsEnabled(), nullptr);
		AnimRow.CustomWidget()
		.NameContent()
		[
			AnimRow.GetPropertyHandle()->CreatePropertyNameWidget()					
		]
		.ValueContent()
		[
			SNew(SObjectPropertyEntryBox)
			.PropertyHandle(AnimRow.GetPropertyHandle())
			.AllowedClass(UAnimSequence::StaticClass())
			.ObjectPath(AnimSequence ? AnimSequence->GetPathName() : FString())
			.ThumbnailPool(StructCustomizationUtils.GetThumbnailPool())
			.OnShouldFilterAsset
			(
				this, 
				&FMLDeformerGeomCacheTrainingInputAnimCustomization::FilterAnimSequences, 
				Skeleton
			)
		];

		ChildBuilder.AddProperty(GeomCacheHandle.ToSharedRef())
			.EditCondition(InputAnim.IsEnabled(), nullptr);

		ChildBuilder.AddProperty(UseCustomRangeHandle.ToSharedRef())
			.EditCondition(InputAnim.IsEnabled(), nullptr);

		ChildBuilder.AddProperty(StartFrameHandle.ToSharedRef())
			.EditCondition(InputAnim.IsEnabled() && InputAnim.GetUseCustomRange(), nullptr);

		ChildBuilder.AddProperty(EndFrameHandle.ToSharedRef())
			.EditCondition(InputAnim.IsEnabled() && InputAnim.GetUseCustomRange(), nullptr);

		// Vertex Map property.
		if (EditorModel->GetSupportsPerTrainingInputAnimVertexMask())
		{
			EditorModel->UpdateVertexAttributeNames();
			IDetailPropertyRow& VertexMapRow = ChildBuilder.AddProperty(VertexMapHandle.ToSharedRef());
			VertexMapRow.EditCondition(InputAnim.IsEnabled(), nullptr);
			VertexMapRow.CustomWidget()
			.NameContent()
			[
				VertexMapRow.GetPropertyHandle()->CreatePropertyNameWidget()					
			]
			.ValueContent()
			[
				SNew(SComboBox<TSharedPtr<FName>>)
				.OptionsSource(&EditorModel->GetVertexAttributeNames())
				.OnGenerateWidget_Lambda
				(
					[&StructCustomizationUtils](const TSharedPtr<FName> Item)
					{
						return SNew(STextBlock)
							.Text(Item.IsValid() ? FText::FromName(*Item) : FText())
							.Font(StructCustomizationUtils.GetRegularFont());
					}
				)
				.OnSelectionChanged_Lambda(
					[&InputAnim, EditorModel](TSharedPtr<FName> Item, ESelectInfo::Type)
					{
						const auto& VertexAttribNames = EditorModel->GetVertexAttributeNames();
						EditorModel->SetResamplingInputOutputsNeeded(true);
						if (!VertexAttribNames.IsEmpty() && Item == VertexAttribNames[0])
						{
							InputAnim.SetVertexMask(FName());
						}
						else
						{
							InputAnim.SetVertexMask(Item.IsValid() ? *Item : FName("<unknown>"));
						}
					}
				)
				.Content()
				[
					SNew(STextBlock)
					.Text_Lambda(
						[&InputAnim]()
						{
							return !InputAnim.GetVertexMask().IsNone() ? FText::FromName(InputAnim.GetVertexMask()) : LOCTEXT("DisabledString", "Disabled");
						}
					)
					.ColorAndOpacity_Lambda(
						[&InputAnim, EditorModel]()
						{
							const FName SelectedMask = InputAnim.GetVertexMask();
							const TSharedPtr<FName>* ItemFound = EditorModel->GetVertexAttributeNames().FindByPredicate(
								[&SelectedMask](const TSharedPtr<FName>& Item)
								{
									return (Item.IsValid() && *Item == SelectedMask);				
								});

							return ItemFound || SelectedMask.IsNone() ? FSlateColor::UseForeground() : FLinearColor::Red;
						}
					)
					.Font(StructCustomizationUtils.GetRegularFont())
				]
			];
		}

		ChildBuilder.AddProperty(EnabledHandle.ToSharedRef());

		// Verify the geometry cache against the anim sequence.
		const FText GeomCacheErrorText = GetGeomCacheAnimSequenceErrorText(GeomCache, AnimSequence);
		ChildBuilder.AddCustomRow(FText::FromString("AnimSeqWarning"))
			.Visibility(!GeomCacheErrorText.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed)
			.WholeRowContent()
			[
				SNew(SBox)
				.Padding(FMargin(0.0f, 4.0f))
				[
					SNew(SWarningOrErrorBox)
					.MessageStyle(EMessageStyle::Warning)
					.Message(GeomCacheErrorText)
				]
			];

		// Show some additional geometry cache errors.
		USkeletalMesh* SkelMesh = Model->GetSkeletalMesh();
		const FText TargetMeshErrorText = GetGeomCacheErrorText(SkelMesh, GeomCache);
		ChildBuilder.AddCustomRow(FText::FromString("TargetMeshError"))
			.Visibility(!TargetMeshErrorText.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed)
			.WholeRowContent()
			[
				SNew(SBox)
				.Padding(FMargin(0.0f, 4.0f))
				[
					SNew(SWarningOrErrorBox)
					.MessageStyle(EMessageStyle::Error)
					.Message(TargetMeshErrorText)
				]
			];

		// Check if our vertex counts changed versus the trained model.
		if (EditorModel)
		{
			const FText ChangedErrorText = EditorModel->GetTargetAssetChangedErrorText();
			ChildBuilder.AddCustomRow(FText::FromString("TargetMeshChangedError"))
				.Visibility(!ChangedErrorText.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed)
				.WholeRowContent()
				[
					SNew(SBox)
					.Padding(FMargin(0.0f, 4.0f))
					[
						SNew(SWarningOrErrorBox)
						.MessageStyle(EMessageStyle::Warning)
						.Message(ChangedErrorText)
					]
				];
		}

		// Check if the skeleton is incompatible or not.
		const FText SkeletonError = EditorModel->GetIncompatibleSkeletonErrorText(Model->GetSkeletalMesh(), InputAnim.GetAnimSequence());
		ChildBuilder.AddCustomRow(FText::FromString("AnimSkeletonMisMatchError"))
			.Visibility(!SkeletonError.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed)
			.WholeRowContent()
			[
				SNew(SBox)
				.Padding(FMargin(0.0f, 4.0f))
				[
					SNew(SWarningOrErrorBox)
					.MessageStyle(EMessageStyle::Warning)
					.Message(SkeletonError)
				]
			];

		// Show geometry cache mapping errors, which maps geom cache tracks to FBX meshes.
		const FText MappingWarningText = GetGeomCacheMeshMappingErrorText(SkelMesh, GeomCache);
		ChildBuilder.AddCustomRow(FText::FromString("MeshMappingWarning"))
			.Visibility(!MappingWarningText.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed)
			.WholeRowContent()
			[
				SNew(SBox)
				.Padding(FMargin(0.0f, 4.0f))
				[
					SNew(SWarningOrErrorBox)
					.MessageStyle(EMessageStyle::Warning)
					.Message(MappingWarningText)
				]
			];
	}

	bool FMLDeformerGeomCacheTrainingInputAnimCustomization::FilterAnimSequences(const FAssetData& AssetData, USkeleton* Skeleton)
	{
		if (Skeleton && Skeleton->IsCompatibleForEditor(AssetData))
		{
			return false;
		}

		return true;
	}

	UMLDeformerGeomCacheModel* FMLDeformerGeomCacheTrainingInputAnimCustomization::FindMLDeformerModel(TSharedRef<IPropertyHandle> StructPropertyHandle) const
	{
		TArray<UObject*> Objects;
		StructPropertyHandle->GetOuterObjects(Objects);
		UObject* OwnerObject = !Objects.IsEmpty() ? Objects[0] : nullptr;
		UMLDeformerGeomCacheModel* Model = Cast<UMLDeformerGeomCacheModel>(OwnerObject);
		return Model;
	}
}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
