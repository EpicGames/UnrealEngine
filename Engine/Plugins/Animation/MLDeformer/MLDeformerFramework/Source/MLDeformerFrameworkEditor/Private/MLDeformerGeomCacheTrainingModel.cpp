// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerGeomCacheTrainingModel.h"
#include "MLDeformerGeomCacheTrainingInputAnim.h"
#include "MLDeformerGeomCacheEditorModel.h"
#include "MLDeformerGeomCacheModel.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopedSlowTask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MLDeformerGeomCacheTrainingModel)

#define LOCTEXT_NAMESPACE "MLDeformerGeomCacheTrainingModel"

void UMLDeformerGeomCacheTrainingModel::Init(UE::MLDeformer::FMLDeformerEditorModel* InEditorModel)
{
	Super::Init(InEditorModel);

	// Find the first valid input anim index to sample from.
	// This modifies the SampleAnimIndex value.
	bFinishedSampling = !FindNextAnimToSample(SampleAnimIndex);
}

bool UMLDeformerGeomCacheTrainingModel::FindNextAnimToSample(int32& OutNextAnimIndex) const
{
	using namespace UE::MLDeformer;

	int32 NumTries = 0;
	int32 AnimIndex = SampleAnimIndex;

	const int32 NumInputAnims = EditorModel->GetNumTrainingInputAnims();
	while (NumTries < NumInputAnims) // Try all input animations at worst case.
	{
		const FMLDeformerTrainingInputAnim* InputAnim = EditorModel->GetTrainingInputAnim(AnimIndex);
		if (InputAnim && InputAnim->IsEnabled())
		{
			if (NumTimesSampled[AnimIndex] < InputAnim->GetNumFramesToSample())
			{
				OutNextAnimIndex = AnimIndex;
				return true;
			}
		}

		// Get the next animation index.
		AnimIndex++;
		AnimIndex %= EditorModel->GetNumTrainingInputAnims();

		NumTries++;
	}

	OutNextAnimIndex = INDEX_NONE;
	return false;
}

bool UMLDeformerGeomCacheTrainingModel::SampleNextFrame()
{
	using namespace UE::MLDeformer;

	// Make sure that there is more left to sample.
	if (bFinishedSampling)
	{
		return false;
	}

	UMLDeformerGeomCacheModel* GeomCacheModel = Cast<UMLDeformerGeomCacheModel>(EditorModel->GetModel());

	// Get the animation to sample from and validate some things.
	const FMLDeformerGeomCacheTrainingInputAnim& InputAnim = GeomCacheModel->GetTrainingInputAnims()[SampleAnimIndex];
	check(InputAnim.IsEnabled());
	check(InputAnim.GetAnimSequence());
	check(InputAnim.GetGeometryCache());

	const int32 StartFrame = InputAnim.GetUseCustomRange() ? FMath::Min<int32>(InputAnim.GetStartFrame(), InputAnim.GetEndFrame()) : 0;
	const int32 CurFrameToSample = StartFrame + NumTimesSampled[SampleAnimIndex];
	check(CurFrameToSample < StartFrame + InputAnim.GetNumFramesToSample()); // We should never sample more frames than the animation has.
	NumTimesSampled[SampleAnimIndex]++;

	// Perform the actual sampling.
	FMLDeformerSampler* Sampler = EditorModel->GetSamplerForTrainingAnim(SampleAnimIndex);
	Sampler->SetVertexDeltaSpace(EVertexDeltaSpace::PreSkinning);
	Sampler->Sample(CurFrameToSample);
	MaskIndexPerSample.Add(GetMaskIndexForAnimIndex(SampleAnimIndex));
	UE_LOGF(LogMLDeformer, Verbose, "Sampling frame %d of anim %d", CurFrameToSample, SampleAnimIndex);

	// Copy sampled values.
	SampleDeltas = Sampler->GetVertexDeltas();
	SampleBoneRotations = Sampler->GetBoneRotations();
	SampleCurveValues = Sampler->GetCurveValues();

	// Now find the next sample we should take.
	// This will return false when we finished sampling everything.
	SampleAnimIndex++;
	SampleAnimIndex %= GeomCacheModel->GetTrainingInputAnims().Num();
	if (!FindNextAnimToSample(SampleAnimIndex))
	{
		bFinishedSampling = true;
		return false;
	}

	// Let the caller know we can still sample more.
	return true;
}

namespace
{
	FString ConvertSecondsToTimeString(double TotalSeconds)
	{
		// Convert total seconds to hours, minutes, and seconds
		const int32 Hours = FMath::FloorToInt(TotalSeconds / 3600);
		TotalSeconds -= Hours * 3600;
		const int32 Minutes = FMath::FloorToInt(TotalSeconds / 60);
		const int32 Seconds = FMath::FloorToInt(TotalSeconds - (Minutes * 60));

		// Format the string with leading zeros
		return FString::Printf(TEXT("%02d:%02d:%02d"), Hours, Minutes, Seconds);
	}
}

bool UMLDeformerGeomCacheTrainingModel::GenerateBasicInputsAndOutputBuffers(const FString& InputsFilePath, const FString& OutputsFilePath)
{
	// Create the task window and show it.
	const int32 NumFrames = NumSamples();
	FScopedSlowTask Task(NumFrames + 1, LOCTEXT("SamplingTaskTitle", "Sampling frames"));
	Task.MakeDialog(true);

	// Create our buffer files that store the sampled inputs and outputs of all frames.
	IFileManager& FileManager = IFileManager::Get();
	const TUniquePtr<FArchive> InputsArchive = TUniquePtr<FArchive>(FileManager.CreateFileWriter(*InputsFilePath));
	const TUniquePtr<FArchive> OutputArchive = TUniquePtr<FArchive>(FileManager.CreateFileWriter(*OutputsFilePath));

	// Make sure we free the data whenever we leave this function.
	ON_SCOPE_EXIT
	{
		InputsArchive->Close();
		OutputArchive->Close();
	};

	// Make sure the archives are valid.
	if (!InputsArchive)
	{
		UE_LOGF(LogMLDeformer, Error,
		       "UMLDeformerGeomCacheTrainingModel::GenerateBasicInputsAndOutputBuffers() - Failed to create inputs buffer file %ls",
		       *InputsFilePath);
		return false;
	}
	
	if (!OutputArchive)
	{
		UE_LOGF(LogMLDeformer, Error,
		       "UMLDeformerGeomCacheTrainingModel::GenerateBasicInputsAndOutputBuffers() - Failed to create output buffer file %ls",
		       *OutputsFilePath);
		return false;
	}

	// Update the number of floats per curve.
	SetNumFloatsPerCurve(GetModel()->GetNumFloatsPerCurve());
	
	// Sample all the frames.
	const double StartTime = FPlatformTime::Seconds();
	for (int32 SampleIndex = 0; SampleIndex < NumFrames; SampleIndex++)
	{
		if (Task.ShouldCancel())
		{
			return false;
		}

		// Sample the next frame, which fills the SampleBoneRotations, SampleCurveValues and SampleDeltas arrays.
		NextSample();

		// Write the sampled inputs.
		InputsArchive->Serialize(SampleBoneRotations.GetData(), SampleBoneRotations.Num() * sizeof(float));
		if (InputsArchive->IsError() || InputsArchive->IsCriticalError())
		{
			UE_LOGF(LogMLDeformer, Error, "Failed to write sampled input data!");
			return false;
		}
		InputsArchive->Serialize(SampleCurveValues.GetData(), SampleCurveValues.Num() * sizeof(float));
		if (InputsArchive->IsError() || InputsArchive->IsCriticalError())
		{
			UE_LOGF(LogMLDeformer, Error, "Failed to write sampled input data!");
			return false;
		}

		// Write the sampled outputs.
		OutputArchive->Serialize(SampleDeltas.GetData(), SampleDeltas.Num() * sizeof(float));
		if (OutputArchive->IsError() || OutputArchive->IsCriticalError())
		{
			UE_LOGF(LogMLDeformer, Error, "Failed to write sampled output data!");
			return false;
		}

		// Calculate remaining time.
		const double ElapsedTime = FPlatformTime::Seconds() - StartTime;
		const int32 SamplesRemaining = NumFrames - SampleIndex;
		const double AverageTimePerSample = ElapsedTime / (SampleIndex + 1);
		const double RemainingSeconds = SamplesRemaining * AverageTimePerSample;
		const FString RemainingTimeString = ConvertSecondsToTimeString(RemainingSeconds);
		const FString ElapsedTimeString = ConvertSecondsToTimeString(ElapsedTime);

		// Update the progress in the task window.
		const FText ProgressText = FText::Format(LOCTEXT("SamplingProgressText", "Sampling frame {0} of {1} - Elapsed: {2} - Remaining: {3}"),
		                                         SampleIndex + 1,
		                                         NumFrames,
		                                         FText::FromString(ElapsedTimeString),
		                                         FText::FromString(RemainingTimeString));
		Task.EnterProgressFrame(1.0f, ProgressText);
	}

	UE_LOGF(LogMLDeformer, Display, "Sampling finished in %.0f seconds", FPlatformTime::Seconds() - StartTime);
	return true;
}

#undef LOCTEXT_NAMESPACE
