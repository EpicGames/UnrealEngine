// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerGeomCacheVizSettingsDetails.h"
#include "MLDeformerGeomCacheModel.h"
#include "MLDeformerGeomCacheEditorModel.h"
#include "MLDeformerEditorModule.h"
#include "MLDeformerGeomCacheVizSettings.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SBox.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "SWarningOrErrorBox.h"

#define LOCTEXT_NAMESPACE "MLDeformerGeomCacheVizSettingsDetails"

namespace UE::MLDeformer
{
	bool FMLDeformerGeomCacheVizSettingsDetails::UpdateMemberPointers(const TArray<TWeakObjectPtr<UObject>>& Objects)
	{
		if (!FMLDeformerVizSettingsDetails::UpdateMemberPointers(Objects))
		{
			return false;
		}

		GeomCacheModel = Cast<UMLDeformerGeomCacheModel>(Model);
		GeomCacheVizSettings = Cast<UMLDeformerGeomCacheVizSettings>(VizSettings);
		return (GeomCacheModel != nullptr && GeomCacheVizSettings != nullptr);
	}

	void FMLDeformerGeomCacheVizSettingsDetails::AddGroundTruth()
	{
		TestAssetsCategory->AddProperty(UMLDeformerGeomCacheVizSettings::GetTestGroundTruthPropertyName(), UMLDeformerGeomCacheVizSettings::StaticClass());

		// Show an error when the test anim sequence duration doesn't match the one of the ground truth.
		const FText AnimErrorText = GetGeomCacheAnimSequenceErrorText(GeomCacheVizSettings->GetTestGroundTruth(), VizSettings->GetTestAnimSequence());
		FDetailWidgetRow& GroundTruthAnimErrorRow = TestAssetsCategory->AddCustomRow(FText::FromString("GroundTruthAnimMismatchError"))
			.Visibility(!AnimErrorText.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed)
			.WholeRowContent()
			[
				SNew(SBox)
				.Padding(FMargin(0.0f, 4.0f))
				[
					SNew(SWarningOrErrorBox)
					.MessageStyle(EMessageStyle::Warning)
					.Message(AnimErrorText)
				]
			];

		const FText GeomErrorText = GetGeomCacheErrorText(Model->GetSkeletalMesh(), GeomCacheVizSettings->GetTestGroundTruth());
		FDetailWidgetRow& GroundTruthGeomErrorRow = TestAssetsCategory->AddCustomRow(FText::FromString("GroundTruthGeomMismatchError"))
			.Visibility(!GeomErrorText.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed)
			.WholeRowContent()
			[
				SNew(SBox)
				.Padding(FMargin(0.0f, 4.0f))
				[
					SNew(SWarningOrErrorBox)
					.MessageStyle(EMessageStyle::Warning)
					.Message(GeomErrorText)
				]
			];
	}
}	//namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
