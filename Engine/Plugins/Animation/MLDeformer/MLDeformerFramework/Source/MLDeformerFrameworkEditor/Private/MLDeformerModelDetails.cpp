// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerModelDetails.h"
#include "Animation/Skeleton.h"
#include "MLDeformerModule.h"
#include "MLDeformerAsset.h"
#include "MLDeformerModel.h"
#include "MLDeformerInputInfo.h"
#include "MLDeformerEditorModel.h"
#include "MLDeformerEditorModule.h"
#include "MLDeformerGeomCacheHelpers.h"
#include "MLDeformerTrainingInputAnim.h"
#include "GeometryCache.h"
#include "GeometryCacheTrack.h"
#include "Animation/AnimSequence.h"
#include "Engine/SkeletalMesh.h"
#include "PropertyCustomizationHelpers.h"
#include "AssetRegistry/AssetData.h"
#include "Modules/ModuleManager.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailsView.h"
#include "IDetailGroup.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "DetailLayoutBuilder.h"
#include "SWarningOrErrorBox.h"
#include "Widgets/Text/STextBlock.h"
#include "SMLDeformerInputWidget.h"

#define LOCTEXT_NAMESPACE "MLDeformerModelDetails"

namespace UE::MLDeformer
{
	bool FMLDeformerModelDetails::UpdateMemberPointers(const TArray<TWeakObjectPtr<UObject>>& Objects)
	{
		Model = nullptr;
		EditorModel = nullptr;
		if (Objects.Num() == 1 && Objects[0] != nullptr)
		{
			Model = Cast<UMLDeformerModel>(Objects[0].Get());

			// Get the editor model for this runtime model.
			FMLDeformerEditorModule& EditorModule = FModuleManager::LoadModuleChecked<FMLDeformerEditorModule>("MLDeformerFrameworkEditor");
			if (Model)
			{
				EditorModel = EditorModule.GetModelRegistry().GetEditorModel(Model);
			}
		}

		return (Model != nullptr && EditorModel != nullptr);
	}

	void FMLDeformerModelDetails::CreateCategories()
	{
		BaseMeshCategoryBuilder = &DetailLayoutBuilder->EditCategory("Base Mesh", FText::GetEmpty(), ECategoryPriority::Important);
		InputOutputCategoryBuilder = &DetailLayoutBuilder->EditCategory("Inputs", FText::GetEmpty(), ECategoryPriority::Important);
		TrainingSettingsCategoryBuilder = &DetailLayoutBuilder->EditCategory("Training Settings", FText::GetEmpty(), ECategoryPriority::Important);
		LODSettingsCategoryBuilder = &DetailLayoutBuilder->EditCategory("LOD Generation Settings", FText::GetEmpty(), ECategoryPriority::Uncommon);
		LODSettingsCategoryBuilder->SetCategoryVisibility(Model->DoesSupportLOD());
	}

	void FMLDeformerModelDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
	{
		DetailLayoutBuilder = &DetailBuilder;

		// Update the pointers and check if they are valid.
		TArray<TWeakObjectPtr<UObject>> Objects;
		DetailBuilder.GetObjectsBeingCustomized(Objects);
		if (!UpdateMemberPointers(Objects))
		{
			return;
		}

		CreateCategories();

		// Base mesh details.
		BaseMeshCategoryBuilder->AddProperty(UMLDeformerModel::GetSkeletalMeshPropertyName(), UMLDeformerModel::StaticClass());

		AddBaseMeshErrors();

		// Check if the vertex counts of our asset have changed.
		const FText ChangedErrorText = EditorModel->GetBaseAssetChangedErrorText();
		FDetailWidgetRow& ChangedErrorRow = BaseMeshCategoryBuilder->AddCustomRow(FText::FromString("BaseMeshChangedError"))
			.Visibility(!ChangedErrorText.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed)
			.WholeRowContent()
			[
				SNew(SBox)
				.Padding(FMargin(0.0f, 4.0f))
				[
					SNew(SWarningOrErrorBox)
					.MessageStyle(EMessageStyle::Warning)
					.Message(ChangedErrorText)
				]
			];


		// Check if our skeletal mesh's imported model contains a list of mesh infos. If not, we need to reimport it as it is an older asset.
		const FText NeedsReimportErrorText = EditorModel->GetSkeletalMeshNeedsReimportErrorText();
		FDetailWidgetRow& NeedsReimportErrorRow = BaseMeshCategoryBuilder->AddCustomRow(FText::FromString("BaseMeshNeedsReimportError"))
			.Visibility(!NeedsReimportErrorText.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed)
			.WholeRowContent()
			[
				SNew(SBox)
				.Padding(FMargin(0.0f, 4.0f))
				[
					SNew(SWarningOrErrorBox)
					.MessageStyle(EMessageStyle::Error)
					.Message(NeedsReimportErrorText)
				]
			];

		const FText VertexMapMisMatchErrorText = EditorModel->GetVertexMapChangedErrorText();
		FDetailWidgetRow& VertexMapErrorRow = BaseMeshCategoryBuilder->AddCustomRow(FText::FromString("VertexMapError"))
			.Visibility(!VertexMapMisMatchErrorText.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed)
			.WholeRowContent()
			[
				SNew(SBox)
				.Padding(FMargin(0.0f, 4.0f))
				[
					SNew(SWarningOrErrorBox)
					.MessageStyle(EMessageStyle::Error)
					.Message(VertexMapMisMatchErrorText)
				]
			];

		AddTrainingInputAnims();
		AddTrainingInputFlags();
		AddTrainingInputErrors();

		InputOutputCategoryBuilder->AddProperty(UMLDeformerModel::GetAlignmentTransformPropertyName(), UMLDeformerModel::StaticClass());

		FDetailWidgetRow& ErrorRow = InputOutputCategoryBuilder->AddCustomRow(FText::FromString("InputsError"))
			.Visibility(TAttribute<EVisibility>::CreateLambda(
				[this]()
				{
					return !EditorModel->GetInputsErrorText().IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed;
				}))
			.WholeRowContent()
			[
				SNew(SBox)
				.Padding(FMargin(0.0f, 4.0f))
				[
					SNew(SWarningOrErrorBox)
					.MessageStyle(EMessageStyle::Error)
					.Message_Lambda([this] { return EditorModel->GetInputsErrorText(); })
				]
			];

		// Create the inputs widget.
		TSharedPtr<SMLDeformerInputWidget> InputWidget = EditorModel->CreateInputWidget();
		EditorModel->SetInputWidget(InputWidget);
		IDetailGroup& NetworkInputsGroup = InputOutputCategoryBuilder->AddGroup("NetworkInputs", LOCTEXT("NetworkInputs", "Network Inputs"), false, false);
		NetworkInputsGroup.AddWidgetRow().WholeRowContent().Widget = InputWidget.ToSharedRef();

		AddBoneInputErrors();
		AddCurveInputErrors();

		// Show a warning when no neural network has been set.
		{		
			FDetailWidgetRow& NeuralNetErrorRow = TrainingSettingsCategoryBuilder->AddCustomRow(FText::FromString("NeuralNetError"))
				.Visibility(!EditorModel->IsTrained() ? EVisibility::Visible : EVisibility::Collapsed)
				.WholeRowContent()
				[
					SNew(SBox)
					.Padding(FMargin(0.0f, 4.0f))
					[
						SNew(SWarningOrErrorBox)
						.MessageStyle(EMessageStyle::Warning)
						.Message(LOCTEXT("NeedsTraining", "Model still needs to be trained."))
					]
				];

			// Check if our network is compatible with the skeletal mesh.
			if (Model->GetSkeletalMesh() && EditorModel->IsTrained())
			{
				FDetailWidgetRow& NeuralNetIncompatibleErrorRow = TrainingSettingsCategoryBuilder->AddCustomRow(FText::FromString("NeuralNetIncompatibleError"))
					.Visibility(!Model->GetInputInfo()->IsCompatible(Model->GetSkeletalMesh()) ? EVisibility::Visible : EVisibility::Collapsed)
					.WholeRowContent()
					[
						SNew(SBox)
						.Padding(FMargin(0.0f, 4.0f))
						[
							SNew(SWarningOrErrorBox)
							.MessageStyle(EMessageStyle::Error)
							.Message(LOCTEXT("TrainingIncompatibleWithSkelMesh", "Trained neural network is incompatible with selected SkeletalMesh."))
						]
					];
			}
		}

		TrainingSettingsCategoryBuilder->AddProperty(UMLDeformerModel::GetTrainingDevicePropertyName(), UMLDeformerModel::StaticClass())
			.Visibility(TAttribute<EVisibility>::CreateLambda(
				[this]()
				{
					return Model->GetTrainingDeviceList().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
				}));
		TrainingSettingsCategoryBuilder->AddProperty(UMLDeformerModel::GetMaxTrainingFramesPropertyName(), UMLDeformerModel::StaticClass());
		AddTrainingSettingsErrors();

		LODSettingsCategoryBuilder->AddProperty(UMLDeformerModel::GetMaxNumLODsPropertyName(), UMLDeformerModel::StaticClass());
	}

	bool FMLDeformerModelDetails::FilterAnimSequences(const FAssetData& AssetData, USkeleton* Skeleton)
	{
		if (Skeleton && Skeleton->IsCompatibleForEditor(AssetData))
		{
			return false;
		}

		return true;
	}

	void FMLDeformerModelDetails::AddTrainingInputAnims()
	{
		UE_LOGF(LogMLDeformer, Warning, "Please implement the AddTrainingInputAnims method in your model details class.");
	}
}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
