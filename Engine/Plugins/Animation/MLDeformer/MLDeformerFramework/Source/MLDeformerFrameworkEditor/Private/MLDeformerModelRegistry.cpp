// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerModelRegistry.h"
#include "MLDeformerModel.h"
#include "MLDeformerEditorModel.h"

namespace UE::MLDeformer
{
	FMLDeformerEditorModelRegistry::~FMLDeformerEditorModelRegistry()
	{
		Map.Empty();
		InstanceMap.Empty();
	}

	void FMLDeformerEditorModelRegistry::RegisterEditorModel(UClass* ModelType, FOnGetEditorModelInstance Delegate, int32 ModelPriority)
	{
		Map.FindOrAdd(ModelType, Delegate);
		ModelPriorities.FindOrAdd(ModelType, ModelPriority);
		UpdateHighestPriorityModel();
	}

	void FMLDeformerEditorModelRegistry::UnregisterEditorModel(const UClass* ModelType)
	{
		if (Map.Contains(ModelType))
		{
			Map.Remove(ModelType);
		}

		if (ModelPriorities.Contains(ModelType))
		{
			ModelPriorities.Remove(ModelType);
			UpdateHighestPriorityModel();
		}
	}

	void FMLDeformerEditorModelRegistry::UpdateHighestPriorityModel()
	{
		int32 HighestPriority = MIN_int32;
		for (auto& Item : ModelPriorities)
		{
			if (Item.Value >= HighestPriority)
			{
				HighestPriority = Item.Value;
				HighestPriorityModelType = Item.Key;
			}
		}
	}

	int32 FMLDeformerEditorModelRegistry::GetPriorityForModel(UClass* ModelType) const
	{
		const int32* Priority = ModelPriorities.Find(ModelType);
		if (Priority)
		{
			return *Priority;
		}

		return MIN_int32;
	}

	int32 FMLDeformerEditorModelRegistry::GetHighestPriorityModelIndex() const
	{
		if (HighestPriorityModelType)
		{
			int32 Index = 0;
			for (auto& Item : Map)
			{
				if (Item.Key == HighestPriorityModelType)
				{
					return Index;
				}
				Index++;
			}
		}
		return -1;
	}

	int32 FMLDeformerEditorModelRegistry::GetNumRegisteredModels() const
	{ 
		return Map.Num();
	}

	int32 FMLDeformerEditorModelRegistry::GetNumInstancedModels() const
	{ 
		return InstanceMap.Num();
	}

	UClass* FMLDeformerEditorModelRegistry::GetHighestPriorityModel() const
	{ 
		return HighestPriorityModelType;
	}

	const TMap<UClass*, FOnGetEditorModelInstance>& FMLDeformerEditorModelRegistry::GetRegisteredModels() const
	{ 
		return Map;
	}

	const TMap<UMLDeformerModel*, FMLDeformerEditorModel*>& FMLDeformerEditorModelRegistry::GetModelInstances() const
	{ 
		return InstanceMap;
	}

	FMLDeformerEditorModel* FMLDeformerEditorModelRegistry::CreateEditorModel(UMLDeformerModel* Model)
	{
		FMLDeformerEditorModel* EditorModel = nullptr;
		FOnGetEditorModelInstance* Delegate = Map.Find(Model->GetClass());
		if (Delegate)
		{
			EditorModel = Delegate->Execute();
			InstanceMap.FindOrAdd(Model, EditorModel);
		}

		return EditorModel;
	}

	FMLDeformerEditorModel* FMLDeformerEditorModelRegistry::GetEditorModel(UMLDeformerModel* Model)
	{
		FMLDeformerEditorModel** EditorModel = InstanceMap.Find(Model);
		if (EditorModel)
		{
			return *EditorModel;
		}

		return nullptr;
	}

	void FMLDeformerEditorModelRegistry::RemoveEditorModelInstance(FMLDeformerEditorModel* EditorModel)
	{
		InstanceMap.Remove(EditorModel->GetModel());
	}
}	// namespace UE::MLDeformer
