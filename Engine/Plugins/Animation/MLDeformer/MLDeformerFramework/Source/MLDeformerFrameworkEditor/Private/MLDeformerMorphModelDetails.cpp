// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerMorphModelDetails.h"
#include "MLDeformerMorphModelEditorModel.h"
#include "MLDeformerMorphModel.h"
#include "MLDeformerGeomCacheHelpers.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "SWarningOrErrorBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"
#include "Misc/MessageDialog.h"

#define LOCTEXT_NAMESPACE "MLDeformerMorphModelDetails"

namespace UE::MLDeformer
{
	bool FMLDeformerMorphModelDetails::UpdateMemberPointers(const TArray<TWeakObjectPtr<UObject>>& Objects)
	{
		if (!FMLDeformerGeomCacheModelDetails::UpdateMemberPointers(Objects))
		{
			return false;
		}

		MorphModel = Cast<UMLDeformerMorphModel>(Model);
		check(MorphModel);
		MorphModelEditorModel = static_cast<FMLDeformerMorphModelEditorModel*>(EditorModel);

		return (MorphModel != nullptr && MorphModelEditorModel != nullptr);
	}

	void FMLDeformerMorphModelDetails::CreateCategories()
	{
		FMLDeformerGeomCacheModelDetails::CreateCategories();
		MorphTargetCategoryBuilder = &DetailLayoutBuilder->EditCategory("Morph Targets", FText::GetEmpty(), ECategoryPriority::Default);
	}

	bool FMLDeformerMorphModelDetails::ShouldShowShadingError() const
	{
		if (!Model || !EditorModel)
		{
			return false;
		}

		if (Model->GetVizSettings()->GetDeformerGraph())
		{
			return false;
		}

		const FMLDeformerEditorActor* MLActor = EditorModel->FindEditorActor(ActorID_Test_MLDeformed);
		if (!MLActor)
		{
			return false;
		}

		const UDebugSkelMeshComponent* SkelMeshComponent = MLActor->GetSkeletalMeshComponent();
		if (!SkelMeshComponent)
		{
			return false;
		}

		// Check if we have skin cache enabled.
		const int32 LOD = 0;
		if (SkelMeshComponent->IsSkinCacheAllowed(LOD))
		{
			return false;
		}

		// Check the materials on parts of the mesh that are modified by the ML Deformer.
		TArray<int32> MaterialIndices;
		const int32 NumTrainingAnims = GeomCacheEditorModel->GetNumTrainingInputAnims();
		for (int32 AnimIndex = 0; AnimIndex < NumTrainingAnims; ++AnimIndex)
		{
			const FMLDeformerGeomCacheTrainingInputAnim* Anim = static_cast<FMLDeformerGeomCacheTrainingInputAnim*>(GeomCacheEditorModel->GetTrainingInputAnim(AnimIndex));
			if (!Anim || !Anim->IsEnabled())
			{
				continue;
			}

			// Get the sampler and try to initialize it when needed.
			FMLDeformerGeomCacheSampler* Sampler = static_cast<FMLDeformerGeomCacheSampler*>(GeomCacheEditorModel->GetSamplerForTrainingAnim(AnimIndex));
			if (!Sampler->IsInitialized())
			{
				Sampler->Init(GeomCacheEditorModel, AnimIndex);
			}

			if (Sampler->IsInitialized()) // This could still fail, when it failed to initialize before.
			{
				for (const FMLDeformerGeomCacheMeshMapping& MeshMapping : Sampler->GetMeshMappings())
				{
					for (const int32 MaterialIndex : MeshMapping.MaterialIndices)
					{
						MaterialIndices.AddUnique(MaterialIndex);
					}
				}
			}
		}

		// Now check if the used materials have the 'Use with morph targets' flag disabled.
		const USkeletalMesh* SkelMesh = SkelMeshComponent->GetSkeletalMeshAsset();
		if (SkelMesh)
		{
			const TArray<FSkeletalMaterial>& Materials = SkelMesh->GetMaterials();
			bool bHasMaterialErrors = false;
			for (const int32 MaterialIndex : MaterialIndices)
			{
				if (Materials.IsValidIndex(MaterialIndex))
				{
					const TObjectPtr<UMaterialInterface> MaterialInterface = Materials[MaterialIndex].MaterialInterface;
					if (MaterialInterface && !MaterialInterface->GetUsageByFlag(MATUSAGE_MorphTargets))
					{
						UE_LOGF(LogMLDeformer, Warning, "Material '%ls' (Index=%d) has the 'Used with Morph Targets' property disabled, while no deformer graph or skin cache is used. This can cause issues with ML Deformer.",
							*Materials[MaterialIndex].MaterialSlotName.ToString(), MaterialIndex);
						bHasMaterialErrors = true;
					}
				}
			}
			
			if (bHasMaterialErrors)
			{
				return true;
			}
		}

		return false;
	}

	void FMLDeformerMorphModelDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
	{
		// Create all the detail categories and add the properties of the base class.
		FMLDeformerGeomCacheModelDetails::CustomizeDetails(DetailBuilder);

		if (!EditorModel)
		{
			return;
		}

		if (ShouldShowShadingError())
		{
			FDetailWidgetRow& CannotRunMLDeformerWarning = BaseMeshCategoryBuilder->AddCustomRow(FText::FromString("CannotRunMLDeformerError"))
				.WholeRowContent()
				[
					SNew(SBox)
					.Padding(FMargin(0.0f, 4.0f))
					[
						SNew(SWarningOrErrorBox)
						.MessageStyle(EMessageStyle::Error)
						.Message_Lambda
						(
							[this]()
							{
								return LOCTEXT("CannotRunMLDeformerErrorMessage", 
									"This ML Deformer cannot work properly because:\n"
									"\n"
									"- No deformer graph is used.\n"
									"- Skin cache is disabled in the project settings or the Skeletal Mesh.\n"
									"- There are materials that have the 'Used with morph targets' property disabled.\n"
									"\n"
									"This can lead to visual shading artifacts or the deformer not working at all.\n"
									"See the log for more details.\n"
									"\n"
									"To fix this, either use a deformer graph, enable skin cache, or enable the mentioned material property."
								);
							}
						)
					]
				];
		}

		MorphTargetCategoryBuilder->AddProperty(DetailLayoutBuilder->GetProperty(UMLDeformerMorphModel::GetClampMorphTargetWeightsPropertyName(), UMLDeformerMorphModel::StaticClass()))
			.Visibility(MorphModelEditorModel->IsMorphWeightClampingSupported() ? EVisibility::Visible : EVisibility::Collapsed);

		MorphTargetCategoryBuilder->AddProperty(DetailLayoutBuilder->GetProperty(UMLDeformerMorphModel::GetIncludeMorphTargetNormalsPropertyName(), UMLDeformerMorphModel::StaticClass()));

		IDetailGroup& CompressionGroup = MorphTargetCategoryBuilder->AddGroup("Compression", LOCTEXT("MorphCompressionGroupLabel", "Compression"), false, false);
		CompressionGroup.AddPropertyRow(DetailLayoutBuilder->GetProperty(UMLDeformerMorphModel::GetMorphDeltaZeroThresholdPropertyName(), UMLDeformerMorphModel::StaticClass()));
		CompressionGroup.AddPropertyRow(DetailLayoutBuilder->GetProperty(UMLDeformerMorphModel::GetMorphCompressionLevelPropertyName(), UMLDeformerMorphModel::StaticClass()));

		IDetailGroup& MaskGroup = MorphTargetCategoryBuilder->AddGroup("Mask", LOCTEXT("MorphMaskGroupLabel", "Masking"), false, false);
		MaskGroup.AddPropertyRow(DetailLayoutBuilder->GetProperty(UMLDeformerMorphModel::GetMaskChannelPropertyName(), UMLDeformerMorphModel::StaticClass()));
		MaskGroup.AddPropertyRow(DetailLayoutBuilder->GetProperty(UMLDeformerMorphModel::GetGlobalMaskAttributePropertyName(), UMLDeformerMorphModel::StaticClass()));
		MaskGroup.AddPropertyRow(DetailLayoutBuilder->GetProperty(UMLDeformerMorphModel::GetInvertMaskChannelPropertyName(), UMLDeformerMorphModel::StaticClass()));

		if (MorphModel && MorphModel->GetMaskChannel() == EMLDeformerMaskChannel::VertexAttribute && !EditorModel->FindVertexAttributes(MorphModel->GetGlobalMaskAttributeName()).IsValid())
		{
			const FText MaskErrorText = LOCTEXT("MorphGlobalMaskWeightMapError", "The weight map attribute you specified does not exist on the skeletal mesh.");
			FDetailWidgetRow& MaskErrorRow = MorphTargetCategoryBuilder->AddCustomRow(FText::FromString("MorphGlobalMaskError"))
				.WholeRowContent()
				[
					SNew(SBox)
					.Padding(FMargin(0.0f, 4.0f))
					[
						SNew(SWarningOrErrorBox)
						.MessageStyle(EMessageStyle::Warning)
						.Message(MaskErrorText)
					]
				];
		}

		if (MorphModel && MorphModel->HasRawMorph() && !MorphModel->CanDynamicallyUpdateMorphTargets())
		{
			const FText DeltaCountMismatchErrorText = LOCTEXT("MorphDeltaCountMismatch", "Dynamic morph target updates disabled until retrained. This is because the vertex count changed after the model was trained.");
			FDetailWidgetRow& DeltaMismatchErrorRow = MorphTargetCategoryBuilder->AddCustomRow(FText::FromString("MorphDeltaCountMismatchError"))
				.WholeRowContent()
				[
					SNew(SBox)
					.Padding(FMargin(0.0f, 4.0f))
					[
						SNew(SWarningOrErrorBox)
						.MessageStyle(EMessageStyle::Warning)
						.Message(DeltaCountMismatchErrorText)
					]
				];
		}

		MorphTargetCategoryBuilder->AddCustomRow(LOCTEXT("FinalizeMorphTargetsButton", "Finalize Morph Targets"))
			.WholeRowContent()
			[
				SNew(SBox)
				.Padding(2, 2)
				[
					SNew(SButton)
					.Text(LOCTEXT("FinalizeMorphTargetsButtonText", "Finalize Morph Targets"))
					.ToolTipText(LOCTEXT("FinalizeMorphTargetsButtonTooltip", 
						"Delete the raw vertex deltas, basically turning the editor asset into a cooked asset.\n"
						"This will reduce the disk size of the uncooked asset, but will make morph target mask and compression settings uneditable until the model is retrained again.\n"
						"Finalizing isn't required, but can be used to reduce the size of the files you submit to source control."))
					.HAlign(HAlign_Center)
					.IsEnabled_Lambda([this]() { return MorphModel && !MorphModel->GetMorphTargetDeltas().IsEmpty(); })
					.OnClicked_Lambda([this]() -> FReply
					{
						const EAppReturnType::Type Result = FMessageDialog::Open(
							EAppMsgType::YesNo,
							LOCTEXT("FinalizeMorphsWarningMessage",
									"This will remove the raw uncompressed trained morph target deltas, just like when cooking the asset. "
									"Doing this will make the uncooked asset that you submit to source control a lot smaller.\n\n"
									"However, after doing this, changing settings in the morph target category will not have an effect until "
									"you retrain the model again. It does not impact how the deformation performs at runtime.\n\n"
									"If you proceed, you cannot undo this operation. To get the uncompressed deltas back you have to train the "
									"model again.\n"
									"\n"
									"Would you like to continue?"),
							LOCTEXT("FinalizeMorphsWarningDialogTitle", "Finalize morph targets?"));

						if (Result == EAppReturnType::Yes)
						{
							if (MorphModel)
							{
								MorphModel->FinalizeMorphTargets();
							}
						}
						return FReply::Handled();
					})
				]
			];
	}
}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
