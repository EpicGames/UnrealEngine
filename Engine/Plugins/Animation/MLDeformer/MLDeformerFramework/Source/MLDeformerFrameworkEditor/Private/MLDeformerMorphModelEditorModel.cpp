// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerMorphModelEditorModel.h"
#include "MLDeformerMorphModel.h"
#include "MLDeformerMorphModelVizSettings.h"
#include "MLDeformerMorphModelInstance.h"
#include "MLDeformerEditorToolkit.h"
#include "MLDeformerEditorStyle.h"
#include "MLDeformerInputInfo.h"
#include "MLDeformerComponent.h"
#include "MLDeformerGeomCacheHelpers.h"
#include "Components/ExternalMorphSet.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Animation/MorphTarget.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Engine/SkeletalMesh.h"
#include "Misc/ScopedSlowTask.h"
#include "BoneWeights.h"
#include "MeshAttributeArray.h"
#include "MeshAttributes.h"
#include "Async/ParallelFor.h"

#define LOCTEXT_NAMESPACE "MLDeformerMorphModelEditorModel"

namespace UE::MLDeformer
{
	FMLDeformerEditorModel* FMLDeformerMorphModelEditorModel::MakeInstance()
	{
		return new FMLDeformerMorphModelEditorModel();
	}

	void FMLDeformerMorphModelEditorModel::CopyBaseSettingsFromModel(const FMLDeformerEditorModel* SourceEditorModel)
	{
		// Copy the morph related settings.
		const UMLDeformerMorphModel* SourceMorphModel = Cast<UMLDeformerMorphModel>(SourceEditorModel->GetModel());
		if (SourceMorphModel)
		{
			UMLDeformerMorphModel* TargetModel = Cast<UMLDeformerMorphModel>(GetModel());
			check(TargetModel);
			TargetModel->SetMorphCompressionLevel(SourceMorphModel->GetMorphCompressionLevel());
			TargetModel->SetMorphDeltaZeroThreshold(SourceMorphModel->GetMorphDeltaZeroThreshold());
			TargetModel->SetIncludeMorphTargetNormals(SourceMorphModel->GetIncludeMorphTargetNormals());
			TargetModel->SetMaskChannel(SourceMorphModel->GetMaskChannel());
			TargetModel->SetInvertMaskChannel(SourceMorphModel->GetInvertMaskChannel());
		}

		// Copy all base class settings.
		FMLDeformerGeomCacheEditorModel::CopyBaseSettingsFromModel(SourceEditorModel);
	}

	bool FMLDeformerMorphModelEditorModel::IsInputMaskingSupported() const
	{
		return false;
	}

	void FMLDeformerMorphModelEditorModel::OnMaxNumLODsChanged()
	{
		UpdateLODMappings();
		if (GetMorphModel()->CanDynamicallyUpdateMorphTargets())
		{
			InitEngineMorphTargets(GetMorphModel()->GetMorphTargetDeltas());
		}
	}

	void FMLDeformerMorphModelEditorModel::OnPropertyChanged(FPropertyChangedEvent& PropertyChangedEvent)
	{
		const FProperty* Property = PropertyChangedEvent.Property;
		if (Property == nullptr)
		{
			return;
		}

		FMLDeformerGeomCacheEditorModel::OnPropertyChanged(PropertyChangedEvent);

		if (Property->GetFName() == UMLDeformerMorphModel::GetMorphDeltaZeroThresholdPropertyName() || 
			Property->GetFName() == UMLDeformerMorphModel::GetMorphCompressionLevelPropertyName() ||
			Property->GetFName() == UMLDeformerMorphModel::GetIncludeMorphTargetNormalsPropertyName() ||
			Property->GetFName() == UMLDeformerMorphModel::GetMaskChannelPropertyName() ||
			Property->GetFName() == UMLDeformerMorphModel::GetInvertMaskChannelPropertyName() ||
			Property->GetFName() == UMLDeformerMorphModel::GetGlobalMaskAttributePropertyName() ||
			Property->GetFName() == UMLDeformerMorphModel::GetSkeletalMeshPropertyName())
		{
			if (PropertyChangedEvent.ChangeType & EPropertyChangeType::ValueSet || PropertyChangedEvent.ChangeType & EPropertyChangeType::ResetToDefault)
			{
				if (GetMorphModel()->CanDynamicallyUpdateMorphTargets())
				{
					InitEngineMorphTargets(GetMorphModel()->GetMorphTargetDeltas());
				}

				if (Property->GetFName() == UMLDeformerMorphModel::GetGlobalMaskAttributePropertyName() || Property->GetFName() == UMLDeformerMorphModel::GetMaskChannelPropertyName())
				{
					GetEditor()->GetModelDetailsView()->ForceRefresh();
				}
			}
		}
		else if (Property->GetFName() == UMLDeformerMorphModelVizSettings::GetMorphTargetNumberPropertyName())
		{
			ClampMorphTargetNumber();
		}
	}

	void FMLDeformerMorphModelEditorModel::ClampMorphTargetNumber()
	{
		const int32 LOD = 0;
		const UMLDeformerMorphModel* MorphModel = GetMorphModel();			
		UMLDeformerMorphModelVizSettings* MorphViz = GetMorphModelVizSettings();
		const int32 NumMorphTargets = MorphModel->GetMorphTargetSet(LOD).IsValid() ? MorphModel->GetMorphTargetSet(LOD)->MorphBuffers.GetNumMorphs() : 0;
		const int32 ClampedMorphTargetNumber = (NumMorphTargets > 0) ? FMath::Min<int32>(MorphViz->GetMorphTargetNumber(), NumMorphTargets - 1) : 0;
		MorphViz->SetMorphTargetNumber(ClampedMorphTargetNumber);
	}

	UMLDeformerMorphModel* FMLDeformerMorphModelEditorModel::GetMorphModel() const
	{ 
		return Cast<UMLDeformerMorphModel>(Model);
	}

	UMLDeformerMorphModelVizSettings* FMLDeformerMorphModelEditorModel::GetMorphModelVizSettings() const
	{
		return Cast<UMLDeformerMorphModelVizSettings>(GetMorphModel()->GetVizSettings());
	}

	FString FMLDeformerMorphModelEditorModel::GetHeatMapDeformerGraphPath() const
	{
		return FString(TEXT("/MLDeformerFramework/Deformers/DG_MLDeformerModel_GPUMorph_HeatMap.DG_MLDeformerModel_GPUMorph_HeatMap"));
	}

	FString FMLDeformerMorphModelEditorModel::GetHeatMapDeformerGraphDualQuatPath() const
	{
		return FString(TEXT("/MLDeformerFramework/Deformers/DG_MLDeformerModel_GPUMorph_HeatMap_DQ.DG_MLDeformerModel_GPUMorph_HeatMap_DQ"));
	}

	void FMLDeformerMorphModelEditorModel::OnPreTraining()
	{
		// Backup the morph target deltas in case we abort training.
		MorphTargetDeltasBackup = GetMorphModel()->GetMorphTargetDeltas();
		MorphTargetsMinMaxWeightsBackup = GetMorphModel()->GetMorphTargetsMinMaxWeights();
	}

	void FMLDeformerMorphModelEditorModel::OnPostTraining(ETrainingResult TrainingResult, bool bUsePartiallyTrainedWhenAborted)
	{
		// We aborted and don't want to use partially trained results, we should restore the deltas that we just overwrote after training.
		if (TrainingResult == ETrainingResult::Aborted && !bUsePartiallyTrainedWhenAborted)
		{
			// Restore the morph target vertex deltas backup.
			GetMorphModel()->SetMorphTargetDeltas(MorphTargetDeltasBackup);
			GetMorphModel()->SetMorphTargetsMinMaxWeights(MorphTargetsMinMaxWeightsBackup);
		}
		else if (TrainingResult == ETrainingResult::Success || (TrainingResult == ETrainingResult::Aborted && bUsePartiallyTrainedWhenAborted))
		{
			// Build morph targets inside the engine, using the engine's compression scheme.
			// Add one as we included the means now as extra morph target.
			InitEngineMorphTargets(GetMorphModel()->GetMorphTargetDeltas());
		}

		// This internally calls InitGPUData() which updates the GPU buffer with the deltas.
		FMLDeformerGeomCacheEditorModel::OnPostTraining(TrainingResult, bUsePartiallyTrainedWhenAborted);
	}

	namespace
	{
		float CalcStandardDeviation(TArrayView<const float> Values)
		{
			if (Values.IsEmpty())
			{
				return 0.0f;
			}

			// First calculate the mean.
			float Mean = 0.0f;
			for (const float Value : Values)
			{
				Mean += Value;
			}
			Mean /= static_cast<float>(Values.Num());

			// Now calculate the standard deviation.
			float Sum = 0.0f;
			for (const float Value : Values)
			{
				Sum += FMath::Square(Value - Mean);
			}
			Sum /= static_cast<float>(Values.Num());

			return FMath::Sqrt(Sum);
		}
	}

	void FMLDeformerMorphModelEditorModel::UpdateMorphErrorValues(TArrayView<UMorphTarget*> MorphTargets)
	{
		if (MorphTargets.IsEmpty())
		{			
			return;
		}

		// Check if we have max morph weight information.
		// If we do not have this yet, we have to initialize the weights to 1.
		UMLDeformerMorphModel* MorphModel = GetMorphModel();
		const TArray<FFloatInterval>& MinMaxMorphWeights = MorphModel->GetMorphTargetsMinMaxWeights();

		const int32 NumMorphs = MinMaxMorphWeights.Num();

		// Preallocate space for the standard deviation of each morph target.
		TArray<float> ErrorValues;
		if (NumMorphs > 0)
		{
			ErrorValues.SetNumZeroed(NumMorphs - 1);
		}

		const int32 LOD = 0;
		TArray<float> DeltaLengths;
		for (int32 MorphIndex = 0; MorphIndex < NumMorphs - 1; ++MorphIndex)	// We have one extra morph for the means, skip that one.
		{
			const UMorphTarget* MorphTarget = MorphTargets[MorphIndex + 1];

			// Calculate the maximum of the absolute values of the min and max weight we saw during training.
			// We will multiply this with the length of the deltas later on to get an estimate of the maximum deformation for all deltas.
			const float MaxWeight = !MinMaxMorphWeights.IsEmpty() ? FMath::Max(FMath::Abs(MinMaxMorphWeights[MorphIndex].Min), FMath::Abs(MinMaxMorphWeights[MorphIndex].Min)) : 1.0f;

			// Get the array of deltas.
			TConstArrayView<FMorphTargetDelta> Deltas = MorphTarget->GetMorphTargetDeltas(LOD);

			// Build the array of position delta lengths.
			DeltaLengths.Reset(Deltas.Num());
			for (const FMorphTargetDelta& VertexDelta: Deltas)
			{
				DeltaLengths.Add(VertexDelta.PositionDelta.Length() * MaxWeight);
			}

			// Now calculate the standard deviation of those lengths.
			const float StandardDeviation = CalcStandardDeviation(DeltaLengths);
			ErrorValues[MorphIndex] = StandardDeviation;
		}

		// Build a list of array indices, so we know the order in which things got sorted.
		TArray<int32> SortedIndices;
		if (NumMorphs > 0)
		{
			SortedIndices.SetNumUninitialized(NumMorphs - 1);
			for (int32 Index = 0; Index < SortedIndices.Num(); ++Index)
			{
				SortedIndices[Index] = Index;
			}

			// Now that we have a list of standard deviations, sort them.
			SortedIndices.Sort
			(
				[&ErrorValues](const int32& IndexA, const int32& IndexB)
				{
					return ErrorValues[IndexA] > ErrorValues[IndexB];
				}
			);
		}

		// Update the morph model with the newly calculated error values.
		MorphModel->SetMorphTargetsErrorOrder(SortedIndices, ErrorValues);
	}

	const TArrayView<const float> FMLDeformerMorphModelEditorModel::GetMaskForMorphTarget(int32 MorphTargetIndex) const
	{
		// Return an empty array view object on default, which essentially disables the masking.
		// EditorModels can override this.
		return TArrayView<const float>();
	}

	void FMLDeformerMorphModelEditorModel::ZeroDeltasByLengthThreshold(TArray<FVector3f>& Deltas, float Threshold)
	{
		for (int32 VertexIndex = 0; VertexIndex < Deltas.Num(); ++VertexIndex)
		{
			if (Deltas[VertexIndex].Length() <= Threshold)
			{
				Deltas[VertexIndex] = FVector3f::ZeroVector;
			}
		}
	}

	void FMLDeformerMorphModelEditorModel::CalcVertexNormals(TArrayView<const FVector3f> VertexPositions, TArrayView<const uint32> IndexArray, TArrayView<const int32> VertexMap, TArray<FVector3f>& OutNormals) const
	{
		const int32 NumVertices = VertexPositions.Num();
		OutNormals.Reset();
		OutNormals.SetNumZeroed(NumVertices);

		checkf(IndexArray.Num() % 3 == 0, TEXT("Expecting a triangle mesh!"));
		const int32 NumTriangles = IndexArray.Num() / 3;
		for (int32 TriangleIndex = 0; TriangleIndex < NumTriangles; ++TriangleIndex)
		{
			const int32 ImportedIndices[3]	{ VertexMap[IndexArray[TriangleIndex * 3]], VertexMap[IndexArray[TriangleIndex * 3 + 1]], VertexMap[IndexArray[TriangleIndex * 3 + 2]] };
			const FVector3f Positions[3]	{ VertexPositions[ImportedIndices[0]], VertexPositions[ImportedIndices[1]], VertexPositions[ImportedIndices[2]] };

			const FVector3f EdgeA = (Positions[1] - Positions[0]).GetSafeNormal();
			const FVector3f EdgeB = (Positions[2] - Positions[0]).GetSafeNormal();
			if (EdgeA.SquaredLength() > 0.00001f && EdgeB.SquaredLength() > 0.00001f)
			{	
				const FVector3f FaceNormal = EdgeB.Cross(EdgeA);
				OutNormals[ImportedIndices[0]] += FaceNormal;
				OutNormals[ImportedIndices[1]] += FaceNormal;
				OutNormals[ImportedIndices[2]] += FaceNormal;
			}
		}

		// Renormalize.
		for (int32 Index = 0; Index < NumVertices; ++Index)
		{
			OutNormals[Index] = OutNormals[Index].GetSafeNormal();
		}
	}

	void FMLDeformerMorphModelEditorModel::CalcMorphTargetNormals(
		int32 LOD,
		const USkeletalMesh* SkelMesh,
		int32 MorphTargetIndex,
		const TArrayView<const FVector3f> Deltas,
		const TArrayView<const FVector3f> BaseVertexPositions,
		const TArrayView<const FVector3f> BaseNormals,
		const TArrayView<const int32> ImportedVertexToRenderVertexMapping,
		const FColorVertexBuffer& ColorBuffer,
		EMLDeformerMaskChannel MaskChannel,
		bool bInvertGlobalMaskChannel,
		const TArray<float>& GlobalMaskWeights,
		TArray<FVector3f>& OutDeltaNormals)
	{
		const FSkeletalMeshModel* ImportedModel = SkelMesh->GetImportedModel();
		if (ImportedModel == nullptr || !ImportedModel->LODModels.IsValidIndex(LOD))
		{
			OutDeltaNormals.Reset();
			OutDeltaNormals.SetNumZeroed(Model->GetNumBaseMeshVerts());
			return;
		}
		const TArray<uint32>& IndexArray = ImportedModel->LODModels[LOD].IndexBuffer;
		const TArray<int32>& VertexMap = ImportedModel->LODModels[LOD].MeshToImportVertexMap;

		// Get the optional input mask for this morph target.
		const TArrayView<const float> MorphMask = (MorphTargetIndex > 0) ? GetMaskForMorphTarget(MorphTargetIndex - 1) : TArrayView<const float>();

		// Build the array of displaced vertex positions.
		TArray<FVector3f> MorphedVertexPositions;
		const int32 NumBaseMeshVerts = Model->GetNumBaseMeshVerts();
		MorphedVertexPositions.SetNumUninitialized(NumBaseMeshVerts);
		for (int32 VertexIndex = 0; VertexIndex < NumBaseMeshVerts; ++VertexIndex)
		{
			const int32 RenderVertexIndex = !ImportedVertexToRenderVertexMapping.IsEmpty() ? ImportedVertexToRenderVertexMapping[VertexIndex] : INDEX_NONE;
			const int32 DeltaIndex = (MorphTargetIndex * NumBaseMeshVerts) + VertexIndex;
			const FVector3f RawDelta = Deltas[DeltaIndex];
			const float MorphMaskWeight = MorphMask.IsEmpty() ? 1.0f : MorphMask[VertexIndex];
			const float GlobalMaskWeight = (RenderVertexIndex != INDEX_NONE) ? GlobalMaskWeights[RenderVertexIndex] : 1.0f;
			FVector3f ScaledDelta;
			FVector3f DummyNormal;
			ProcessVertexDelta(ScaledDelta, DummyNormal, RawDelta, FVector3f::ZeroVector, 0.0f, MorphMaskWeight, GlobalMaskWeight);
			MorphedVertexPositions[VertexIndex] = BaseVertexPositions[VertexIndex] + ScaledDelta;
		}

		// Calculate the normals of that displaced mesh.
		TArray<FVector3f> MorphedNormals;
		CalcVertexNormals(MorphedVertexPositions, IndexArray, VertexMap, MorphedNormals);

		// Calculate and output the difference between the morphed normal and base normal.
		OutDeltaNormals.Reset();
		OutDeltaNormals.SetNumUninitialized(NumBaseMeshVerts);
		for (int32 VertexIndex = 0; VertexIndex < NumBaseMeshVerts; ++VertexIndex)
		{
			OutDeltaNormals[VertexIndex] = MorphedNormals[VertexIndex] - BaseNormals[VertexIndex];
			if (OutDeltaNormals[VertexIndex].SquaredLength() <= 0.00001f)
			{
				OutDeltaNormals[VertexIndex] = FVector3f::ZeroVector;
			}
		}
	}

	bool FMLDeformerMorphModelEditorModel::ProcessVertexDelta(FVector3f& OutScaledDelta, FVector3f& OutScaledDeltaNormal, const FVector3f RawDelta, const FVector3f RawDeltaNormal, float DeltaThreshold, float MorphMaskWeight, float GlobalMaskWeight) const
	{
		OutScaledDelta = RawDelta * MorphMaskWeight * GlobalMaskWeight;
		OutScaledDeltaNormal = RawDeltaNormal * GlobalMaskWeight;
		return (OutScaledDelta.Length() >= DeltaThreshold);
	}

	TArray<float> FMLDeformerMorphModelEditorModel::CalcGlobalMaskWeights(const TArray<int32>& VertexMap, const FColorVertexBuffer& ColorBuffer, EMLDeformerMaskChannel MaskChannel, bool bInvertMaskChannel) const
	{
		TArray<float> OutWeights;
		OutWeights.SetNumUninitialized(VertexMap.Num());

		const int32 NumRenderVertices = VertexMap.Num();

		// If we use vertex colors.
		if (MaskChannel == EMLDeformerMaskChannel::VertexColorRed || 
			MaskChannel == EMLDeformerMaskChannel::VertexColorGreen || 
			MaskChannel == EMLDeformerMaskChannel::VertexColorBlue || 
			MaskChannel == EMLDeformerMaskChannel::VertexColorAlpha)
		{
			const int32 NumColorVerts = ColorBuffer.GetNumVertices();
			for (int32 RenderVertexIndex = 0; RenderVertexIndex < NumRenderVertices; ++RenderVertexIndex)
			{
				float OutWeight = 1.0f;
				if (RenderVertexIndex != INDEX_NONE && NumColorVerts > 0)
				{
					const int32 ImportedVertexNumber = VertexMap[RenderVertexIndex];
					if (ImportedVertexNumber != INDEX_NONE)
					{
						const FLinearColor VertexColor = ColorBuffer.VertexColor(RenderVertexIndex);
						switch (MaskChannel)
						{
							case EMLDeformerMaskChannel::VertexColorRed:	{ OutWeight = VertexColor.R; break; }
							case EMLDeformerMaskChannel::VertexColorGreen:	{ OutWeight = VertexColor.G; break; }
							case EMLDeformerMaskChannel::VertexColorBlue:	{ OutWeight = VertexColor.B; break; }
							case EMLDeformerMaskChannel::VertexColorAlpha:	{ OutWeight = VertexColor.A; break; }
							default: 
								checkf(false, TEXT("Unexpected mask channel value."));
								break;
						};
					}
				}

				OutWeights[RenderVertexIndex] = OutWeight;
			}

		}
		else if (MaskChannel == EMLDeformerMaskChannel::VertexAttribute)	// Using a weight map as setup inside the skeletal mesh editor.
		{
			TVertexAttributesConstRef<float> WeightMapAttributes = FindVertexAttributes(GetMorphModel()->GetGlobalMaskAttributeName());
			if (WeightMapAttributes.IsValid())
			{
				for (int32 RenderVertexIndex = 0; RenderVertexIndex < NumRenderVertices; ++RenderVertexIndex)
				{
					const float VertexWeight = (VertexMap[RenderVertexIndex] != INDEX_NONE) ? WeightMapAttributes.Get(VertexMap[RenderVertexIndex]) : 1.0f;
					OutWeights[RenderVertexIndex] = FMath::Clamp(VertexWeight, 0.0f, 1.0f);
				}
			}
			else
			{
				for (int32 RenderVertexIndex = 0; RenderVertexIndex < NumRenderVertices; ++RenderVertexIndex)
				{
					OutWeights[RenderVertexIndex] = 1.0f;
				}
			}
		}
		else if (MaskChannel == EMLDeformerMaskChannel::Disabled)	// We disabled the mask, fill with 1.
		{
			for (int32 RenderVertexIndex = 0; RenderVertexIndex < NumRenderVertices; ++RenderVertexIndex)
			{
				OutWeights[RenderVertexIndex] = 1.0f;
			}
		}

		// Invert the weights if desired.
		if (bInvertMaskChannel)
		{
			for (int32 Index = 0; Index < OutWeights.Num(); ++Index)
			{
				const float Weight = OutWeights[Index];
				OutWeights[Index] = FMath::Clamp<float>(1.0f - Weight, 0.0f, 1.0f);
			}
		}

		return MoveTemp(OutWeights);
	}

	void FMLDeformerMorphModelEditorModel::CreateMorphTargets(
		TArray<UMorphTarget*>& OutMorphTargets, 
		const TArray<FVector3f>& Deltas, 
		const FString& NamePrefix, 
		int32 LOD, 
		float DeltaThreshold, 
		bool bIncludeNormals, 
		EMLDeformerMaskChannel MaskChannel,
		bool bInvertMaskChannel)
	{
		// In case when ActiveTrainingInputAnimIndex == INDEX_NONE but we have a sampler (e.g. testing), use that.
		FMLDeformerSampler* Sampler = GetSamplerForActiveAnim();
		if (Sampler == nullptr)
		{
			Sampler = GetSamplerForTrainingAnim(0);
		}

		OutMorphTargets.Reset();
		if (Deltas.IsEmpty() || Sampler == nullptr)
		{
			return;
		}

		if (Sampler->GetSkeletalMeshComponent()->GetSkeletalMeshAsset() != Model->GetSkeletalMesh())
		{
			return;
		}

		const int32 NumBaseMeshVerts = Model->GetNumBaseMeshVerts();
		if (Deltas.Num() % NumBaseMeshVerts != 0)
		{
			return;
		}

		const int32 NumMorphTargets = Deltas.Num() / NumBaseMeshVerts;
		check((Deltas.Num() / NumMorphTargets) == NumBaseMeshVerts);
		check(!Model->GetVertexMap().IsEmpty());

		FScopedSlowTask Task(1.0f, LOCTEXT("CreateMorphTargetProgress", "Creating morph targets"));
		Task.MakeDialogDelayed(2.0f, false);

		USkeletalMesh* SkelMesh = Model->GetSkeletalMesh();
		FSkeletalMeshRenderData* RenderData = SkelMesh->GetResourceForRendering();
		check(RenderData);
		check(!RenderData->LODRenderData.IsEmpty());
		const int32 NumRenderVertices = RenderData->LODRenderData[LOD].GetNumVertices();

		// Calculate the normals for the base mesh.
		const FSkeletalMeshModel* ImportedModel = SkelMesh->GetImportedModel();
		const TArrayView<const uint32> IndexArray = ImportedModel->LODModels[LOD].IndexBuffer;
		const TArray<int32>& VertexMap = ImportedModel->LODModels[LOD].MeshToImportVertexMap;
		const TArrayView<const FVector3f> BaseVertexPositions = Sampler->GetUnskinnedVertexPositions();
		const FColorVertexBuffer& ColorBuffer = RenderData->LODRenderData[LOD].StaticVertexBuffers.ColorVertexBuffer;

		// Precalc an array that maps imported vertices to a render vertex.
		TArray<int32> ImportedVertexToRenderVertexMapping;
		ImportedVertexToRenderVertexMapping.SetNumUninitialized(NumBaseMeshVerts);
		ParallelFor(TEXT("CreateMorphTargets_BuildMapping"), NumBaseMeshVerts, 5, [&](int32 VertexIndex)
		{
			ImportedVertexToRenderVertexMapping[VertexIndex] = VertexMap.Find(VertexIndex);
		});

		TArray<FVector3f> BaseNormals;
		if (bIncludeNormals)
		{
			CalcVertexNormals(BaseVertexPositions, IndexArray, VertexMap, BaseNormals);
		}

		// Calculate the global mask weights.
		const TArray<float> GlobalMaskWeights = CalcGlobalMaskWeights(VertexMap, ColorBuffer, MaskChannel, bInvertMaskChannel);

		// Initialize an engine morph target for each model morph target.
		UE_LOGF(LogMLDeformer, Display, "Initializing %d engine morph targets of %d vertices each", NumMorphTargets, Deltas.Num() / NumMorphTargets);

		for (int32 MorphTargetIndex = 0; MorphTargetIndex < NumMorphTargets; ++MorphTargetIndex)
		{
			const FName MorphName = *FString::Printf(TEXT("%s%.3d"), *NamePrefix, MorphTargetIndex);
			UMorphTarget* MorphTarget = NewObject<UMorphTarget>(SkelMesh, MorphName);
			MorphTarget->BaseSkelMesh = SkelMesh;
			OutMorphTargets.Add(MorphTarget);
		}
		
		ParallelFor(NumMorphTargets, [&](int32 MorphTargetIndex)
		{
			TArray<FVector3f> DeltaNormals;
			if (bIncludeNormals)
			{
				CalcMorphTargetNormals(LOD, SkelMesh, MorphTargetIndex, Deltas, BaseVertexPositions, BaseNormals, ImportedVertexToRenderVertexMapping, 
					ColorBuffer, MaskChannel, bInvertMaskChannel, GlobalMaskWeights, DeltaNormals);
			}

			UMorphTarget* MorphTarget = OutMorphTargets[MorphTargetIndex];

			// Create a new LOD model for this morph.
			TArray<FMorphTargetLODModel>& MorphLODs = MorphTarget->GetMorphLODModels();
			MorphLODs.AddDefaulted();
			FMorphTargetLODModel& MorphLODModel = MorphLODs.Last();

			// Initialize the morph target LOD level.
			MorphLODModel.Reset();
			MorphLODModel.bGeneratedByEngine = true;
			MorphLODModel.NumBaseMeshVerts = NumRenderVertices;
			MorphLODModel.NumVertices = NumRenderVertices;

			// Get the optional input mask for this morph target.
			const TArrayView<const float> MorphMask = (MorphTargetIndex > 0) ? GetMaskForMorphTarget(MorphTargetIndex - 1) : TArrayView<const float>();

			// Init deltas for this morph target.
			MorphLODModel.Vertices.Reserve(NumRenderVertices);
			TSet<int32> SectionIndices;
			for (int32 VertexIndex = 0; VertexIndex < NumRenderVertices; ++VertexIndex)
			{
				const int32 ImportedVertexNumber = VertexMap[VertexIndex];
				if (ImportedVertexNumber != INDEX_NONE)
				{
					const float GlobalMaskWeight = GlobalMaskWeights[VertexIndex];
					const float MorphMaskWeight = MorphMask.IsEmpty() ? 1.0f : MorphMask[ImportedVertexNumber];
					const FVector3f RawDelta = Deltas[ImportedVertexNumber + MorphTargetIndex * NumBaseMeshVerts];
					const FVector3f RawDeltaNormal = !DeltaNormals.IsEmpty() ? DeltaNormals[ImportedVertexNumber] : FVector3f::ZeroVector;
					FVector3f ScaledDelta;
					FVector3f ScaledDeltaNormal;				
					if (ProcessVertexDelta(ScaledDelta, ScaledDeltaNormal, RawDelta, RawDeltaNormal, DeltaThreshold, MorphMaskWeight, GlobalMaskWeight))
					{
						MorphLODModel.Vertices.AddDefaulted();
						FMorphTargetDelta& MorphTargetDelta = MorphLODModel.Vertices.Last();
						MorphTargetDelta.PositionDelta = ScaledDelta;
						MorphTargetDelta.SourceIdx = VertexIndex;
						MorphTargetDelta.TangentZDelta = bIncludeNormals ? ScaledDeltaNormal : FVector3f::ZeroVector;

						// Make sure we update the list of sections that we touch.
						int32 RenderSection = INDEX_NONE;
						int32 TempVertexIndex = INDEX_NONE;
						RenderData->LODRenderData[0].GetSectionFromVertexIndex(VertexIndex, RenderSection, TempVertexIndex);
						if (RenderSection != INDEX_NONE)
						{
							SectionIndices.Add(RenderSection);
						}
					}
				}
			}	// for all render vertices
			
			// Add all unique section indices.
			for (int32 SectionIndex : SectionIndices)
			{
				MorphLODModel.SectionIndices.Add(SectionIndex);
			}

			MorphLODModel.Vertices.Shrink();
		}); // For all morph targets.

		bool bHasOnlyEmptyMorphs = true;
		for (int32 MorphTargetIndex = 0; MorphTargetIndex < NumMorphTargets; ++MorphTargetIndex)
		{
			if (!OutMorphTargets[MorphTargetIndex]->GetMorphLODModels()[0].Vertices.IsEmpty())
			{
				bHasOnlyEmptyMorphs = false;
				break;
			}
		}

		GetMorphModel()->SetHasOnlyEmptyMorphs(bHasOnlyEmptyMorphs);
		Task.TickProgress();
	}

	void FMLDeformerMorphModelEditorModel::CompressMorphTargets(FMorphTargetVertexInfoBuffers& OutMorphBuffers, const TArray<UMorphTarget*>& MorphTargets, int32 LOD, float MorphErrorTolerance)
	{
		FScopedSlowTask Task(1, LOCTEXT("CompressMorphTargetsProgress", "Compressing morph targets"));
		Task.MakeDialogDelayed(2.0f, false);	

		USkeletalMesh* SkelMesh = Model->GetSkeletalMesh();
		FSkeletalMeshRenderData* RenderData = SkelMesh->GetResourceForRendering();
		check(RenderData);
		check(!RenderData->LODRenderData.IsEmpty());
		const int32 NumRenderVertices = RenderData->LODRenderData[LOD].GetNumVertices();

		// Release any existing morph buffer data.
		if (OutMorphBuffers.IsRHIInitialized() && OutMorphBuffers.IsInitialized())
		{
			ReleaseResourceAndFlush(&OutMorphBuffers);
		}

		// Don't empty the array of morph target data when we initialize the RHI buffers, as we need them to serialize later on.
		OutMorphBuffers = FMorphTargetVertexInfoBuffers();
		OutMorphBuffers.SetEmptyMorphCPUDataOnInitRHI(false);

		// Initialize the compressed morph target buffers.
		OutMorphBuffers.InitMorphResources
		(
			GMaxRHIShaderPlatform,
			RenderData->LODRenderData[LOD].RenderSections,
			MorphTargets,
			NumRenderVertices,
			LOD,
			MorphErrorTolerance
		);

		// Reinit the render resources.
		if (OutMorphBuffers.IsMorphCPUDataValid() && OutMorphBuffers.GetNumMorphs() > 0 && OutMorphBuffers.GetNumBatches() > 0)
		{
			BeginInitResource(&OutMorphBuffers);
		}

		Task.EnterProgressFrame();
	}

	void FMLDeformerMorphModelEditorModel::DebugDrawMorphTarget(FPrimitiveDrawInterface* PDI, const TArray<FVector3f>& MorphDeltas, float DeltaThreshold, int32 MorphTargetIndex, const FVector& DrawOffset)
	{
		FMLDeformerSampler* Sampler = GetSamplerForActiveAnim();
		if (Sampler == nullptr)
		{
			return;
		}

		UMLDeformerVizSettings* VizSettings = Model->GetVizSettings();
		const int32 NumVerts = Model->GetNumBaseMeshVerts();
		const TArray<FVector3f>& UnskinnedPositions = Sampler->GetUnskinnedVertexPositions();
		if (!MorphDeltas.IsEmpty() && 
			(MorphDeltas.Num() % NumVerts == 0) &&
			NumVerts == Model->GetInputInfo()->GetNumBaseMeshVertices() &&
			UnskinnedPositions.Num() == NumVerts)
		{
			// Get the optional input mask for this morph target.
			const TArrayView<const float> MorphMask = (MorphTargetIndex > 0) ? GetMaskForMorphTarget(MorphTargetIndex - 1) : TArrayView<const float>();

			const int32 NumMorphTargets = MorphDeltas.Num() / NumVerts;
			const int32 FinalMorphTargetIndex = FMath::Clamp<int32>(MorphTargetIndex, 0, NumMorphTargets - 1);
			const FLinearColor IncludedColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.Morphs.IncludedVertexColor");
			const FLinearColor ExcludedColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.Morphs.ExcludedVertexColor");
			for (int32 VertexIndex = 0; VertexIndex < NumVerts; ++VertexIndex)
			{
				const FVector StartPoint = FVector(UnskinnedPositions[VertexIndex]) + DrawOffset;
				const int32 DeltaArrayOffset = NumVerts * FinalMorphTargetIndex + VertexIndex;
				const FVector3f RawDelta(MorphDeltas[DeltaArrayOffset]);
				const float MorphMaskValue = MorphMask.IsEmpty() ? 1.0f : MorphMask[VertexIndex];
				FVector3f ScaledDelta;
				FVector3f ScaledDeltaNormal;
				if (ProcessVertexDelta(ScaledDelta, ScaledDeltaNormal, RawDelta, FVector3f::ZeroVector, DeltaThreshold, MorphMaskValue, 1.0f))
				{
					PDI->DrawPoint(StartPoint, IncludedColor, 1.0f, 0);
					PDI->DrawLine(StartPoint, StartPoint + FVector(ScaledDelta), IncludedColor, 0);
				}
				else
				{
					PDI->DrawPoint(StartPoint + FVector(ScaledDelta), ExcludedColor, 0.75f, 0);
				}
			}
		}
	}

	void FMLDeformerMorphModelEditorModel::InitEngineMorphTargets(const TArray<FVector3f>& Deltas)
	{
		UMLDeformerMorphModel* MorphModel = GetMorphModel();
		if (Deltas.IsEmpty())
		{
			return;
		}

		TArray<FVector3f> MorphTargetDeltas = Deltas;
		ZeroDeltasByLengthThreshold(MorphTargetDeltas, MorphModel->GetMorphDeltaZeroThreshold());

		// Turn the delta buffer into a set of engine morph targets.
		constexpr int32 LODZero = 0;
		const bool bIncludeNormals = MorphModel->GetIncludeMorphTargetNormals();
		const EMLDeformerMaskChannel MaskChannel = MorphModel->GetMaskChannel();
		const bool bInvertMaskChannel = MorphModel->GetInvertMaskChannel();
	
		// Create the engine morph targets.
		TArray<UMorphTarget*> MorphTargets;
		CreateMorphTargets(
			MorphTargets,
			Deltas, 
			TEXT("MLDeformerMorph_"), 
			LODZero,
			MorphModel->GetMorphDeltaZeroThreshold(),
			bIncludeNormals,
			MaskChannel,
			bInvertMaskChannel);

		// Analyze the error values of the morph targets.
		UpdateMorphErrorValues(MorphTargets);

		// Transfer morphs to the LOD levels.
		TransferMorphTargets(MorphTargets);
	
		// Resize to the new desired size.
		MorphModel->ClearMorphTargetSets();
		const int32 NumLODs = !MorphTargets.IsEmpty() ? MorphTargets[0]->GetMorphLODModels().Num() : 0;
		MorphModel->AddMorphSets(NumLODs);

		// Now compress the morph targets to GPU friendly buffers.
		for (int32 LOD = 0; LOD < NumLODs; ++LOD)
		{
			FMorphTargetVertexInfoBuffers& MorphBuffers = MorphModel->GetMorphTargetSet(LOD)->MorphBuffers;
			CompressMorphTargets(MorphBuffers, MorphTargets, LOD, MorphModel->GetMorphCompressionLevel());

			if (MorphBuffers.GetNumBatches() == 0 || MorphBuffers.GetNumMorphs() == 0)
			{
				MorphBuffers = FMorphTargetVertexInfoBuffers();
			}
		}

		MorphModel->UpdateStatistics();

		// Remove the morph targets again, as we don't need them anymore.
		for (UMorphTarget* MorphTarget : MorphTargets)
		{
			MorphTarget->ConditionalBeginDestroy();
		}

		// Update the editor actor skel mesh components for all the ones that also have an ML Deformer on it.
		for (FMLDeformerEditorActor* EditorActor : EditorActors)
		{
			UMLDeformerComponent* MLDeformerComponent = EditorActor->GetMLDeformerComponent();
			USkeletalMeshComponent* SkelMeshComponent = EditorActor->GetSkeletalMeshComponent();
			if (SkelMeshComponent && MLDeformerComponent)
			{
				SkelMeshComponent->RefreshExternalMorphTargetWeights();			
				MLDeformerComponent->UpdateSkeletalMeshComponent();
			}
		}

		UpdateMemoryUsage();
	}

	void FMLDeformerMorphModelEditorModel::TransferMorphTargets(TArray<UMorphTarget*> MorphTargetsLODZero)
	{
		const USkeletalMesh* SkelMesh = Model->GetSkeletalMesh();
		if (!SkelMesh || !SkelMesh->GetImportedModel())
		{
			return;
		}

		const int32 MaxLodWithMorphs = Model->GetMaxNumLODs();
		const int32 NumLODs = FMath::Min(SkelMesh->GetLODNum(), MaxLodWithMorphs);
		const int32 NumTaskItems = 1 + (NumLODs - 1) * MorphTargetsLODZero.Num();	// +1 because we see preparing the lookup tables as one task item as well.
		FScopedSlowTask Task(NumTaskItems, LOCTEXT("TransferMorphTargetProgress", "Generating morph target LODs"));
		Task.MakeDialogDelayed(2.0f, false);	

		const double StartTime = FPlatformTime::Seconds();

		FSkeletalMeshRenderData* RenderData = SkelMesh->GetResourceForRendering();
		check(RenderData);

		// Build a mapping table to eliminate linear searches.
		// For every morph target build a map that maps a render vertex to a morph target vertex.
		TArray<TMap<int32, int32>> RenderVertexToMorphVertexLodZero;
		RenderVertexToMorphVertexLodZero.SetNum(MorphTargetsLODZero.Num());
		for (int32 MorphIndex = 0; MorphIndex < MorphTargetsLODZero.Num(); ++MorphIndex)
		{
			UMorphTarget* MorphTarget = MorphTargetsLODZero[MorphIndex];
			const TArray<FMorphTargetDelta>& MorphVertices = MorphTarget->GetMorphLODModels()[0].Vertices;
			const int32 NumMorphVerts = MorphVertices.Num();
			RenderVertexToMorphVertexLodZero[MorphIndex].Reserve(NumMorphVerts);
			for (int32 MorphTargetVertexIndex = 0; MorphTargetVertexIndex < NumMorphVerts; ++MorphTargetVertexIndex)
			{
				const int32 RenderVertexIndex = MorphVertices[MorphTargetVertexIndex].SourceIdx;
				RenderVertexToMorphVertexLodZero[MorphIndex].Add(RenderVertexIndex, MorphTargetVertexIndex);
			}
		}

		Task.EnterProgressFrame();

		for (int32 LOD = 1; LOD < NumLODs; ++LOD)
		{
			for (int32 MorphIndex = 0; MorphIndex < MorphTargetsLODZero.Num(); ++MorphIndex)
			{
				// Add an LOD to this morph target.
				UMorphTarget* MorphTarget = MorphTargetsLODZero[MorphIndex];
				MorphTarget->GetMorphLODModels().AddDefaulted();
				FMorphTargetLODModel& MorphLODModel = MorphTarget->GetMorphLODModels().Last();

				const FSkeletalMeshLODModel& LODModel = SkelMesh->GetImportedModel()->LODModels[LOD];
				const int32 NumRenderVertices = LODModel.NumVertices;

				// Initialize the morph target LOD level.
				MorphLODModel.Reset();
				MorphLODModel.bGeneratedByEngine = true;
				MorphLODModel.Vertices.Reserve(NumRenderVertices);

				const TArray<int32>& MappingToLODZero = LODMappings[LOD].VtxMappingToLODZero;
				const FMorphTargetLODModel& MorphLODModelZero = MorphTarget->GetMorphLODModels()[0];

				// Add all vertices.
				for (int32 RenderVertexIndex = 0; RenderVertexIndex < NumRenderVertices; ++RenderVertexIndex)
				{
					// Try to locate a vertex in the morph target that uses the same render vertex index.
					const int32* MorphVertexIndexLODZero = RenderVertexToMorphVertexLodZero[MorphIndex].Find(MappingToLODZero[RenderVertexIndex]);
					const FMorphTargetDelta* DeltaInLODZero = MorphVertexIndexLODZero ? &MorphLODModelZero.Vertices[*MorphVertexIndexLODZero] : nullptr;
					
					// Make sure we found one, if we didn't find it, we can skip this vertex.
					if (!DeltaInLODZero)
					{
						continue;
					}

					// Add the vertex to the morph target.
					MorphLODModel.Vertices.AddDefaulted();
					FMorphTargetDelta& MorphTargetDelta = MorphLODModel.Vertices.Last();
					MorphTargetDelta.PositionDelta = DeltaInLODZero->PositionDelta;
					MorphTargetDelta.SourceIdx = RenderVertexIndex;
					MorphTargetDelta.TangentZDelta = DeltaInLODZero->TangentZDelta;

					// Make sure we update the list of sections that we touch.
					int32 RenderSection = INDEX_NONE;
					int32 TempVertexIndex = INDEX_NONE;
					RenderData->LODRenderData[LOD].GetSectionFromVertexIndex(RenderVertexIndex, RenderSection, TempVertexIndex);
					if (RenderSection != INDEX_NONE)
					{
						MorphLODModel.SectionIndices.AddUnique(RenderSection);
					}
				}
				MorphLODModel.NumBaseMeshVerts = NumRenderVertices;
				MorphLODModel.NumVertices = NumRenderVertices;
				MorphLODModel.Vertices.Shrink();
			} // For each morph target.
		} // For each LOD to generate.

		const double TotalTime = FPlatformTime::Seconds() - StartTime;
		UE_LOGF(LogMLDeformer, Display, "Finished Morph Target LOD generation in %.2f seconds", TotalTime);
	}

	void FMLDeformerMorphModelEditorModel::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
	{
		FMLDeformerGeomCacheEditorModel::Render(View, Viewport, PDI);

		// Debug draw the selected morph target.
		UMLDeformerMorphModel* MorphModel = GetMorphModel();
		const UMLDeformerMorphModelVizSettings* VizSettings = GetMorphModelVizSettings();
		if (VizSettings->GetDrawMorphTargets() && VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData)
		{
			const FVector DrawOffset = -VizSettings->GetMeshSpacingOffsetVector();
			DebugDrawMorphTarget(PDI, GetMorphModel()->GetMorphTargetDeltas(), MorphModel->GetMorphDeltaZeroThreshold(), VizSettings->GetMorphTargetNumber(), DrawOffset);
		}
	}

	void FMLDeformerMorphModelEditorModel::FillMaskValues(TArrayView<float> ItemMaskBuffer, float Value) const
	{
		const int32 NumBaseMeshVerts = Model->GetNumBaseMeshVerts();
		check(ItemMaskBuffer.Num() == NumBaseMeshVerts);
		for (int32 Index = 0; Index < NumBaseMeshVerts; ++Index)
		{
			ItemMaskBuffer[Index] = Value;
		}
	}

	void FMLDeformerMorphModelEditorModel::ApplyMaskInfoToBuffer(const USkeletalMesh* SkeletalMesh, const FMLDeformerMaskInfo& MaskInfo, TArrayView<float> ItemMaskBuffer)
	{
		// Apply the bones to the mask buffer.
		if (MaskInfo.MaskMode == EMLDeformerMaskingMode::Generated)
		{
			const FReferenceSkeleton& RefSkel = SkeletalMesh->GetRefSkeleton();
			for (const FName MaskBoneName : MaskInfo.BoneNames)
			{
				const int32 MaskBoneIndex = RefSkel.FindBoneIndex(MaskBoneName);
				if (MaskBoneIndex != INDEX_NONE)
				{
					ApplyBoneToMask(MaskBoneIndex, ItemMaskBuffer);
				}
				else
				{
					UE_LOGF(LogMLDeformer, Warning, "Mask contains a bone named '%ls', which cannot be found in the ref skeleton of skeletal mesh '%ls'.",
						*MaskBoneName.ToString(),
						*SkeletalMesh->GetName());
				}
			}
		}
		else // We're using a painted mask.
		{
			check(MaskInfo.MaskMode == EMLDeformerMaskingMode::VertexAttribute);
			const FName VertexAttributeName = MaskInfo.VertexAttributeName;
			if (!VertexAttributeName.IsNone())
			{
				TVertexAttributesConstRef<float> AttributeValues = FindVertexAttributes(VertexAttributeName);
				if (AttributeValues.IsValid())
				{
					const int32 NumAttributes = AttributeValues.GetNumElements();
					check(NumAttributes == ItemMaskBuffer.Num());
					for (int32 Index = 0; Index < NumAttributes; ++Index)
					{
						ItemMaskBuffer[Index] = AttributeValues.Get(Index);
					}
				}
				else
				{
					UE_LOGF(LogMLDeformer, Warning, "Mask references a vertex attribute '%ls' which doesn't exist on skeletal mesh %ls.", *VertexAttributeName.ToString(), *SkeletalMesh->GetName());
					FillMaskValues(ItemMaskBuffer, 1.0f);
				}
			}
			else
			{
				UE_LOGF(LogMLDeformer, Warning, "Mask is set to use a vertex attribute, but none is specified.");
				FillMaskValues(ItemMaskBuffer, 1.0f);
			}
		}
	}

	void FMLDeformerMorphModelEditorModel::ApplyGeneratedMaskToVertexAttributes(USkeletalMesh* SkeletalMesh, FMLDeformerMaskInfo& MaskInfo, TVertexAttributesRef<float> AttributeRef)
	{
		// Output some buffer with the generated mask.
		TArray<float> GeneratedBuffer;
		const int32 NumVerts = GetModel()->GetNumBaseMeshVerts();
		GeneratedBuffer.SetNumZeroed(NumVerts);
		EMLDeformerMaskingMode ModeBackup = MaskInfo.MaskMode;
		MaskInfo.MaskMode = EMLDeformerMaskingMode::Generated;	// Force generating.
		ApplyMaskInfoToBuffer(SkeletalMesh, MaskInfo, GeneratedBuffer);
		MaskInfo.MaskMode = ModeBackup;

		check(NumVerts == GeneratedBuffer.Num());

		// Calculate the average to scale the weights.
		// Normalizing the values doesn't really work well, as many values will have quite tiny weights then.
		// We tried normalizing, median the average, and the average seems to give the best vertex attribute weights.
		float AverageValue = 0.0f;
		for (int32 Index = 0; Index < NumVerts; ++Index)
		{
			AverageValue += GeneratedBuffer[Index];
		}

		if (NumVerts > 0)
		{
			AverageValue /= static_cast<float>(NumVerts);
			AverageValue *= 4.0f;	// Scale it a bit, to have slightly smoother edges.
		}

		if (FMath::IsNearlyZero(AverageValue))
		{
			AverageValue = 1.0f;
		}

		// Now apply this to the vertex attributes.
		for (int32 Index = 0; Index < NumVerts; ++Index)
		{
			const float Weight = FMath::Clamp(GeneratedBuffer[Index] / AverageValue, 0.0f, 1.0f);
			AttributeRef.Set(Index, Weight);
		}
	}

	void FMLDeformerMorphModelEditorModel::ApplyBoneToMask(int32 SkeletonBoneIndex, TArrayView<float> MaskBuffer)
	{
		const int32 LOD = 0;

		const int32 NumVerts = Model->GetNumBaseMeshVerts();
		check(MaskBuffer.Num() == NumVerts);

		const USkeletalMesh* SkeletalMesh = Model->GetSkeletalMesh();
		FSkeletalMeshModel* ImportedModel = SkeletalMesh->GetImportedModel();
		const FMLDeformerEditorActor* EditorActor = FindEditorActor(ActorID_Train_Base);
		if (ImportedModel == nullptr || EditorActor == nullptr || !ImportedModel->LODModels.IsValidIndex(LOD))
		{
			return;
		}

		const TArray<int32>& ImportedVertexNumbers = ImportedModel->LODModels[LOD].MeshToImportVertexMap;
		
		const UDebugSkelMeshComponent* SkeletalMeshComponent = EditorActor->GetSkeletalMeshComponent();
		FSkinWeightVertexBuffer& SkinWeightBuffer = *SkeletalMeshComponent->GetSkinWeightBuffer(LOD);
		const FSkeletalMeshLODRenderData& LODData = SkeletalMesh->GetResourceForRendering()->LODRenderData[LOD];
		const int32 NumRenderVerts = LODData.GetNumVertices();
		ParallelFor(TEXT("ApplyBoneToMask"), NumRenderVerts, 5, [&](int32 VertexIndex)
		{
			const int32 ImportedVertexNumber = ImportedVertexNumbers[VertexIndex];
			if (ImportedVertexNumber == INDEX_NONE)
			{
				return;
			}

			// Find the render section, which we need to find the right bone index.
			int32 SectionIndex = INDEX_NONE;
			int32 SectionVertexIndex = INDEX_NONE;
			LODData.GetSectionFromVertexIndex(VertexIndex, SectionIndex, SectionVertexIndex);

			// Iterate over all skinning influences.
			const int32 NumInfluences = SkinWeightBuffer.GetMaxBoneInfluences();
			for (int32 InfluenceIndex = 0; InfluenceIndex < NumInfluences; ++InfluenceIndex)
			{
				const int32 BoneIndex = SkinWeightBuffer.GetBoneIndex(VertexIndex, InfluenceIndex);
				const uint16 WeightByte = SkinWeightBuffer.GetBoneWeight(VertexIndex, InfluenceIndex);
				if (WeightByte > 0)
				{
					const int32 RealBoneIndex = LODData.RenderSections[SectionIndex].BoneMap[BoneIndex];
					if (RealBoneIndex == SkeletonBoneIndex)
					{
						const float Weight = static_cast<float>(WeightByte) * UE::AnimationCore::InvMaxRawBoneWeightFloat;
						MaskBuffer[ImportedVertexNumber] += Weight;
					}
				}
			}
		});		
	}

	void FMLDeformerMorphModelEditorModel::RecursiveAddBoneToMaskUpwards(const FReferenceSkeleton& RefSkel, int32 SkeletonBoneIndex, int32 MaxHierarchyDepth, TArray<int32>& OutBonesAdded, int32 CurHierarchyDepth)
	{
		if (CurHierarchyDepth > MaxHierarchyDepth)
		{
			return;
		}

		// Apply the current bone to the mask.
		if (!OutBonesAdded.Contains(SkeletonBoneIndex))
		{
			OutBonesAdded.Add(SkeletonBoneIndex);
		}

		// Apply the parent bone.
		const int32 ParentSkeletonBoneIndex = RefSkel.GetParentIndex(SkeletonBoneIndex);
		if (ParentSkeletonBoneIndex != INDEX_NONE)
		{
			RecursiveAddBoneToMaskUpwards(RefSkel, ParentSkeletonBoneIndex, MaxHierarchyDepth, OutBonesAdded, CurHierarchyDepth + 1);
		}
	}

	void FMLDeformerMorphModelEditorModel::RecursiveAddBoneToMaskDownwards(const FReferenceSkeleton& RefSkel, int32 SkeletonBoneIndex, int32 MaxHierarchyDepth, TArray<int32>& OutBonesAdded, int32 CurHierarchyDepth)
	{
		if (CurHierarchyDepth > MaxHierarchyDepth)
		{
			return;
		}

		// Apply the current bone to the mask.
		if (!OutBonesAdded.Contains(SkeletonBoneIndex))
		{
			OutBonesAdded.Add(SkeletonBoneIndex);
		}

		// Find all child bones.
		TArray<int32> ChildBones;
		ChildBones.Reserve(8);
		RefSkel.GetDirectChildBones(SkeletonBoneIndex, ChildBones);

		// Now recursively add the child bones.
		for (const int32 ChildIndex : ChildBones)
		{
			RecursiveAddBoneToMaskDownwards(RefSkel, ChildIndex, MaxHierarchyDepth, OutBonesAdded, CurHierarchyDepth + 1);
		}
	}

	void FMLDeformerMorphModelEditorModel::OnObjectModified(UObject* Object)
	{
		if (Model->GetSkeletalMesh() == Object || Model->GetInputInfo()->GetSkeletalMesh() == Object)
		{
			if (GetMorphModel()->CanDynamicallyUpdateMorphTargets())
			{
				InitEngineMorphTargets(GetMorphModel()->GetMorphTargetDeltas());
			}
			bNeedsAssetReinit = true;
		}

		FMLDeformerGeomCacheEditorModel::OnObjectModified(Object);
	}
}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
