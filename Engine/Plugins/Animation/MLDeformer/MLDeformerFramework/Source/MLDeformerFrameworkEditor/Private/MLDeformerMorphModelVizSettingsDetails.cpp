// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerMorphModelVizSettingsDetails.h"
#include "MLDeformerMorphModel.h"
#include "MLDeformerMorphModelEditorModel.h"
#include "MLDeformerModule.h"
#include "MLDeformerComponent.h"
#include "MLDeformerMorphModelInstance.h"
#include "MLDeformerEditorModule.h"
#include "MLDeformerEditorActor.h"
#include "MLDeformerGeomCacheHelpers.h"
#include "MLDeformerMorphModelVizSettings.h"
#include "Components/ExternalMorphSet.h"
#include "Components/SkeletalMeshComponent.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailsView.h"
#include "IDetailGroup.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "PropertyCustomizationHelpers.h"
#include "DetailLayoutBuilder.h"
#include "SWarningOrErrorBox.h"

#define LOCTEXT_NAMESPACE "MLDeformerMorphModelVizSettingsDetails"

namespace UE::MLDeformer
{
	bool FMLDeformerMorphModelVizSettingsDetails::UpdateMemberPointers(const TArray<TWeakObjectPtr<UObject>>& Objects)
	{
		if (!FMLDeformerGeomCacheVizSettingsDetails::UpdateMemberPointers(Objects))
		{
			return false;
		}

		MorphModel = Cast<UMLDeformerMorphModel>(Model);
		MorphModelVizSettings = Cast<UMLDeformerMorphModelVizSettings>(VizSettings);
		return (MorphModel != nullptr && MorphModelVizSettings != nullptr);
	}

	bool FMLDeformerMorphModelVizSettingsDetails::IsMorphTargetsEnabled() const
	{
		return MorphModelVizSettings->GetDrawMorphTargets() && !MorphModel->GetMorphTargetDeltas().IsEmpty();
	}

	int32 GetNumActiveMorphs(FMLDeformerMorphModelEditorModel* MorphEditorModel)
	{
		const UMLDeformerComponent* MLDeformerComponent = MorphEditorModel->FindMLDeformerComponent(ActorID_Test_MLDeformed);
		if (MLDeformerComponent)
		{
			const UMLDeformerMorphModelInstance* MorphInstance = Cast<UMLDeformerMorphModelInstance>(MLDeformerComponent->GetModelInstance());
			if (MorphInstance && MorphInstance->GetFinalSkeletalMeshComponent())
			{
				const int32 LOD = MorphInstance->GetFinalSkeletalMeshComponent()->GetPredictedLODLevel();
				FExternalMorphSetWeights* WeightData = MorphInstance->FindWeightData(LOD);
				if (WeightData)
				{
					WeightData->UpdateNumActiveMorphTargets();
					return WeightData->NumActiveMorphTargets;
				}
			}
		}

		return 0;
	}

	void FMLDeformerMorphModelVizSettingsDetails::AddAdditionalSettings()
	{
		IDetailGroup& MorphsGroup = LiveSettingsCategory->AddGroup("Morph Targets", LOCTEXT("MorphTargetsLabel", "Morph Targets"), false, true);
		MorphsGroup.AddPropertyRow(DetailLayoutBuilder->GetProperty(UMLDeformerMorphModelVizSettings::GetDrawMorphTargetsPropertyName(), UMLDeformerMorphModelVizSettings::StaticClass()))
			.EditCondition(!MorphModel->GetMorphTargetDeltas().IsEmpty() && MorphModel->CanDynamicallyUpdateMorphTargets(), nullptr);

		MorphsGroup.AddPropertyRow(DetailLayoutBuilder->GetProperty(UMLDeformerMorphModelVizSettings::GetMorphTargetNumberPropertyName(), UMLDeformerMorphModelVizSettings::StaticClass()))
			.EditCondition(TAttribute<bool>(this, &FMLDeformerMorphModelVizSettingsDetails::IsMorphTargetsEnabled), nullptr);
	
		FMLDeformerMorphModelEditorModel* MorphEditorModel = static_cast<FMLDeformerMorphModelEditorModel*>(EditorModel);
		MorphsGroup.AddWidgetRow()
			.NameContent()
			[			
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("ActiveMorphs", "Num Active Morphs"))
			]
			.ValueContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text_Lambda
				(
					[MorphEditorModel]()
					{
						int32 LOD = 0;
						const UMLDeformerComponent* MLDeformerComponent = MorphEditorModel->FindMLDeformerComponent(ActorID_Test_MLDeformed);
						if (MLDeformerComponent)
						{
							const UMLDeformerMorphModelInstance* MorphInstance = Cast<UMLDeformerMorphModelInstance>(MLDeformerComponent->GetModelInstance());
							if (MorphInstance)
							{
								LOD = MorphInstance->GetFinalSkeletalMeshComponent() ? MorphInstance->GetFinalSkeletalMeshComponent()->GetPredictedLODLevel() : 0;
							}
						}

						const UMLDeformerMorphModel* MorphModelPtr = Cast<UMLDeformerMorphModel>(MorphEditorModel->GetModel());					
						const int32 NumActiveMorphs = GetNumActiveMorphs(MorphEditorModel);
						const int32 NumMorphs = MorphModelPtr->GetNumMorphTargets(LOD);
						const float Percentage = (NumMorphs > 0) ? (NumActiveMorphs / static_cast<float>(NumMorphs)) * 100.0f : 100.0f;

						FNumberFormattingOptions TwoDigitsFormat;
						TwoDigitsFormat.SetMaximumFractionalDigits(0);
						return FText::Format(LOCTEXT("ActiveMorphsFormat", "{0} / {1} ({2} %) - LOD {3}"), NumActiveMorphs, NumMorphs, FText::AsNumber(Percentage, &TwoDigitsFormat), LOD);
					}
				)
			];
	}
}	//namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
