// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerOctree.h"
#include "PrimitiveDrawingUtils.h"

namespace UE::MLDeformer
{
	void FMLDeformerOctree::Build(const TArray<FSoftSkinVertex>& InVertices, int32 MaxNumVertsPerNode, int32 MaxDepth)
	{
		RootNode = MakeUnique<FMLDeformerOctree::FNode>();
		MeshVertices = InVertices;

		// First calculate the bounding box that contains all points as root node.
		RootNode->AABB.Init();
		const int32 NumVerts = MeshVertices.Num();
		RootNode->VertexIndices.SetNum(NumVerts);
		for (int32 VertexIndex = 0; VertexIndex < NumVerts; ++VertexIndex)
		{
			const FVector VertexPos = (FVector)MeshVertices[VertexIndex].Position;
			RootNode->AABB += VertexPos;
			RootNode->VertexIndices[VertexIndex] = VertexIndex;
		}

		// Recursively split if desired.
		if (RootNode->VertexIndices.Num() > MaxNumVertsPerNode && MaxDepth > 0)
		{
			Split(RootNode.Get(), MaxNumVertsPerNode, MaxDepth, 1);
		}
	}

	void FMLDeformerOctree::Render(FPrimitiveDrawInterface* PDI)
	{
		RecursiveRender(RootNode.Get(), PDI);
	}

	int32 FMLDeformerOctree::FindClosestMeshVertexIndex(FVector VertexPos, double FallBackDistanceThreshold) const
	{
		int32 ClosestVertexIndex = INDEX_NONE;
		double ClosestVertexDist = DBL_MAX;

		// Try to find the node this vertex would be in.
		const FNode* Node = FindNodeForPosition(RootNode.Get(), VertexPos);
		if (Node)
		{
			// Try and find the closest vertex inside here.
			const int32 NumVerts = Node->VertexIndices.Num();
			for (int32 Index = 0; Index < NumVerts; ++Index)
			{
				const int32 MeshVertexIndex = Node->VertexIndices[Index];
				const FVector NodeVertexPos = (FVector)MeshVertices[MeshVertexIndex].Position;

				// If we have an exact match, look no further.
				if (NodeVertexPos == VertexPos)
				{
					return MeshVertexIndex;
				}

				// Update the closest match if this one is closer.
				const double DistSq = (NodeVertexPos - VertexPos).SquaredLength();
				if (DistSq < ClosestVertexDist)
				{
					ClosestVertexDist = DistSq;
					ClosestVertexIndex = MeshVertexIndex;
				}
			}
		}

		// Worst case fallback scenario, just check against all vertices in the mesh.
		// We do this when there is no matching node, or when our distance is too large.
		// Remember that the LOD levels can have different topology, so this can happen.			
		if (ClosestVertexIndex == INDEX_NONE || ClosestVertexDist > FallBackDistanceThreshold)
		{
			ClosestVertexDist = DBL_MAX;
			const int32 NumVerts = MeshVertices.Num();
			for (int32 Index = 0; Index < NumVerts; ++Index)
			{
				const FVector NodeVertexPos = (FVector)MeshVertices[Index].Position;

				// If we have an exact match, look no further.
				if (NodeVertexPos == VertexPos)
				{
					return Index;
				}

				// Update the closest match if this one is closer.
				const double DistSq = (NodeVertexPos - VertexPos).SquaredLength();
				if (DistSq < ClosestVertexDist)
				{
					ClosestVertexDist = DistSq;
					ClosestVertexIndex = Index;
				}
			}
		}

		return ClosestVertexIndex;
	}

	void FMLDeformerOctree::Split(FNode* Parent, int32 MaxNumVertsPerNode, int32 MaxDepth, int32 CurDepth)
	{
		// Create the child nodes.
		Parent->ChildNodes.Reserve(8);
		for (int32 Index = 0; Index < 8; Index++)
		{
			Parent->ChildNodes.Add(MakeUnique<FMLDeformerOctree::FNode>());
			Parent->ChildNodes.Last()->Parent = Parent;
		}

		// Get the parent's box center and extents.
		FVector Center;
		FVector Extents;
		Parent->AABB.GetCenterAndExtents(Center, Extents);
		Extents *= 0.5; // Our splits are half the size.

		// Split the parent AABB into 8 boxes.
		// Top half.
		Parent->ChildNodes[0]->AABB = FBox::BuildAABB(Center + FVector(-Extents.X, -Extents.Y,  Extents.Z), Extents);
		Parent->ChildNodes[1]->AABB = FBox::BuildAABB(Center + FVector( Extents.X, -Extents.Y,  Extents.Z), Extents);
		Parent->ChildNodes[2]->AABB = FBox::BuildAABB(Center + FVector(-Extents.X,  Extents.Y,  Extents.Z), Extents);
		Parent->ChildNodes[3]->AABB = FBox::BuildAABB(Center + FVector( Extents.X,  Extents.Y,  Extents.Z), Extents);

		// Bottom half.
		Parent->ChildNodes[4]->AABB = FBox::BuildAABB(Center + FVector(-Extents.X, -Extents.Y, -Extents.Z), Extents);
		Parent->ChildNodes[5]->AABB = FBox::BuildAABB(Center + FVector( Extents.X, -Extents.Y, -Extents.Z), Extents);
		Parent->ChildNodes[6]->AABB = FBox::BuildAABB(Center + FVector(-Extents.X,  Extents.Y, -Extents.Z), Extents);
		Parent->ChildNodes[7]->AABB = FBox::BuildAABB(Center + FVector( Extents.X,  Extents.Y, -Extents.Z), Extents);

		// Pass the parent vertices to the child nodes.
		for (int32 Index = 0; Index < Parent->ChildNodes.Num(); ++Index)
		{
			FNode* ChildNode = Parent->ChildNodes[Index].Get();
			const FBox& ChildAABB = ChildNode->AABB;

			const int32 NumParentVerts = Parent->VertexIndices.Num();
			for (int32 VertexIndex = 0; VertexIndex < NumParentVerts; ++VertexIndex)
			{
				const int32 MeshVertexIndex = Parent->VertexIndices[VertexIndex];
				const FVector Pos = (FVector)MeshVertices[MeshVertexIndex].Position;
				if (ChildAABB.IsInsideOrOn(Pos))
				{
					ChildNode->VertexIndices.Add(MeshVertexIndex);	// TODO: reserve upfront?
				}
			}
		}

		// Get rid of the parent node vertices, as we transferred them into the children now.
		Parent->VertexIndices.Empty();

		// Remove empty children.
		for (int32 Index = 0; Index < Parent->ChildNodes.Num();)
		{
			if (Parent->ChildNodes[Index]->VertexIndices.IsEmpty())
			{
				Parent->ChildNodes.RemoveAt(Index);
			}
			else
			{
				Index++;
			}
		}

		// Split further if desired.
		for (int32 Index = 0; Index < Parent->ChildNodes.Num(); ++Index)
		{
			FNode* ChildNode = Parent->ChildNodes[Index].Get();
			if (ChildNode->VertexIndices.Num() > MaxNumVertsPerNode && CurDepth < MaxDepth)
			{
				Split(Parent->ChildNodes[Index].Get(), MaxNumVertsPerNode, MaxDepth, CurDepth + 1);
			}
		}
	}

	FMLDeformerOctree::FNode* FMLDeformerOctree::FindNodeForPosition(FNode* Node, const FVector Pos) const
	{
		if (Node->AABB.IsInsideOrOn(Pos))
		{
			for (int32 ChildIndex = 0; ChildIndex < Node->ChildNodes.Num(); ++ChildIndex)
			{
				FNode* ChildNode = Node->ChildNodes[ChildIndex].Get();
				if (ChildNode->AABB.IsInside(Pos))
				{
					return FindNodeForPosition(ChildNode, Pos);
				}
			}

			return Node;
		}

		return nullptr;
	}

	void FMLDeformerOctree::RecursiveRender(FNode* Node, FPrimitiveDrawInterface* PDI)
	{
		for (int32 ChildIndex = 0; ChildIndex < Node->ChildNodes.Num(); ++ChildIndex)
		{
			FNode* ChildNode = Node->ChildNodes[ChildIndex].Get();
			RecursiveRender(ChildNode, PDI);
		}

		if (Node->VertexIndices.IsEmpty())
		{
			return;
		}

		FVector BoxVerts[8];
		Node->AABB.GetVertices(BoxVerts);

		const FLinearColor BoxColor = FLinearColor::Blue;

		// Front side.
		PDI->DrawLine(BoxVerts[1], BoxVerts[5], BoxColor, 0);
		PDI->DrawLine(BoxVerts[0], BoxVerts[3], BoxColor, 0);
		PDI->DrawLine(BoxVerts[0], BoxVerts[1], BoxColor, 0);
		PDI->DrawLine(BoxVerts[3], BoxVerts[5], BoxColor, 0);

		// Back side.
		PDI->DrawLine(BoxVerts[6], BoxVerts[7], BoxColor, 0);
		PDI->DrawLine(BoxVerts[2], BoxVerts[4], BoxColor, 0);
		PDI->DrawLine(BoxVerts[6], BoxVerts[2], BoxColor, 0);
		PDI->DrawLine(BoxVerts[7], BoxVerts[4], BoxColor, 0);

		// Connections between front to back sides.
		PDI->DrawLine(BoxVerts[2], BoxVerts[0], BoxColor, 0);
		PDI->DrawLine(BoxVerts[4], BoxVerts[3], BoxColor, 0);
		PDI->DrawLine(BoxVerts[7], BoxVerts[5], BoxColor, 0);
		PDI->DrawLine(BoxVerts[6], BoxVerts[1], BoxColor, 0);
	}
}	// namespace UE::MLDeformer
