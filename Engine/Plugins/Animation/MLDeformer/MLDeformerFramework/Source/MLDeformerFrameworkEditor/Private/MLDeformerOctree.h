// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Templates/UniquePtr.h"

class FPrimitiveDrawInterface;

namespace UE::MLDeformer
{
	/**
	 * An octree around a mesh consisting of FSoftSkinVertex vertices.
	 * This class is used to accelerate finding a close matching vertex in a lower LOD, inside another LOD.
	 * For example, for a given vertex position in LOD 1, we want to find the best matching vertex in LOD 0.
	 * We do that by finding a close match. But instead of always iterating over all vertices in LOD 0, we would 
	 * first only check a small subset of nearby vertices.
	 */
	class FMLDeformerOctree
	{
		struct FNode;

	public:
		void Build(const TArray<FSoftSkinVertex>& InVertices, int32 MaxNumVertsPerNode = 200, int32 MaxDepth = 5);
		void Render(FPrimitiveDrawInterface* PDI);
		int32 FindClosestMeshVertexIndex(FVector VertexPos, double FallBackDistanceThreshold = 0.1) const;

	private:
		void Split(FNode* Parent, int32 MaxNumVertsPerNode, int32 MaxDepth, int32 CurDepth);
		void RecursiveRender(FNode* Node, FPrimitiveDrawInterface* PDI);
		FNode* FindNodeForPosition(FNode* Node, const FVector Pos) const;

	private:
		/* The octree node. */
		struct FNode
		{
			TArray<int32> VertexIndices;			// Points inside the MeshVertices member array.
			TArray<TUniquePtr<FNode>> ChildNodes;	// Doesn't need to be 8, can be less, or empty.
			FNode* Parent = nullptr;				// Pointer to parent node, or nullptr in case of the root node.
			FBox AABB;								// The bounding box of this node.
		};

		/** The root node of the octree. */
		TUniquePtr<FNode> RootNode;

		/** The actual array of vertices. */
		TArray<FSoftSkinVertex> MeshVertices;
	};
}	// namespace UE::MLDeformer
