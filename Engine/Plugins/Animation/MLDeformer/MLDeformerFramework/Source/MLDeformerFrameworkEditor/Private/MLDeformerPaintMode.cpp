// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerPaintMode.h"
#include "MLDeformerPaintModeToolkit.h"
#include "MeshAttributePaintTool.h"
#include "EdMode.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "EditorModeManager.h"
#include "ToolTargets/SkeletalMeshComponentToolTarget.h"
#include "ModelingToolsManagerActions.h"
#include "ToolTargetManager.h"
#include "ToolBuilderUtil.h"
#include "Components/SkeletalMeshComponent.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "SkeletalMeshAttributes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MLDeformerPaintMode)

#define LOCTEXT_NAMESPACE "MLDeformerPaintMode"

const FEditorModeID UMLDeformerPaintMode::Id("MLDeformerPaintMode");

UMLDeformerPaintMode::UMLDeformerPaintMode() 
{
	Info = FEditorModeInfo(Id, LOCTEXT("MLDeformerPaintMode", "ML Deformer Paint Mode"), FSlateIcon(), false);
}

void UMLDeformerPaintMode::Enter()
{
	UEdMode::Enter();

	UEditorInteractiveToolsContext* InteractiveToolsContext = GetInteractiveToolsContext();
	InteractiveToolsContext->TargetManager->AddTargetFactory(NewObject<USkeletalMeshComponentToolTargetFactory>(InteractiveToolsContext->TargetManager));

	const FModelingToolsManagerCommands& ToolManagerCommands = FModelingToolsManagerCommands::Get();
	RegisterTool(ToolManagerCommands.BeginMeshAttributePaintTool, TEXT("BeginMeshAttributePaintTool"), NewObject<UMeshAttributePaintToolBuilder>());

	GetInteractiveToolsContext()->ToolManager->SelectActiveToolType(EToolSide::Left, TEXT("BeginMeshAttributePaintTool"));
	GetInteractiveToolsContext()->ToolManager->ActivateTool(EToolSide::Left);

	// Try to get the bind pose vertex positions.
	UInteractiveTool* Tool = GetInteractiveToolsContext()->ToolManager->GetActiveTool(EToolSide::Left);
	UMeshAttributePaintTool* PaintTool = Cast<UMeshAttributePaintTool>(Tool);
	if (PaintTool)
	{
		TObjectPtr<UPreviewMesh> PreviewMesh = PaintTool->GetPreviewMesh();
		if (PreviewMesh)
		{
			const UE::Geometry::FDynamicMesh3* Mesh = PreviewMesh->GetMesh();
			if (Mesh)
			{
				BindPosePositions.Reset();
				BindPosePositions.SetNumZeroed(Mesh->VertexCount());

				for (const int32 VertexID : Mesh->VertexIndicesItr())
				{
					BindPosePositions[VertexID] = FVector3f(Mesh->GetVertex(VertexID));
				}						
			}
		}
	}
}

void UMLDeformerPaintMode::CreateToolkit()
{
	Toolkit = MakeShareable(new UE::MLDeformer::FMLDeformerPaintModeToolkit());
}

void UMLDeformerPaintMode::UpdatePose(USkeletalMeshComponent* SkeletalMeshComponent, bool bFullUpdate)
{
	using namespace UE::MLDeformer;

	UInteractiveTool* Tool = GetInteractiveToolsContext()->ToolManager->GetActiveTool(EToolSide::Left);
	UMeshAttributePaintTool* PaintTool = Cast<UMeshAttributePaintTool>(Tool);
	if (!PaintTool || !SkeletalMeshComponent || !SkeletalMeshComponent->GetSkeletalMeshAsset() || BindPosePositions.IsEmpty())
	{
		return;
	}

	TObjectPtr<UPreviewMesh> PreviewMesh = PaintTool->GetPreviewMesh();
	if (!PreviewMesh)
	{
		return;
	}

	// Get the current bone transforms.
	TArray<FMatrix44f> BoneMatrices;
	SkeletalMeshComponent->RefreshBoneTransforms();
	SkeletalMeshComponent->CacheRefToLocalMatrices(BoneMatrices);
	
	// Modify the vertex positions of the dynamic mesh.
	// We perform linear blend skinning using the bone matrices we just extracted.
	PreviewMesh->DeferredEditMesh([this, &BoneMatrices](FDynamicMesh3& Mesh)
	{
		const UE::Geometry::FDynamicMeshVertexSkinWeightsAttribute* SkinAttribute = Mesh.Attributes()->GetSkinWeightsAttribute(FSkeletalMeshAttributes::DefaultSkinWeightProfileName);
		if (!SkinAttribute)
		{
			return;
		}

		UE::AnimationCore::FBoneWeights BoneInfluences;
		for (const int32 VertexID : Mesh.VertexIndicesItr())
		{
			// Get the bone weights and bone indices.
			SkinAttribute->GetValue(VertexID, BoneInfluences);

			// Perform linear skinning.
			FVector3f SkinnedPos = FVector3f::ZeroVector;
			const FVector3f BindPoseVertex = BindPosePositions[VertexID];
			for (int32 SkinWeightIndex = 0; SkinWeightIndex < BoneInfluences.Num(); ++SkinWeightIndex)
			{				
				const UE::AnimationCore::FBoneWeight& BoneInfluence = BoneInfluences[SkinWeightIndex];
				const int32 BoneIndex = BoneInfluence.GetBoneIndex();
				SkinnedPos += BoneMatrices[BoneIndex].TransformPosition(BindPoseVertex) * BoneInfluence.GetWeight();
			}

			Mesh.SetVertex(VertexID, FVector(SkinnedPos), false);
		}
	}, false);

	// Update the render mesh, otherwise we don't see the changes visually.
	PreviewMesh->NotifyDeferredEditCompleted(UPreviewMesh::ERenderUpdateMode::FastUpdate, EMeshRenderAttributeFlags::Positions, /*RebuildSpatial*/bFullUpdate);

	if (bFullUpdate)
	{
		// Update the octree, otherwise the painting doesn't work properly.
		// We do this by re-inserting all vertices.
		const FDynamicMesh3* DynamicMesh = PreviewMesh->GetMesh();
		TSet<int32> Vertices;
		Vertices.Reserve(DynamicMesh->VertexCount());
		for (const int32 VertexID : DynamicMesh->VertexIndicesItr())
		{
			Vertices.Add(VertexID);	
		}
		PaintTool->GetVerticesOctree().ReinsertVertices(Vertices);
	}
}

void UMLDeformerPaintMode::SetMLDeformerEditor(UE::MLDeformer::FMLDeformerEditorToolkit* Editor)
{ 
	if (Toolkit.IsValid())
	{
		UE::MLDeformer::FMLDeformerPaintModeToolkit* PaintToolkit = static_cast<UE::MLDeformer::FMLDeformerPaintModeToolkit*>(Toolkit.Get());
		PaintToolkit->SetMLDeformerEditor(Editor);
	}
}

#undef LOCTEXT_NAMESPACE
