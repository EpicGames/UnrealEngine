// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerPaintModeToolkit.h"
#include "MLDeformerEditorToolkit.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailsView.h"
#include "InteractiveTool.h"
#include "InteractiveToolsContext.h"
#include "ModelingToolsEditorModeStyle.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Tools/UEdMode.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "ModelingToolsManagerActions.h"
#include "SPrimaryButton.h"
#include "Toolkits/AssetEditorModeUILayer.h"
#include "EditorModeManager.h"

#define LOCTEXT_NAMESPACE "MLDeformerPaintModeToolkit"

namespace UE::MLDeformer
{
	FMLDeformerPaintModeToolkit::~FMLDeformerPaintModeToolkit()
	{
		if (UEditorInteractiveToolsContext* Context = GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode))
		{
			Context->OnToolNotificationMessage.RemoveAll(this);
			Context->OnToolWarningMessage.RemoveAll(this);
		}
	}

	void FMLDeformerPaintModeToolkit::Init(const TSharedPtr<IToolkitHost>& InToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode)
	{
		bUsesToolkitBuilder = true;

		FModeToolkit::Init(InToolkitHost, InOwningMode);

		ModeWarningArea = SNew(STextBlock)
			.AutoWrapText(true)
			.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
			.ColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.15f, 0.15f)));
		ModeWarningArea->SetText(FText::GetEmpty());
		ModeWarningArea->SetVisibility(EVisibility::Collapsed);

		ModeHeaderArea = SNew(STextBlock)
			.AutoWrapText(true)
			.Font(FCoreStyle::GetDefaultFontStyle("Bold", 12));
		ModeHeaderArea->SetText(LOCTEXT("SelectToolLabel", "Select a Tool from the Toolbar"));
		ModeHeaderArea->SetJustification(ETextJustify::Center);
	
		ToolWarningArea = SNew(STextBlock)
			.AutoWrapText(true)
			.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
			.ColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.15f, 0.15f)));
		ToolWarningArea->SetText(FText::GetEmpty());

		RegisterPalettes();
		ClearNotification();
		ClearWarning();
	
		// create the toolkit widget
		{
			ToolkitSections->ModeWarningArea = ModeWarningArea;
			ToolkitSections->DetailsView = ModeDetailsView;
			ToolkitSections->ToolWarningArea = ToolWarningArea;

			SAssignNew(ToolkitWidget, SBorder)
			.HAlign(HAlign_Fill)
			.Padding(0)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				ToolkitBuilder->GenerateWidget()->AsShared()
			];	
		}
	
		ActiveToolName = FText::GetEmpty();
		ActiveToolMessage = FText::GetEmpty();

		if (UEditorInteractiveToolsContext* Context = GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode))
		{
			Context->OnToolNotificationMessage.AddSP(this, &FMLDeformerPaintModeToolkit::PostNotification);
			Context->OnToolWarningMessage.AddSP(this, &FMLDeformerPaintModeToolkit::PostWarning);
		}

		// add viewport overlay widget to accept / cancel tool
		MakeToolAcceptCancelWidget();
	}

	FName FMLDeformerPaintModeToolkit::GetToolkitFName() const
	{
		return FName("MLDeformerPaintModeToolkit");
	}

	FText FMLDeformerPaintModeToolkit::GetBaseToolkitName() const
	{
		return LOCTEXT("DisplayName", "ML Deformer Paint Mode");
	}

	void FMLDeformerPaintModeToolkit::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
	{
		UpdateActiveToolProperties(Tool);

		Tool->OnPropertySetsModified.AddSP(this, &FMLDeformerPaintModeToolkit::UpdateActiveToolProperties, Tool);

		ModeHeaderArea->SetVisibility(EVisibility::Collapsed);
		ActiveToolName = Tool->GetToolInfo().ToolDisplayName;

		FString ActiveToolIdentifier = GetScriptableEditorMode()->GetToolManager()->GetActiveToolName(EToolSide::Left);
		ActiveToolIdentifier.InsertAt(0, ".");
		FName ActiveToolIconName = ISlateStyle::Join(FModelingToolsManagerCommands::Get().GetContextName(), TCHAR_TO_ANSI(*ActiveToolIdentifier));
		ActiveToolIcon = FModelingToolsEditorModeStyle::Get()->GetOptionalBrush(ActiveToolIconName);
	
		GetToolkitHost()->AddViewportOverlayWidget(ViewportOverlayWidget.ToSharedRef());
	}

	void FMLDeformerPaintModeToolkit::OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
	{
		if (IsHosted())
		{
			GetToolkitHost()->RemoveViewportOverlayWidget(ViewportOverlayWidget.ToSharedRef());
		}
	
		if (Tool)
		{
			Tool->OnPropertySetsModified.RemoveAll(this);
		}

		ModeHeaderArea->SetVisibility(EVisibility::Visible);
		ModeDetailsView->SetObject(nullptr);
		ActiveToolName = FText::GetEmpty();
		ClearNotification();
		ClearWarning();

		if (MLDeformerEditor)
		{
			MLDeformerEditor->SetNeedsPaintModeDisable(true);
		}
	}

	FText FMLDeformerPaintModeToolkit::GetActiveToolDisplayName() const
	{
		return ActiveToolName;
	}

	FText FMLDeformerPaintModeToolkit::GetActiveToolMessage() const
	{
		return ActiveToolMessage;
	}

	void FMLDeformerPaintModeToolkit::RegisterPalettes()
	{
		const FModelingToolsManagerCommands& Commands = FModelingToolsManagerCommands::Get();
		const TSharedPtr<FUICommandList> CommandList = GetToolkitCommands();
	
		ToolkitSections = MakeShared<FToolkitSections>();

		FToolkitBuilderArgs ToolkitBuilderArgs(GetScriptableEditorMode()->GetModeInfo().ToolbarCustomizationName);
		ToolkitBuilderArgs.ToolkitCommandList = GetToolkitCommands();
		ToolkitBuilderArgs.ToolkitSections = ToolkitSections;
		ToolkitBuilderArgs.SelectedCategoryTitleVisibility = EVisibility::Collapsed;
		ToolkitBuilder = MakeShared<FToolkitBuilder>(ToolkitBuilderArgs);

		const TArray<TSharedPtr<FUICommandInfo>> SkinCommands({
			Commands.BeginMeshAttributePaintTool
		});
		ToolkitBuilder->AddPalette(MakeShareable(new FToolPalette(Commands.LoadSkinTools.ToSharedRef(), SkinCommands)));
		ToolkitBuilder->SetActivePaletteOnLoad(Commands.LoadSkinTools.Get());
		ToolkitBuilder->UpdateWidget();
	
		// If selected palette changes, make sure we are showing the palette command buttons, which may be hidden by active Tool.
		ActivePaletteChangedHandle = ToolkitBuilder->OnActivePaletteChanged.AddLambda([this]()
		{
			ToolkitBuilder->SetActivePaletteCommandsVisibility(EVisibility::Visible);
		});
	}

	void FMLDeformerPaintModeToolkit::MakeToolAcceptCancelWidget()
	{
		SAssignNew(ViewportOverlayWidget, SHorizontalBox)
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Bottom)
		.Padding(FMargin(0.0f, 0.0f, 0.f, 15.f))
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("EditorViewport.OverlayBrush"))
			.Padding(8.f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
				[
					SNew(SImage)
					.Image_Lambda([this] () { return ActiveToolIcon; })
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
				[
					SNew(STextBlock)
					.Text(this, &FMLDeformerPaintModeToolkit::GetActiveToolDisplayName)
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(0.0, 0.f, 2.f, 0.f))
				[
					SNew(SPrimaryButton)
					.Text(LOCTEXT("OverlayAccept", "Accept"))
					.ToolTipText(LOCTEXT("OverlayAcceptTooltip", "Accept/Commit the results of the active Tool [Enter]"))
					.OnClicked_Lambda([this]() { GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Accept); return FReply::Handled(); })
					.IsEnabled_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->CanAcceptActiveTool(); })
					.Visibility_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->ActiveToolHasAccept() ? EVisibility::Visible : EVisibility::Collapsed; })
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
				[
					SNew(SButton)
					.TextStyle( FAppStyle::Get(), "DialogButtonText" )
					.Text(LOCTEXT("OverlayCancel", "Cancel"))
					.ToolTipText(LOCTEXT("OverlayCancelTooltip", "Cancel the active Tool [Esc]"))
					.HAlign(HAlign_Center)
					.OnClicked_Lambda([this]() { GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Cancel); return FReply::Handled(); })
					.IsEnabled_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->CanCancelActiveTool(); })
					.Visibility_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->ActiveToolHasAccept() ? EVisibility::Visible : EVisibility::Collapsed; })
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
				[
					SNew(SPrimaryButton)
					.Text(LOCTEXT("OverlayComplete", "Complete"))
					.ToolTipText(LOCTEXT("OverlayCompleteTooltip", "Exit the active Tool [Enter]"))
					.OnClicked_Lambda([this]() { GetScriptableEditorMode()->GetInteractiveToolsContext()->EndTool(EToolShutdownType::Completed); return FReply::Handled(); })
					.IsEnabled_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->CanCompleteActiveTool(); })
					.Visibility_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext()->CanCompleteActiveTool() ? EVisibility::Visible : EVisibility::Collapsed; })
				]
			]	
		];
	}

	void FMLDeformerPaintModeToolkit::PostNotification(const FText& InMessage)
	{
		ClearNotification();
	
		ActiveToolMessage = InMessage;

		if (ModeUILayer.IsValid())
		{
			const FName StatusBarName = ModeUILayer.Pin()->GetStatusBarName();
			ActiveToolMessageHandle = GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->PushStatusBarMessage(StatusBarName, ActiveToolMessage);
		}
	}

	void FMLDeformerPaintModeToolkit::ClearNotification()
	{
		ActiveToolMessage = FText::GetEmpty();

		if (ModeUILayer.IsValid())
		{
			const FName StatusBarName = ModeUILayer.Pin()->GetStatusBarName();
			GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->PopStatusBarMessage(StatusBarName, ActiveToolMessageHandle);
		}
		ActiveToolMessageHandle.Reset();
	}

	void FMLDeformerPaintModeToolkit::PostWarning(const FText& Message)
	{
		ToolWarningArea->SetText(Message);
		ToolWarningArea->SetVisibility(EVisibility::Visible);
	}

	void FMLDeformerPaintModeToolkit::ClearWarning()
	{
		ToolWarningArea->SetText(FText());
		ToolWarningArea->SetVisibility(EVisibility::Collapsed);
	}

	void FMLDeformerPaintModeToolkit::UpdateActiveToolProperties(UInteractiveTool* Tool)
	{
		UInteractiveTool* CurTool = GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode)->GetActiveTool(EToolSide::Left);
		if (CurTool == nullptr)
		{
			return;
		}
		
		ModeDetailsView->SetObjects(CurTool->GetToolProperties(true));
	}
} // namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
