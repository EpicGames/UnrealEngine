// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerSampler.h"
#include "MLDeformerInputInfo.h"
#include "MLDeformerModel.h"
#include "MLDeformerEditorToolkit.h"
#include "MLDeformerTrainingInputAnim.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimInstance.h"
#include "BoneContainer.h"
#include "BoneWeights.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshRenderData.h"

namespace UE::MLDeformer
{
	FMLDeformerSampler::~FMLDeformerSampler()
	{
		if (SkelMeshActor)
		{
			SkelMeshActor->Destroy();
			SkelMeshActor = nullptr;
		}

		if (TargetMeshActor)
		{
			TargetMeshActor->Destroy();
			TargetMeshActor = nullptr;
		}
	}

	// Initializer a sampler item.
	void FMLDeformerSampler::Init(FMLDeformerEditorModel* InEditorModel, int32 InAnimIndex)
	{
		check(InEditorModel);
		if (InEditorModel->GetEditor()->GetPersonaToolkitPointer() == nullptr)
		{
			return;
		}

		EditorModel = InEditorModel;
		Model = EditorModel->GetModel();
		NumImportedVertices = UMLDeformerModel::ExtractNumImportedSkinnedVertices(Model->GetSkeletalMesh());
		AnimFrameIndex = 0;
		AnimIndex = InAnimIndex;
		SampleTime = 0.0f;
		NumFloatsPerCurve = 1;

		// Create the actors and components.
		// This internally skips creating them if they already exist.
		CreateActors();

		SkinnedVertexPositions.Reset();
		SkinnedVertexPositions.AddUninitialized(NumImportedVertices);
		VertexDeltas.Reset();
		BoneMatrices.Reset();
		BoneRotations.Reset();
		CurveValues.Reset();

		const int32 LODIndex = 0;
		ExtractUnskinnedPositions(LODIndex, UnskinnedVertexPositions);
	}

	void FMLDeformerSampler::UpdateSkeletalMeshComponent()
	{
		if (SkeletalMeshComponent && SkeletalMeshComponent->GetSkeletalMeshAsset())
		{
			// Sample the transforms at the frame time.
			SkeletalMeshComponent->SetPosition(SampleTime);
			SkeletalMeshComponent->bPauseAnims = true;
			SkeletalMeshComponent->RefreshBoneTransforms();
			SkeletalMeshComponent->CacheRefToLocalMatrices(BoneMatrices);
			if (SkeletalMeshComponent->GetAnimInstance())
			{
				SkeletalMeshComponent->GetAnimInstance()->GetRequiredBones().SetUseRAWData(true);
			}
		}
	}

	void FMLDeformerSampler::UpdateSkinnedPositions()
	{
		constexpr int32 LODIndex = 0;
		ExtractSkinnedPositions(LODIndex, BoneMatrices, TempVertexPositions, SkinnedVertexPositions);
	}

	void FMLDeformerSampler::UpdateBoneRotations()
	{
		const UMLDeformerInputInfo* InputInfo = EditorModel->GetEditorInputInfo();
		InputInfo->ExtractBoneRotations(SkeletalMeshComponent, BoneRotations);
	}

	void FMLDeformerSampler::UpdateCurveValues()
	{
		const UMLDeformerInputInfo* InputInfo = EditorModel->GetEditorInputInfo();
		InputInfo->ExtractCurveValues(SkeletalMeshComponent, CurveValues, NumFloatsPerCurve);
	}

	void FMLDeformerSampler::Sample(int32 InAnimFrameIndex)
	{
		FMLDeformerTrainingInputAnim* TrainingInputAnim = EditorModel->GetTrainingInputAnim(AnimIndex);
		if (!TrainingInputAnim || !TrainingInputAnim->IsValid())
		{
			return;
		}
		UAnimSequence* TrainingAnimSequence = EditorModel->GetTrainingInputAnim(AnimIndex)->GetAnimSequence();
		check(TrainingAnimSequence);
		const EAnimInterpolationType InterpolationTypeBackup = TrainingAnimSequence->Interpolation;
		TrainingAnimSequence->Interpolation = EAnimInterpolationType::Step;

		AnimFrameIndex = InAnimFrameIndex;
		SampleTime = GetTimeAtFrame(InAnimFrameIndex);

		const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
		if (SkeletalMeshComponent && SkeletalMesh)
		{
			UpdateSkeletalMeshComponent();
			if (VertexDeltaSpace == EVertexDeltaSpace::PostSkinning || SkinningMode == EMLDeformerSkinningMode::DualQuaternion)
			{
				UpdateSkinnedPositions();
			}
			UpdateBoneRotations();
			UpdateCurveValues();

			// Zero the deltas.
			const int NumFloats = SkeletalMesh->GetNumImportedVertices() * 3;	// xyz per vertex.
			VertexDeltas.Reset(NumFloats);
			VertexDeltas.AddZeroed(NumFloats);
		}

		TrainingAnimSequence->Interpolation = InterpolationTypeBackup;
	}

	// Calculate the inverse skinning transform. This is basically inv(sum(BoneTransform_i * inv(BoneRestTransform_i) * Weight_i)), where i is for each skinning influence for the given vertex.
	FMatrix44f FMLDeformerSampler::CalcInverseSkinningTransform(int32 VertexIndex, const FSkeletalMeshLODRenderData& SkelMeshLODData, const FSkinWeightVertexBuffer& SkinWeightBuffer) const
	{
		checkSlow(SkeletalMeshComponent);
		const USkeletalMesh* Mesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
		checkSlow(Mesh);

		// Find the render section, which we need to find the right bone index.
		int32 SectionIndex = INDEX_NONE;
		int32 SectionVertexIndex = INDEX_NONE;
		const FSkeletalMeshLODRenderData& LODData = Mesh->GetResourceForRendering()->LODRenderData[0];
		LODData.GetSectionFromVertexIndex(VertexIndex, SectionIndex, SectionVertexIndex);

		// Init the matrix at full zeros.
		FMatrix44f InvSkinningTransform = FMatrix44f(FVector3f::ZeroVector, FVector3f::ZeroVector, FVector3f::ZeroVector, FVector3f::ZeroVector);
		InvSkinningTransform.M[3][3] = 0.0f;

		// For each influence, sum up the weighted skinning matrices.
		const int32 NumInfluences = SkinWeightBuffer.GetMaxBoneInfluences();
		for (int32 InfluenceIndex = 0; InfluenceIndex < NumInfluences; ++InfluenceIndex)
		{
			const int32 BoneIndex = SkinWeightBuffer.GetBoneIndex(VertexIndex, InfluenceIndex);
			const uint16 WeightByte = SkinWeightBuffer.GetBoneWeight(VertexIndex, InfluenceIndex);
			if (WeightByte > 0)
			{
				const int32 RealBoneIndex = LODData.RenderSections[SectionIndex].BoneMap[BoneIndex];
				const float	Weight = static_cast<float>(WeightByte) * UE::AnimationCore::InvMaxRawBoneWeightFloat;
				const FMatrix44f& SkinningTransform = BoneMatrices[RealBoneIndex];
				InvSkinningTransform += SkinningTransform * Weight;
			}
		}

		// Return the inverse skinning transform matrix.
		return InvSkinningTransform.Inverse();
	}

	namespace
	{
		template<typename QuatType>
		QuatType Conjugate(const QuatType& Q)
		{
			return QuatType(-Q.X, -Q.Y, -Q.Z, Q.W);
		}
		
		template<typename QuatType>
		float Inner(const QuatType& Q1, const QuatType& Q2)
		{
			return Q1.X * Q2.X + Q1.Y * Q2.Y + Q1.Z * Q2.Z + Q1.W * Q2.W;
		}
		
		template<typename QuatType>
		QuatType FromVector(const FVector3f& V)
		{
			return QuatType(V[0], V[1], V[2], 0);
		}
		
		template<typename QuatType>
		FVector3f ToVector(const QuatType& Q)
		{
			return FVector3f(Q.X, Q.Y, Q.Z);
		}
	};

	FVector3f FMLDeformerSampler::CalcDualQuaternionDelta(int32 VertexIndex, const FVector3f& WorldDelta, const FSkeletalMeshLODRenderData& SkelMeshLODData, const FSkinWeightVertexBuffer& SkinWeightBuffer) const
	{
		check(SkeletalMeshComponent);
		const USkeletalMesh* Mesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
		check(Mesh);
		const FSkeletalMeshRenderData* RenderData = Mesh->GetResourceForRendering();
		constexpr int32 LODIndex = 0;
		check(RenderData && RenderData->LODRenderData.IsValidIndex(LODIndex));
		const FSkeletalMeshLODRenderData& LODData = RenderData->LODRenderData[LODIndex];

		// Find the render section, which we need to find the right bone index.
		int32 SectionIndex = INDEX_NONE;
		int32 SectionVertexIndex = INDEX_NONE;
		LODData.GetSectionFromVertexIndex(VertexIndex, SectionIndex, SectionVertexIndex);

		FQuat4f QuatSum = FQuat4f(0, 0, 0, 0);

		FQuat4f R0;
		float Sign = 1;
		const int32 NumInfluences = SkinWeightBuffer.GetMaxBoneInfluences();
		for (int32 InfluenceIndex = 0; InfluenceIndex < NumInfluences; ++InfluenceIndex)
		{
			const int32 BoneIndex = SkinWeightBuffer.GetBoneIndex(VertexIndex, InfluenceIndex);
			const uint16 WeightByte = SkinWeightBuffer.GetBoneWeight(VertexIndex, InfluenceIndex);
			// Weight must be > 0 when InflueceIndex == 0
			ensure(InfluenceIndex > 0 || WeightByte > 0);
			if (WeightByte > 0)
			{
				const int32 RealBoneIndex = LODData.RenderSections[SectionIndex].BoneMap[BoneIndex];
				const FQuat4f R(BoneMatrices[RealBoneIndex].GetMatrixWithoutScale());
				if (InfluenceIndex == 0)
				{
					R0 = R; Sign = 1;
				}
				else
				{
					Sign = Inner(R0, R) < 0 ? -1 : 1;
				}
				const float Weight = Sign * static_cast<float>(WeightByte) * UE::AnimationCore::InvMaxRawBoneWeightFloat;
				QuatSum += R * Weight;
			}
		}

		const float SizeSquared = QuatSum.SizeSquared();
		if (SizeSquared > SMALL_NUMBER)
		{
			/** Unrotate vector using v' = q^{-1} v q if q is unit size 
			 * Because QuatSum is not unit size
			 * v' =  q^* v q / |q|^2
			 */
			return ToVector<FQuat4f>(Conjugate(QuatSum) * FromVector<FQuat4f>(WorldDelta) * QuatSum) / SizeSquared;
		}
		else
		{
			return WorldDelta;			
		}
	}

	void FMLDeformerSampler::ExtractUnskinnedPositions(int32 LODIndex, TArray<FVector3f>& OutPositions) const
	{
		OutPositions.Reset();

		if (SkeletalMeshComponent == nullptr)
		{
			return;
		}

		USkeletalMesh* Mesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
		if (Mesh == nullptr)
		{
			return;
		}

		FSkeletalMeshLODRenderData& SkelMeshLODData = Mesh->GetResourceForRendering()->LODRenderData[LODIndex];
		const FPositionVertexBuffer& RenderPositions = SkelMeshLODData.StaticVertexBuffers.PositionVertexBuffer;
		const int32 NumRenderVerts = RenderPositions.GetNumVertices();
		const FSkeletalMeshModel* SkeletalMeshModel = Mesh->GetImportedModel();
		const TArray<int32>& ImportedVertexNumbers = SkeletalMeshModel->LODModels[LODIndex].MeshToImportVertexMap;

		// Get the originally imported vertex numbers from the DCC.
		if (ImportedVertexNumbers.Num() > 0)
		{
			// Store the vertex positions for the original imported vertices (8 vertices for a cube).
			OutPositions.AddZeroed(NumImportedVertices);
			for (int32 Index = 0; Index < NumRenderVerts; ++Index)
			{
				const int32 ImportedVertex = ImportedVertexNumbers[Index];
				OutPositions[ImportedVertex] = RenderPositions.VertexPosition(Index);
			}
		}
	}

	void FMLDeformerSampler::ExtractSkinnedPositions(int32 LODIndex, TArray<FVector3f>& OutPositions)
	{
		ExtractSkinnedPositions(LODIndex, BoneMatrices, TempVertexPositions, OutPositions);
	}

	void FMLDeformerSampler::ExtractSkinnedPositions(int32 LODIndex, TArray<FMatrix44f>& InBoneMatrices, TArray<FVector3f>& TempPositions, TArray<FVector3f>& OutPositions) const
	{
		OutPositions.Reset();
		TempPositions.Reset();

		if (SkeletalMeshComponent == nullptr)
		{
			return;
		}

		USkeletalMesh* Mesh = SkeletalMeshComponent->GetSkeletalMeshAsset();
		if (Mesh == nullptr)
		{
			return;
		}

		FSkeletalMeshLODRenderData& SkelMeshLODData = Mesh->GetResourceForRendering()->LODRenderData[LODIndex];
		FSkinWeightVertexBuffer* SkinWeightBuffer = SkeletalMeshComponent->GetSkinWeightBuffer(LODIndex);
		USkeletalMeshComponent::ComputeSkinnedPositions(SkeletalMeshComponent, TempPositions, InBoneMatrices, SkelMeshLODData, *SkinWeightBuffer);

		// Get the originally imported vertex numbers from the DCC.
		const FSkeletalMeshModel* SkeletalMeshModel = Mesh->GetImportedModel();
		const TArray<int32>& ImportedVertexNumbers = SkeletalMeshModel->LODModels[LODIndex].MeshToImportVertexMap;
		if (ImportedVertexNumbers.Num() > 0)
		{
			// Store the vertex positions for the original imported vertices (8 vertices for a cube).
			OutPositions.AddZeroed(NumImportedVertices);
			for (int32 Index = 0; Index < TempPositions.Num(); ++Index)
			{
				const int32 ImportedVertex = ImportedVertexNumbers[Index];
				OutPositions[ImportedVertex] = TempPositions[Index];
			}
		}
	}

	int32 FMLDeformerSampler::GetNumBones() const
	{
		const UMLDeformerInputInfo* InputInfo = EditorModel->GetEditorInputInfo();
		return InputInfo->GetNumBones();
	}

	void FMLDeformerSampler::CreateActors()
	{
		// Create the skeletal mesh Actor.
		if (SkelMeshActor.Get() == nullptr)
		{
			SkelMeshActor = CreateNewActor(EditorModel->GetWorld(), "SkelMeshSamplerActor");
			SkelMeshActor->SetActorTransform(FTransform::Identity);
		}

		// Create the skeletal mesh component.
		USkeletalMesh* SkeletalMesh = Model->GetSkeletalMesh();
		UAnimSequence* TrainingAnimSequence = EditorModel->GetTrainingInputAnim(AnimIndex)->GetAnimSequence();
		if (SkeletalMeshComponent.Get() == nullptr)
		{
			SkeletalMeshComponent = NewObject<UDebugSkelMeshComponent>(SkelMeshActor);
			SkeletalMeshComponent->RegisterComponent();
			SkelMeshActor->SetRootComponent(SkeletalMeshComponent);
		}
		FMLDeformerEditorModel::ChangeSkeletalMeshOnComponent(SkeletalMeshComponent, SkeletalMesh);
		SkeletalMeshComponent->SetAnimationMode(EAnimationMode::AnimationSingleNode);
		SkeletalMeshComponent->SetAnimation(TrainingAnimSequence);
		SkeletalMeshComponent->SetPosition(0.0f);
		SkeletalMeshComponent->SetPlayRate(1.0f);
		SkeletalMeshComponent->Play(false);
		SkeletalMeshComponent->SetVisibility(false);
		SkeletalMeshComponent->RefreshBoneTransforms();
		if (SkeletalMeshComponent->GetAnimInstance())
		{
			SkeletalMeshComponent->GetAnimInstance()->GetRequiredBones().SetUseRAWData(true);
		}

		if (TargetMeshActor.Get() == nullptr)
		{
			TargetMeshActor = CreateNewActor(EditorModel->GetWorld(), "TargetMeshSamplerActor");
			TargetMeshActor->SetActorTransform(FTransform::Identity);
		}
	}

	AActor* FMLDeformerSampler::CreateNewActor(UWorld* InWorld, const FName& Name) const
	{
		FActorSpawnParameters SpawnParams;
		SpawnParams.Name = MakeUniqueObjectName(InWorld, AActor::StaticClass(), Name);
		AActor* Actor = InWorld->SpawnActor<AActor>(SpawnParams);
		Actor->SetFlags(RF_Transient);
		return Actor;
	}

	SIZE_T FMLDeformerSampler::CalcMemUsagePerFrameInBytes() const
	{
		SIZE_T NumBytes = 0;
		NumBytes += VertexDeltas.GetAllocatedSize();
		NumBytes += BoneRotations.GetAllocatedSize();
		NumBytes += CurveValues.GetAllocatedSize();
		return NumBytes;
	}

}	// namespace UE::MLDeformer
