// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerTestHelpers.h"
#include "MLDeformerModule.h"
#include "MLDeformerEditorToolkit.h"
#include "MLDeformerEditorModule.h"
#include "MLDeformerModelRegistry.h"
#include "MLDeformerAsset.h"

namespace UE::MLDeformer
{	
	//--------------------------------------------------------------------------------------
	// FMLDeformerScopedEditor
	//--------------------------------------------------------------------------------------
	FMLDeformerScopedEditor::FMLDeformerScopedEditor(FMLDeformerEditorToolkit* InEditor)
	{
		Editor = InEditor;
	}

	FMLDeformerScopedEditor::FMLDeformerScopedEditor(UMLDeformerAsset* Asset)
	{
		Editor = FMLDeformerTestHelpers::OpenAssetEditor(Asset);		
	}

	FMLDeformerScopedEditor::FMLDeformerScopedEditor(const FString& AssetName)
	{
		Editor = FMLDeformerTestHelpers::OpenAssetEditor(AssetName);
	}

	FMLDeformerScopedEditor::~FMLDeformerScopedEditor()
	{
		if (Editor && bCloseEditor)
		{
			Editor->CloseWindow(EAssetEditorCloseReason::AssetEditorHostClosed);
		}
	}

	FMLDeformerEditorToolkit* FMLDeformerScopedEditor::Get() const
	{
		return Editor;
	}

	void FMLDeformerScopedEditor::SetCloseEditor(bool bCloseOnDestroy)
	{
		bCloseEditor = bCloseOnDestroy;
	}

	bool FMLDeformerScopedEditor::IsValid() const
	{
		return (Editor != nullptr);
	}

	FMLDeformerEditorToolkit* FMLDeformerScopedEditor::operator->() const
	{ 
		return Editor;
	}

	//--------------------------------------------------------------------------------------
	// FMLDeformerTestHelpers
	//--------------------------------------------------------------------------------------
	FMLDeformerEditorToolkit* FMLDeformerTestHelpers::OpenAssetEditor(UMLDeformerAsset* Asset)
	{
		if (Asset == nullptr)
		{
			UE_LOGF(LogMLDeformer, Error, "Asset is a nullptr.");
			return nullptr;
		}

		if (!GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Asset))
		{
			UE_LOGF(LogMLDeformer, Error, "Failed to open the asset editor for ML Deformer asset '%ls'", *Asset->GetName());
			return nullptr;
		}

		// This checks to see if the asset sub editor is loaded.
		FMLDeformerEditorToolkit* Editor = static_cast<FMLDeformerEditorToolkit*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(Asset, true));
		if (Editor == nullptr)
		{
			UE_LOGF(LogMLDeformer, Error, "Failed to open asset editor for ML Deformer asset %ls, as we couldn't find the editor.", *Asset->GetName());
			return nullptr;
		}

		return Editor;
	}

	FMLDeformerEditorToolkit* FMLDeformerTestHelpers::OpenAssetEditor(const FString& AssetName)
	{
		// AssetName is something like "/MLDeformerFramework/Tests/MLD_Rampage.MLD_Rampage"

		// Try to load the object from the package.
		UObject* Object = StaticLoadObject(UMLDeformerAsset::StaticClass(), NULL, *AssetName);
		UMLDeformerAsset* Asset = Cast<UMLDeformerAsset>(Object);
		if (Asset == nullptr)
		{
			UE_LOGF(LogMLDeformer, Error, "Failed to load ML Deformer asset from '%ls'", *AssetName);
			return nullptr;
		}

		return OpenAssetEditor(Asset);
	}

	bool FMLDeformerTestHelpers::PostModelPropertyChanged(UMLDeformerModel* Model, const FName PropertyName)
	{
		FProperty* Property = Model->GetClass()->FindPropertyByName(PropertyName);
		if (Property == nullptr)
		{
			UE_LOGF(LogMLDeformer, Error, "Failed to find property '%ls' in class '%ls'", *PropertyName.ToString(), *Model->GetName());
			return false;
		}

		FPropertyChangedEvent Event(Property, EPropertyChangeType::ValueSet);
		Model->PostEditChangeProperty(Event);
		return true;
	}

}	// namespace UE::MLDeformer
