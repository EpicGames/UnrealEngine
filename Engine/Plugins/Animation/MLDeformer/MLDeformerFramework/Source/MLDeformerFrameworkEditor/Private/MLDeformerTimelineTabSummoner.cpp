// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerTimelineTabSummoner.h"
#include "MLDeformerEditorToolkit.h"
#include "MLDeformerEditorStyle.h"
#include "MLDeformerEditorModel.h"
#include "MLDeformerModel.h"

#include "IDocumentation.h"
#include "SSimpleTimeSlider.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "SMLDeformerTimeline.h"

#define LOCTEXT_NAMESPACE "MLDeformerTimelineTabSummoner"

namespace UE::MLDeformer
{
	const FName FMLDeformerTimelineTabSummoner::TabID(TEXT("MLDeformerTimeline"));

	FMLDeformerTimelineTabSummoner::FMLDeformerTimelineTabSummoner(const TSharedRef<FMLDeformerEditorToolkit>& InEditor)
		: FWorkflowTabFactory(TabID, InEditor)
		, Editor(&InEditor.Get())
	{
		bIsSingleton = true; // only allow a single instance of this tab
		TabLabel = LOCTEXT("TimelineTabLabel", "Timeline");
		TabIcon = FSlateIcon(FMLDeformerEditorStyle::Get().GetStyleSetName(), "MLDeformer.Timeline.TabIcon");
		ViewMenuDescription = LOCTEXT("ViewMenu_Desc", "Timeline");
		ViewMenuTooltip = LOCTEXT("ViewMenu_ToolTip", "Show the ML Deformer timeline.");
	}

	TSharedPtr<SToolTip> FMLDeformerTimelineTabSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
	{
		return IDocumentation::Get()->CreateToolTip(
			LOCTEXT("TimelineTooltip", "The timeline widget that controls the offset in the training or test anim sequence."), 
			NULL, 
			TEXT("Shared/Editors/Persona"), 
			TEXT("MLDeformerTimeline_Window"));
	}

	TSharedRef<SWidget> FMLDeformerTimelineTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
	{
		TSharedRef<SMLDeformerTimeline> TimeSlider = SNew(SMLDeformerTimeline, Editor);

		Editor->SetTimeSlider(TimeSlider);
		// Add the time slider.
		TSharedRef<SHorizontalBox> Content = SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(EVerticalAlignment::VAlign_Top)
			[
				TimeSlider
			];

		return Content;
	}
}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE 
