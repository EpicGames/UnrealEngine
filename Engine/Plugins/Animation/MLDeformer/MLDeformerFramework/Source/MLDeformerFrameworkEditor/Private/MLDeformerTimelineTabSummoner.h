// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class IDetailsView;

namespace UE::MLDeformer
{
	class FMLDeformerEditorToolkit;

	struct FMLDeformerTimelineTabSummoner : public FWorkflowTabFactory
	{
	public:
		static const FName TabID;

		FMLDeformerTimelineTabSummoner(const TSharedRef<FMLDeformerEditorToolkit>& InEditor);
		virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
		virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override;

	protected:
		FMLDeformerEditorToolkit* Editor = nullptr;
		TSharedPtr<IDetailsView> DetailsView;
	};
}	// namespace UE::MLDeformer
