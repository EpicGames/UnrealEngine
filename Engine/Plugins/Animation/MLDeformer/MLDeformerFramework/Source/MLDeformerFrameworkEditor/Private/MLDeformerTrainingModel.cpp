// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerTrainingModel.h"
#include "MLDeformerEditorModel.h"
#include "MLDeformerModel.h"
#include "MLDeformerInputInfo.h"
#include "MLDeformerSampler.h"
#include "MLDeformerTrainingInputAnim.h"
#include "SkeletalMeshAttributes.h"
#include "Engine/SkeletalMesh.h"
#include "Misc/FileHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MLDeformerTrainingModel)

FName UMLDeformerTrainingModel::DefaultMaskName = FName("MLD_DefaultMask");

UMLDeformerModel* UMLDeformerTrainingModel::GetModel() const
{
	return EditorModel->GetModel();
}

void UMLDeformerTrainingModel::Init(UE::MLDeformer::FMLDeformerEditorModel* InEditorModel)
{
	EditorModel = InEditorModel;
	ResetSampling();
}

void UMLDeformerTrainingModel::SetEditorModel(UE::MLDeformer::FMLDeformerEditorModel* InModel)
{ 
	EditorModel = InModel;
}

UE::MLDeformer::FMLDeformerEditorModel* UMLDeformerTrainingModel::GetEditorModel() const
{
	return EditorModel;
}

int32 UMLDeformerTrainingModel::GetNumberSampleTransforms() const
{
	return EditorModel->GetEditorInputInfo()->GetNumBones();
}

int32 UMLDeformerTrainingModel::GetNumberSampleCurves() const
{
	return EditorModel->GetEditorInputInfo()->GetNumCurves();
}

int32 UMLDeformerTrainingModel::NumSamples() const
{
	return EditorModel->GetNumFramesForTraining();
}

void UMLDeformerTrainingModel::ResetSampling()
{
	NumTimesSampled.Reset();
	NumTimesSampled.AddZeroed(EditorModel->GetNumTrainingInputAnims());
	SampleAnimIndex = 0;
	MaskIndexPerSample.Reset();
	bFinishedSampling = !FindNextAnimToSample(SampleAnimIndex);
	MaskNames = GetTrainingInputAnimMasks();
}

int32 UMLDeformerTrainingModel::GetNumberSampleDeltas() const
{
	return EditorModel->GetEditorInputInfo()->GetNumBaseMeshVertices();
}

void UMLDeformerTrainingModel::SetNumFloatsPerCurve(int32 NumFloatsPerCurve)
{
	const int32 NumAnims = EditorModel->GetNumTrainingInputAnims();
	for (int32 AnimIndex = 0; AnimIndex < NumAnims; ++AnimIndex)
	{
		UE::MLDeformer::FMLDeformerSampler* Sampler = EditorModel->GetSamplerForTrainingAnim(AnimIndex);
		if (Sampler)
		{
			Sampler->SetNumFloatsPerCurve(NumFloatsPerCurve);
		}
	}
}

TArray<FName> UMLDeformerTrainingModel::GetTrainingInputAnimMasks() const
{
	TArray<FName> ValidMasks;

	ValidMasks.Add(DefaultMaskName);

	const int32 NumAnims = EditorModel->GetNumTrainingInputAnims();
	for (int32 AnimIndex = 0; AnimIndex < NumAnims; ++AnimIndex)
	{
		const FMLDeformerTrainingInputAnim* Anim = EditorModel->GetTrainingInputAnim(AnimIndex);
		const FName MaskName = Anim->GetVertexMask();
		if (MaskName.IsNone() || !Anim->IsEnabled())
		{
			continue;
		}

		TVertexAttributesConstRef<float> MaskData = EditorModel->FindVertexAttributes(MaskName);
		if (MaskData.IsValid())
		{
			ValidMasks.Add(MaskName);
		}
	}

	return MoveTemp(ValidMasks);
}

TArray<float> UMLDeformerTrainingModel::GetTrainingInputAnimMaskData(FName MaskName) const
{
	TArray<float> PerVertexValues;

	// Get the imported vertex numbers.
	constexpr int32 LodIndex = 0;
	const USkeletalMesh* SkeletalMesh = EditorModel->GetModel()->GetSkeletalMesh();
	FMeshDescription* MeshDescription = SkeletalMesh->GetMeshDescription(LodIndex);
	const TVertexAttributesConstRef<int32> ImportPointIndex = MeshDescription ? MeshDescription->VertexAttributes().GetAttributesRef<int32>(MeshAttribute::Vertex::ImportPointIndex) : TVertexAttributesConstRef<int32>(); 

	const TVertexAttributesConstRef<float> MaskData = EditorModel->FindVertexAttributes(MaskName);
	if (MaskData.IsValid())
	{
		const int32 NumModelVerts = EditorModel->GetModel()->GetNumBaseMeshVerts();
		PerVertexValues.SetNumZeroed(NumModelVerts);

		if (ImportPointIndex.IsValid())
		{
			check(ImportPointIndex.GetNumElements() == MaskData.GetNumElements());
			for (int32 Index = 0; Index < MaskData.GetNumElements(); ++Index)
			{
				PerVertexValues[ImportPointIndex[Index]] = MaskData.Get(Index);
			}
		}
		else
		{
			check(PerVertexValues.Num() == MaskData.GetNumElements());
			for (int32 Index = 0; Index < MaskData.GetNumElements(); ++Index)
			{
				PerVertexValues[Index] = MaskData.Get(Index);
			}
		}
	}
	else
	{
		if (MaskName != DefaultMaskName)
		{
			UE_LOGF(LogMLDeformer, Warning, "Failed to get vertex mask data for mask '%ls'. A fully painted mask will be used.", *MaskName.ToString());
		}
		
		const int32 NumModelVerts = EditorModel->GetModel()->GetNumBaseMeshVerts();
		PerVertexValues.SetNumUninitialized(NumModelVerts);		
		for (int32 Index = 0; Index < PerVertexValues.Num(); ++Index)
		{
			PerVertexValues[Index] = 1.0f;
		}
	}

	return MoveTemp(PerVertexValues);
}

int32 UMLDeformerTrainingModel::GetMaskIndexForAnimIndex(int32 AnimIndex) const
{
	const FMLDeformerTrainingInputAnim* Anim = EditorModel->GetTrainingInputAnim(AnimIndex);
	const FName AnimMaskName = Anim->GetVertexMask();
	if (AnimMaskName.IsNone())
	{
		return 0;
	}

	const int32 MaskIndex = MaskNames.Find(AnimMaskName);
	if (MaskIndex == INDEX_NONE)
	{
		return 0;
	}

	return MaskIndex;
}

bool UMLDeformerTrainingModel::SetCurrentSampleIndex(int32 Index)
{
	return NextSample();
}

bool UMLDeformerTrainingModel::GetNeedsResampling() const
{
	return EditorModel->GetResamplingInputOutputsNeeded();
}

void UMLDeformerTrainingModel::SetNeedsResampling(bool bNeedsResampling)
{
	EditorModel->SetResamplingInputOutputsNeeded(bNeedsResampling);
}

bool UMLDeformerTrainingModel::NextSample()
{
	return SampleNextFrame();
}

bool UMLDeformerTrainingModel::SampleNextFrame()
{
	UE_LOGF(LogMLDeformer, Warning, "Please override the SampleNextFrame method in your UMLDeformerTrainingModel inherited class.");
	return false;
}

void UMLDeformerTrainingModel::SetDeviceList(const TArray<FString>& DeviceNames, int32 PreferredDeviceIndex)
{
	UMLDeformerModel* Model = EditorModel->GetModel();
	Model->SetTrainingDeviceList(DeviceNames);

	if (Model->GetTrainingDevice().IsEmpty() || !DeviceNames.Contains(Model->GetTrainingDevice()))
	{
		check(DeviceNames.IsValidIndex(PreferredDeviceIndex));
		Model->SetTrainingDevice(DeviceNames[PreferredDeviceIndex]);
	}
}
