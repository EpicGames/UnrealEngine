// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerVizSettingsDetails.h"
#include "Animation/Skeleton.h"
#include "MLDeformerModule.h"
#include "MLDeformerModel.h"
#include "MLDeformerEditorModel.h"
#include "MLDeformerEditorModule.h"
#include "MLDeformerVizSettings.h"
#include "MLDeformerModelInstance.h"
#include "MLDeformerComponent.h"
#include "MLDeformerPerfCounter.h"
#include "MLDeformerEditorStyle.h"
#include "MLDeformerEditorToolkit.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailsView.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "Modules/ModuleManager.h"
#include "Layout/Margin.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "PropertyCustomizationHelpers.h"
#include "AssetRegistry/AssetData.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "SWarningOrErrorBox.h"
#include "GeometryCache.h"
#include "Animation/AnimSequence.h"
#include "Animation/MeshDeformer.h"
#include "Engine/SkeletalMesh.h"
#include "HAL/PlatformTime.h"

#define LOCTEXT_NAMESPACE "MLDeformerVizSettingsDetails"

namespace UE::MLDeformer
{
	// UE_DEPRECATED(5.4, Remove these constructors when StatsMainMemUsageGroup is removed and there is no longer warnings in the implicit functions)
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FMLDeformerVizSettingsDetails::FMLDeformerVizSettingsDetails(const FMLDeformerVizSettingsDetails&) = default;
	FMLDeformerVizSettingsDetails::FMLDeformerVizSettingsDetails(FMLDeformerVizSettingsDetails&&) = default;
	FMLDeformerVizSettingsDetails& FMLDeformerVizSettingsDetails::operator=(const FMLDeformerVizSettingsDetails&) = default;
	FMLDeformerVizSettingsDetails& FMLDeformerVizSettingsDetails::operator=(FMLDeformerVizSettingsDetails&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	// END UE_DEPRECATED(5.4)

	bool FMLDeformerVizSettingsDetails::UpdateMemberPointers(const TArray<TWeakObjectPtr<UObject>>& Objects)
	{
		Model = nullptr;
		VizSettings = nullptr;
		EditorModel = nullptr;

		if (Objects.Num() == 1)
		{
			FMLDeformerEditorModule& EditorModule = FModuleManager::LoadModuleChecked<FMLDeformerEditorModule>("MLDeformerFrameworkEditor");
			VizSettings = Cast<UMLDeformerVizSettings>(Objects[0]);	
			Model = VizSettings ? Cast<UMLDeformerModel>(VizSettings->GetOuter()) : nullptr;
			EditorModel = Model ? EditorModule.GetModelRegistry().GetEditorModel(Model) : nullptr;
		}

		return (Model != nullptr && VizSettings != nullptr && EditorModel != nullptr);
	}

	void FMLDeformerVizSettingsDetails::CreateCategories()
	{
		SharedCategoryBuilder = &DetailLayoutBuilder->EditCategory("Shared Settings", FText::GetEmpty(), ECategoryPriority::Important);
		TestAssetsCategory = &DetailLayoutBuilder->EditCategory("Test Assets", FText::GetEmpty(), ECategoryPriority::Important);
		LiveSettingsCategory = &DetailLayoutBuilder->EditCategory("Live Settings", FText::GetEmpty(), ECategoryPriority::Important);
		TrainingMeshesCategoryBuilder = &DetailLayoutBuilder->EditCategory("Training Meshes", FText::GetEmpty(), ECategoryPriority::Important);
		StatsCategoryBuilder = &DetailLayoutBuilder->EditCategory("Statistics", FText::GetEmpty(), ECategoryPriority::Default);
	}

	void FMLDeformerVizSettingsDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
	{
		DetailLayoutBuilder = &DetailBuilder;

		PerformanceMetricFormat.SetMaximumFractionalDigits(0);
		PerformanceMetricFormat.SetMinimumFractionalDigits(0);
		MemUsageMetricFormat.SetMaximumFractionalDigits(2);

		// Try update the member model, editormodel and viz settings pointers.
		TArray<TWeakObjectPtr<UObject>> Objects;
		DetailLayoutBuilder->GetObjectsBeingCustomized(Objects);
		if (!UpdateMemberPointers(Objects))
		{
			return;
		}

		// Create the categories.
		CreateCategories();

		const bool bShowTrainingData = VizSettings ? (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TrainingData) : true;
		const bool bShowTestData = VizSettings ? (VizSettings->GetVisualizationMode() == EMLDeformerVizMode::TestData) : true;

		// Shared settings.
		SharedCategoryBuilder->AddProperty(UMLDeformerVizSettings::GetDrawLabelsPropertyName(), UMLDeformerVizSettings::StaticClass());
		SharedCategoryBuilder->AddProperty(UMLDeformerVizSettings::GetLabelHeightPropertyName(), UMLDeformerVizSettings::StaticClass());
		SharedCategoryBuilder->AddProperty(UMLDeformerVizSettings::GetLabelScalePropertyName(), UMLDeformerVizSettings::StaticClass());
		SharedCategoryBuilder->AddProperty(UMLDeformerVizSettings::GetMeshSpacingPropertyName(), UMLDeformerVizSettings::StaticClass());

		// Test Assets.
		TestAssetsCategory->SetCategoryVisibility(bShowTestData);
	
		IDetailPropertyRow& TestAnimRow = TestAssetsCategory->AddProperty(UMLDeformerVizSettings::GetTestAnimSequencePropertyName(), UMLDeformerVizSettings::StaticClass());
		TestAnimRow.CustomWidget()
		.NameContent()
		[
			TestAnimRow.GetPropertyHandle()->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SObjectPropertyEntryBox)
			.PropertyHandle(TestAnimRow.GetPropertyHandle())
			.AllowedClass(UAnimSequence::StaticClass())
			.ObjectPath( (VizSettings && VizSettings->GetTestAnimSequence() )? VizSettings->GetTestAnimSequence()->GetPathName() : FString())
			.ThumbnailPool(DetailBuilder.GetThumbnailPool())
			.OnShouldFilterAsset(
				this, 
				&FMLDeformerVizSettingsDetails::FilterAnimSequences, 
				Model->GetSkeletalMesh() ? Model->GetSkeletalMesh()->GetSkeleton() : nullptr
			)
		];

		AddTestSequenceErrors();

		const FText AnimErrorText = EditorModel->GetIncompatibleSkeletonErrorText(Model->GetSkeletalMesh(), VizSettings->GetTestAnimSequence());
		FDetailWidgetRow& AnimErrorRow = TestAssetsCategory->AddCustomRow(FText::FromString("AnimSkeletonMisMatchError"))
			.Visibility(!AnimErrorText.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed)
			.WholeRowContent()
			[
				SNew(SBox)
				.Padding(FMargin(0.0f, 4.0f))
				[
					SNew(SWarningOrErrorBox)
					.MessageStyle(EMessageStyle::Warning)
					.Message(AnimErrorText)
				]
			];

		FIsResetToDefaultVisible IsResetVisible = FIsResetToDefaultVisible::CreateSP(this, &FMLDeformerVizSettingsDetails::IsResetToDefaultDeformerGraphVisible);
		FResetToDefaultHandler ResetHandler = FResetToDefaultHandler::CreateSP(this, &FMLDeformerVizSettingsDetails::OnResetToDefaultDeformerGraph);
		FResetToDefaultOverride ResetOverride = FResetToDefaultOverride::Create(IsResetVisible, ResetHandler);
		TestAssetsCategory->AddProperty(UMLDeformerVizSettings::GetDeformerGraphPropertyName(), UMLDeformerVizSettings::StaticClass()).OverrideResetToDefault(ResetOverride);

		AddDeformerGraphErrors();

		// Show a warning when no deformer graph has been selected.
		UObject* Graph = nullptr;
		TSharedRef<IPropertyHandle> DeformerGraphProperty = DetailBuilder.GetProperty(UMLDeformerVizSettings::GetDeformerGraphPropertyName(), UMLDeformerVizSettings::StaticClass());
		if (DeformerGraphProperty->GetValue(Graph) == FPropertyAccess::Result::Success)
		{
			const bool bShowError = (Graph == nullptr) && !Model->GetDefaultDeformerGraphAssetPath().IsEmpty();
			FDetailWidgetRow& GraphErrorRow = TestAssetsCategory->AddCustomRow(FText::FromString("GraphError"))
				.Visibility(bShowError ? EVisibility::Visible : EVisibility::Collapsed)
				.WholeRowContent()
				[
					SNew(SBox)
					.Padding(FMargin(0.0f, 4.0f))
					[
						SNew(SWarningOrErrorBox)
						.MessageStyle(EMessageStyle::Warning)
						.Message(LOCTEXT("GraphErrorText", "This model requires a deformer graph.\nWithout it linear skinning is used."))
					]
				];
		}

		AddGroundTruth();

		IDetailPropertyRow& CompareAssetRow = TestAssetsCategory->AddProperty(UMLDeformerVizSettings::GetCompareActorsPropertyName(), UMLDeformerVizSettings::StaticClass())
			.ShouldAutoExpand(true);

		LiveSettingsCategory->SetCategoryVisibility(bShowTestData);
		LiveSettingsCategory->AddProperty(UMLDeformerVizSettings::GetWeightPropertyName(), UMLDeformerVizSettings::StaticClass());
		LiveSettingsCategory->AddProperty(UMLDeformerVizSettings::GetAnimPlaySpeedPropertyName(), UMLDeformerVizSettings::StaticClass());
		LiveSettingsCategory->AddProperty(UMLDeformerVizSettings::GetTestingFrameNumberPropertyName(), UMLDeformerVizSettings::StaticClass());

		IDetailGroup& HeatMapGroup = LiveSettingsCategory->AddGroup("HeatMap", LOCTEXT("HeatMap", "Heat Map"), false, true);
		HeatMapGroup.AddPropertyRow(DetailBuilder.GetProperty(UMLDeformerVizSettings::GetShowHeatMapPropertyName(), UMLDeformerVizSettings::StaticClass()));
		HeatMapGroup.AddPropertyRow(DetailBuilder.GetProperty(UMLDeformerVizSettings::GetHeatMapModePropertyName(), UMLDeformerVizSettings::StaticClass()));
		HeatMapGroup.AddPropertyRow(DetailBuilder.GetProperty(UMLDeformerVizSettings::GetHeatMapMaxPropertyName(), UMLDeformerVizSettings::StaticClass()));
		HeatMapGroup.AddPropertyRow(DetailBuilder.GetProperty(UMLDeformerVizSettings::GetGroundTruthLerpPropertyName(), UMLDeformerVizSettings::StaticClass()));

		AddAdditionalSettings();

		// Debugging group.
		IDetailGroup& DebuggingGroup = LiveSettingsCategory->AddGroup("Debugging", LOCTEXT("DebuggingLabel", "Debugging"), false, true);
		FDetailWidgetRow& DebuggingRow = DebuggingGroup.HeaderRow();
		DebuggingRow.NameContent().Widget = 
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(FText::FromName(DebuggingGroup.GetGroupName()));	
		DebuggingRow.ValueContent().Widget =
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text_Lambda
			(
				[this]()
				{
					FMLDeformerEditorToolkit* Editor = EditorModel->GetEditor();
					if (Editor && IsValid(Editor->GetDebugActor()))
					{
						return LOCTEXT("DebuggingActiveText", "Debugging Active");
					}

					return GEditor->GetPIEViewport() ? LOCTEXT("DebuggingPIEActiveText", "PIE Active") : FText();
				}
			)
			.ColorAndOpacity_Lambda			
			(
				[this]()
				{
					return GEditor->GetPIEViewport() ? FSlateColor(FColor::Green) : FSlateColor::UseForeground();
				}
			);
		DebuggingGroup.AddPropertyRow(DetailBuilder.GetProperty(UMLDeformerVizSettings::GetDrawDebugActorBoundsPropertyName(), UMLDeformerVizSettings::StaticClass()));
		DebuggingGroup.AddPropertyRow(DetailBuilder.GetProperty(UMLDeformerVizSettings::GetDebugBoundsColorPropertyName(), UMLDeformerVizSettings::StaticClass()));

		// Visibility group.
		IDetailGroup& VisGroup = LiveSettingsCategory->AddGroup("Visibility", LOCTEXT("VisibilityLabel", "Visibility"), false, true);
		VisGroup.AddPropertyRow(DetailBuilder.GetProperty(UMLDeformerVizSettings::GetDrawLinearSkinnedActorPropertyName(), UMLDeformerVizSettings::StaticClass()));
		VisGroup.AddPropertyRow(DetailBuilder.GetProperty(UMLDeformerVizSettings::GetDrawMLDeformedActorPropertyName(), UMLDeformerVizSettings::StaticClass()));
		VisGroup.AddPropertyRow(DetailBuilder.GetProperty(UMLDeformerVizSettings::GetDrawGroundTruthActorPropertyName(), UMLDeformerVizSettings::StaticClass()))
			.Visibility(VizSettings->HasTestGroundTruth() ? EVisibility::Visible : EVisibility::Collapsed);
		VisGroup.AddPropertyRow(DetailBuilder.GetProperty(UMLDeformerVizSettings::GetDrawMLCompareActorsPropertyName(), UMLDeformerVizSettings::StaticClass()))
			.Visibility(!VizSettings->GetCompareActors().IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed);

		TrainingMeshesCategoryBuilder->SetCategoryVisibility(bShowTrainingData);
		TrainingMeshesCategoryBuilder->AddProperty(UMLDeformerVizSettings::GetTrainingFrameNumberPropertyName(), UMLDeformerVizSettings::StaticClass());
		TrainingMeshesCategoryBuilder->AddProperty(UMLDeformerVizSettings::GetDrawVertexDeltasPropertyName(), UMLDeformerVizSettings::StaticClass());
		TrainingMeshesCategoryBuilder->AddProperty(UMLDeformerVizSettings::GetXRayDeltasPropertyName(), UMLDeformerVizSettings::StaticClass());

		AddStatistics();
	}

	bool FMLDeformerVizSettingsDetails::FilterAnimSequences(const FAssetData& AssetData, USkeleton* Skeleton)
	{
		if (Skeleton && Skeleton->IsCompatibleForEditor(AssetData))
		{
			return false;
		}

		return true;
	}

	void FMLDeformerVizSettingsDetails::OnResetToDefaultDeformerGraph(TSharedPtr<IPropertyHandle> PropertyHandle)
	{
		if (EditorModel)
		{
			UMeshDeformer* MeshDeformer = EditorModel->LoadDefaultDeformerGraph();
			PropertyHandle->SetValue(MeshDeformer);
		}
	}

	bool FMLDeformerVizSettingsDetails::IsResetToDefaultDeformerGraphVisible(TSharedPtr<IPropertyHandle> PropertyHandle)
	{
		UObject* CurrentGraph = nullptr;
		PropertyHandle->GetValue(CurrentGraph);
		if (CurrentGraph == nullptr)
		{
			const FString DefaultPath = Model->GetDefaultDeformerGraphAssetPath();
			return !DefaultPath.IsEmpty();
		}

		if (EditorModel)
		{
			// Check if we already assigned the default asset.
			const FAssetData CurrentGraphAssetData(CurrentGraph);
			const FString CurrentPath = CurrentGraphAssetData.GetObjectPathString();
			const FString DefaultPath = Model->GetDefaultDeformerGraphAssetPath();
			return (DefaultPath != CurrentPath);
		}

		return false;
	}

	void FMLDeformerVizSettingsDetails::AddStatsPerfRow(IDetailGroup& Group, const FText& Label, const FMLDeformerEditorModel* InEditorModel, const FNumberFormattingOptions& Format, bool bHighlight, TFunctionRef<int32(const FMLDeformerEditorModel*)> GetCyclesFunction)
	{
		const FLinearColor HighlightColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.Statistics.Performance");

		Group.AddWidgetRow()
			.NameContent()
			[			
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(Label)
			]
			.ValueContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ColorAndOpacity(bHighlight ? HighlightColor : FSlateColor::UseForeground())
				.Text_Lambda
				(
					[EditorModel=InEditorModel, Format=Format, GetCyclesFunction]()
					{
						const int32 Cycles = GetCyclesFunction(EditorModel);
						const float MilliSeconds = FPlatformTime::ToMilliseconds(Cycles);
						return FText::Format(LOCTEXT("PerfTimeValue", "{0} \u00B5s"), FText::AsNumber(MilliSeconds * 1000.0f, &Format));
					}
				)
			];
	}

	void FMLDeformerVizSettingsDetails::AddStatistics()
	{
		const FLinearColor HighlightColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.Statistics.Performance");

		// Create the groups
		StatsPerformanceGroup = &StatsCategoryBuilder->AddGroup("CPU Performance", LOCTEXT("CPUPerformanceLabel", "CPU Performance"), false, true);
		StatsMemUsageGroup = &StatsCategoryBuilder->AddGroup("Estimated Memory Usage", LOCTEXT("MemoryUsageLabel", "Estimated Memory Usage"), false, true);
		StatsAssetSizeGroup = &StatsCategoryBuilder->AddGroup("Estimated Asset Sizes", LOCTEXT("AssetSize", "Estimated Asset Sizes"), false, true);

		AddStatsPerfRow(*StatsPerformanceGroup, LOCTEXT("AvgTickTimeLabel", "Avg Time"), EditorModel, PerformanceMetricFormat, true,
			[](const FMLDeformerEditorModel* EditorModelPtr)
			{
				UMLDeformerComponent* MLDeformerComponent = EditorModelPtr->FindMLDeformerComponent(ActorID_Test_MLDeformed);
				return MLDeformerComponent ? MLDeformerComponent->GetTickPerfCounter().GetCyclesAverage() : 0;
			});

		AddStatsPerfRow(*StatsPerformanceGroup, LOCTEXT("MinTickTimeLabel", "Min Time"), EditorModel, PerformanceMetricFormat, false,
			[](const FMLDeformerEditorModel* EditorModelPtr)
			{
				UMLDeformerComponent* MLDeformerComponent = EditorModelPtr->FindMLDeformerComponent(ActorID_Test_MLDeformed);
				return MLDeformerComponent ? MLDeformerComponent->GetTickPerfCounter().GetCyclesMin() : 0;
			});

		AddStatsPerfRow(*StatsPerformanceGroup, LOCTEXT("MaxTickTimeLabel", "Max Time"), EditorModel, PerformanceMetricFormat, false,
			[](const FMLDeformerEditorModel* EditorModelPtr)
			{
				UMLDeformerComponent* MLDeformerComponent = EditorModelPtr->FindMLDeformerComponent(ActorID_Test_MLDeformed);
				return MLDeformerComponent ? MLDeformerComponent->GetTickPerfCounter().GetCyclesMax() : 0;
			});

		StatsMemUsageGroup->AddWidgetRow()
			.NameContent()
			[			
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("MainMemoryLabel", "Main Memory"))
			]
			.ValueContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ColorAndOpacity(HighlightColor)
				.Text_Lambda
				(
					[this]
					{
						const uint64 TotalBytes = Model->GetMainMemUsageInBytes();
						return FText::Format(LOCTEXT("MainMemoryValue", "{0} mb"), FText::AsNumber(TotalBytes / static_cast<float>(1024*1024), &MemUsageMetricFormat));
					}
				)
			];

		StatsMemUsageGroup->AddWidgetRow()
			.NameContent()
			[			
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("GPUMemoryLabel", "GPU Memory"))
			]
			.ValueContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ColorAndOpacity(HighlightColor)
				.Text_Lambda
				(
					[this]
					{
						const uint64 TotalBytes = Model->GetGPUMemUsageInBytes();
						return FText::Format(LOCTEXT("GPUMemoryValue", "{0} mb"), FText::AsNumber(TotalBytes / static_cast<float>(1024*1024), &MemUsageMetricFormat));
					}
				)
			];

		StatsAssetSizeGroup->AddWidgetRow()
			.NameContent()
			[			
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("CookedAssetSizeLabel", "Cooked"))
			]
			.ValueContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ColorAndOpacity(HighlightColor)
				.Text_Lambda
				(
					[this]
					{
						const uint64 TotalBytes = Model->GetCookedAssetSizeInBytes();
						return FText::Format(LOCTEXT("CookedAssetSizeInBytesValue", "{0} mb"), FText::AsNumber(TotalBytes / static_cast<float>(1024*1024), &MemUsageMetricFormat));
					}
				)
			];

		StatsAssetSizeGroup->AddWidgetRow()
			.NameContent()
			[			
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("EditorAssetSizeLabel", "Uncooked"))
			]
			.ValueContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text_Lambda
				(
					[this]
					{
						const uint64 TotalBytes = Model->GetEditorAssetSizeInBytes();
						return FText::Format(LOCTEXT("EditorAssetSizeInBytesValue", "{0} mb"), FText::AsNumber(TotalBytes / static_cast<float>(1024*1024), &MemUsageMetricFormat));
					}
				)
			];
	}
}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
