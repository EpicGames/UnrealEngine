// Copyright Epic Games, Inc. All Rights Reserved.

#include "MLDeformerVizSettingsTabSummoner.h"
#include "DetailsViewArgs.h"
#include "MLDeformerEditorToolkit.h"
#include "MLDeformerEditorStyle.h"
#include "MLDeformerEditorModel.h"
#include "MLDeformerModel.h"
#include "IDocumentation.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "MLDeformerVizSettingsTabSummoner"

namespace UE::MLDeformer
{
	const FName FMLDeformerVizSettingsTabSummoner::TabID(TEXT("MLDeformerVizSettings"));

	FMLDeformerVizSettingsTabSummoner::FMLDeformerVizSettingsTabSummoner(const TSharedRef<FMLDeformerEditorToolkit>& InEditor)
		: FWorkflowTabFactory(TabID, InEditor)
		, Editor(InEditor)
	{
		bIsSingleton = true; // only allow a single instance of this tab

		TabLabel = LOCTEXT("VizSettingsTabLabel", "Visualization");
		TabIcon = FSlateIcon(FMLDeformerEditorStyle::Get().GetStyleSetName(), "MLDeformer.VizSettings.TabIcon");
		ViewMenuDescription = LOCTEXT("ViewMenu_Desc", "Visualization Settings");
		ViewMenuTooltip = LOCTEXT("ViewMenu_ToolTip", "Show the ML Deformer Debug Visualization Settings");

		// Create details view.
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		DetailsViewArgs.bHideSelectionTip = true;

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		Editor.Pin()->SetVizSettingsDetailsView(DetailsView);
	}

	TSharedPtr<SToolTip> FMLDeformerVizSettingsTabSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
	{
		return IDocumentation::Get()->CreateToolTip(
			LOCTEXT("VizSettingsTooltip", "The visualization settings for the ML Deformer."), 
			nullptr, 
			TEXT("Shared/Editors/Persona"), 
			TEXT("MLDeformerVizSettings_Window"));
	}

	TSharedRef<SWidget> FMLDeformerVizSettingsTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
	{
		TSharedRef<SVerticalBox> Content = SNew(SVerticalBox);
		Content->AddSlot()
			.FillHeight(1.0f)
			[
				DetailsView.ToSharedRef()
			];

		// Init the viz settings details panel.
		UMLDeformerVizSettings* VizSettings = Editor.Pin()->GetActiveModel() ? Editor.Pin()->GetActiveModel()->GetModel()->GetVizSettings() : nullptr;
		Editor.Pin()->GetVizSettingsDetailsView()->OnFinishedChangingProperties().AddSP(Editor.Pin().ToSharedRef(), &FMLDeformerEditorToolkit::OnFinishedChangingDetails);
		Editor.Pin()->GetVizSettingsDetailsView()->SetObject(VizSettings);

		return Content;
	}
}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE 
