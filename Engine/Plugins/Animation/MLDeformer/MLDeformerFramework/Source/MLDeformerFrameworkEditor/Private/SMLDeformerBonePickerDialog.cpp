// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMLDeformerBonePickerDialog.h"
#include "Widgets/Input/SSearchBox.h"
#include "Dialog/SCustomDialog.h"
#include "ReferenceSkeleton.h"

#define LOCTEXT_NAMESPACE "MLDeformerBonePickerDialog"

namespace UE::MLDeformer
{
	TSharedRef<ITableRow> FMLDeformerBonePickerTreeElement::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FMLDeformerBonePickerTreeElement> InTreeElement, TSharedPtr<SMLDeformerBonePickerTreeWidget> InTreeWidget)
	{
		return SNew(SMLDeformerBonePickerTreeRowWidget, InOwnerTable, InTreeElement, InTreeWidget);
	}

	void SMLDeformerBonePickerTreeWidget::Construct(const FArguments& InArgs)
	{
		PickerWidget = InArgs._PickerWidget;

		STreeView<TSharedPtr<FMLDeformerBonePickerTreeElement>>::FArguments SuperArgs;
		SuperArgs.TreeItemsSource(&RootElements);
		SuperArgs.SelectionMode(InArgs._AllowMultiSelect ? ESelectionMode::Multi : ESelectionMode::Single);
		SuperArgs.OnGenerateRow(this, &SMLDeformerBonePickerTreeWidget::MakeTableRowWidget);
		SuperArgs.OnGetChildren(this, &SMLDeformerBonePickerTreeWidget::HandleGetChildrenForTree);
		SuperArgs.OnMouseButtonDoubleClick(this, &SMLDeformerBonePickerTreeWidget::OnMouseDoubleClicked);
		SuperArgs.HighlightParentNodesForSelection(false);
		SuperArgs.AllowInvisibleItemSelection(true);

		STreeView<TSharedPtr<FMLDeformerBonePickerTreeElement>>::Construct(SuperArgs);
	}

	void SMLDeformerBonePickerTreeWidget::OnMouseDoubleClicked(TSharedPtr<FMLDeformerBonePickerTreeElement> ClickedItem)
	{
		if (ClickedItem.IsValid() && PickerWidget.IsValid())
		{
			PickerWidget->OnOkClicked();
			PickerWidget->RequestDestroyWindow();
		}
	}

	void SMLDeformerBonePickerTreeWidget::HandleGetChildrenForTree(TSharedPtr<FMLDeformerBonePickerTreeElement> InItem, TArray<TSharedPtr<FMLDeformerBonePickerTreeElement>>& OutChildren)
	{
		OutChildren = InItem.Get()->Children;
	}

	void SMLDeformerBonePickerTreeWidget::AddElement(TSharedPtr<FMLDeformerBonePickerTreeElement> Element, TSharedPtr<FMLDeformerBonePickerTreeElement> ParentElement)
	{
		if (!ParentElement)
		{
			RootElements.Add(Element);
		}
		else
		{
			ParentElement->Children.Add(Element);
		}
	}

	TSharedRef<ITableRow> SMLDeformerBonePickerTreeWidget::MakeTableRowWidget(TSharedPtr<FMLDeformerBonePickerTreeElement> InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return InItem->MakeTreeRowWidget(OwnerTable, InItem.ToSharedRef(), SharedThis(this));
	}

	void SMLDeformerBonePickerTreeWidget::Clear()
	{
		RootElements.Reset();
	}

	void SMLDeformerBonePickerTreeRowWidget::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FMLDeformerBonePickerTreeElement> InTreeElement, TSharedPtr<SMLDeformerBonePickerTreeWidget> InTreeView)
	{
		WeakTreeElement = InTreeElement;

		STableRow<TSharedPtr<FMLDeformerBonePickerTreeElement>>::Construct
		(
			STableRow<TSharedPtr<FMLDeformerBonePickerTreeElement>>::FArguments()
			.ShowWires(true)
			.Content()
			[
				SNew(STextBlock)
				.Text(this, &SMLDeformerBonePickerTreeRowWidget::GetName)
				.ColorAndOpacity_Lambda
				(
					[this]()
					{
						return WeakTreeElement.IsValid() ? WeakTreeElement.Pin()->TextColor : FSlateColor::UseForeground();
					}
				)
			], 
			OwnerTable
		);
	}

	FText SMLDeformerBonePickerTreeRowWidget::GetName() const
	{
		if (WeakTreeElement.IsValid())
		{
			return FText::FromName(WeakTreeElement.Pin()->Name);
		}
		return FText();
	}

	void SMLDeformerBonePickerDialog::RefreshBoneTreeElements() const
	{
		BoneTreeWidget->Clear();

		// Add all bones in the reference skeleton.
		TMap<FName, TSharedPtr<FMLDeformerBonePickerTreeElement>> NameToElementMap;
		const int32 NumRefBones = RefSkeleton->GetNum();
		for (int32 Index = 0; Index < NumRefBones; ++Index)
		{
			const FName RefBoneName = RefSkeleton->GetBoneName(Index);
			const int32 ParentIndex = RefSkeleton->GetParentIndex(Index);
			TSharedPtr<FMLDeformerBonePickerTreeElement> ParentElement;
			if (ParentIndex != INDEX_NONE)
			{
				const FName ParentBoneName = RefSkeleton->GetBoneName(ParentIndex);
				const TSharedPtr<FMLDeformerBonePickerTreeElement>* ParentElementPtr = NameToElementMap.Find(ParentBoneName);
				if (ParentElementPtr)
				{
					ParentElement = *ParentElementPtr;
				}
			}

			// Check if we want to include this bone.
			bool bIncludeBone = (FilterText.IsEmpty() || RefBoneName.ToString().Contains(FilterText));
			if (!IncludeList.IsEmpty())
			{
				bIncludeBone &= IncludeList.Contains(RefBoneName);
			}

			// Create the tree element.
			if (bIncludeBone)
			{
				TSharedPtr<FMLDeformerBonePickerTreeElement> Element = MakeShared<FMLDeformerBonePickerTreeElement>();
				Element->Name = RefBoneName;
				Element->TextColor = HighlightBoneNames.Contains(RefBoneName) ? HighlightColor : FSlateColor::UseForeground();
				BoneTreeWidget->AddElement(Element, ParentElement);
				BoneTreeWidget->SetItemExpansion(Element, true);
				NameToElementMap.Add(RefBoneName, Element);
			}
		}
	}

	void SMLDeformerBonePickerDialog::SelectInitialItemsRecursive(const TSharedPtr<FMLDeformerBonePickerTreeElement>& Item)
	{
		if (!Item.IsValid())
		{
			return;
		}

		const bool bSelected = InitialSelectedBoneNames.Contains(Item->Name);
		BoneTreeWidget->SetItemSelection(Item, bSelected, ESelectInfo::Type::Direct);
		
		for (const TSharedPtr<FMLDeformerBonePickerTreeElement>& ChildItem : Item->Children)
		{
			SelectInitialItemsRecursive(ChildItem);
		}
	}

	void SMLDeformerBonePickerDialog::SelectInitialItems()
	{
		for (const TSharedPtr<FMLDeformerBonePickerTreeElement>& Item : BoneTreeWidget->GetRootItems())
		{
			SelectInitialItemsRecursive(Item);
		}
	}

	TSharedRef<SMLDeformerBonePickerTreeWidget> SMLDeformerBonePickerDialog::CreateBoneTree()
	{
		TSharedRef<SMLDeformerBonePickerTreeWidget> TreeWidget = 
			SAssignNew(BoneTreeWidget, SMLDeformerBonePickerTreeWidget)
			.AllowMultiSelect(bAllowMultiSelect)
			.PickerWidget(SharedThis(this));

		RefreshBoneTreeElements();
		SelectInitialItems();
		return TreeWidget;
	}

	const TArray<FName>& SMLDeformerBonePickerDialog::GetPickedBoneNames() const
	{
		return PickedBoneNames;
	}

	void SMLDeformerBonePickerDialog::OnOkClicked()
	{
		const TArray<TSharedPtr<FMLDeformerBonePickerTreeElement>> SelectedElements = BoneTreeWidget->GetSelectedItems();
		PickedBoneNames.Reset();
		PickedBoneNames.SetNumUninitialized(SelectedElements.Num());

		for (int32 Index = 0; Index < SelectedElements.Num(); ++Index)
		{
			PickedBoneNames[Index] = SelectedElements[Index]->Name;
		}
	}

	void SMLDeformerBonePickerDialog::OnCancelClicked()
	{
		PickedBoneNames.Reset();
	}

	FReply SMLDeformerBonePickerDialog::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
	{
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			PickedBoneNames.Reset();
			RequestDestroyWindow();
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	void SMLDeformerBonePickerDialog::OnFilterTextChanged(const FText& InFilterText)
	{
		FilterText = InFilterText.ToString();
		RefreshBoneTreeElements();
		BoneTreeWidget->RequestTreeRefresh();
	}

	void SMLDeformerBonePickerDialog::Construct(const FArguments& InArgs)
	{
		HighlightBoneNames = InArgs._HighlightBoneNames;
		RefSkeleton = InArgs._RefSkeleton;
		HighlightColor = InArgs._HighlightBoneNamesColor;
		bAllowMultiSelect = InArgs._AllowMultiSelect;
		IncludeList = InArgs._IncludeList;
		ExtraWidget = InArgs._ExtraWidget;
		InitialSelectedBoneNames = InArgs._InitialSelectedBoneNames;

		FText DialogTitle;
		if (InArgs._Title.ToString().IsEmpty())
		{
			if (InArgs._AllowMultiSelect)
			{
				DialogTitle = LOCTEXT("DefaultTitleMulti", "Please select bones");
			}
			else
			{
				DialogTitle = LOCTEXT("DefaultTitleSingle", "Please select a single bone");
			}
		}
		else
		{
			DialogTitle = InArgs._Title;
		}

		const TSharedPtr<SWidget> FinalExtraWidget = ExtraWidget.IsValid() ? ExtraWidget : SNullWidget::NullWidget;		
		SCustomDialog::Construct
		(
			SCustomDialog::FArguments()
			.AutoCloseOnButtonPress(true)
			.Title(DialogTitle)
			.UseScrollBox(false)
			.Buttons(
			{
				SCustomDialog::FButton(LOCTEXT("OKText", "OK"))
				.SetPrimary(true)
				.SetOnClicked(FSimpleDelegate::CreateSP(this, &SMLDeformerBonePickerDialog::OnOkClicked))
				.SetFocus(),
				SCustomDialog::FButton(LOCTEXT("CancelText", "Cancel"))
				.SetOnClicked(FSimpleDelegate::CreateSP(this, &SMLDeformerBonePickerDialog::OnCancelClicked))
			})
			.Content()
			[
				SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.MinDesiredWidth(400.0f)
				.MinDesiredHeight(500.0f)
				.HAlign(EHorizontalAlignment::HAlign_Fill)
				.VAlign(EVerticalAlignment::VAlign_Fill)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SSearchBox)
						.HintText(LOCTEXT("SearchBonesHint", "Search Bones"))
						.OnTextChanged(this, &SMLDeformerBonePickerDialog::OnFilterTextChanged)
					]
					+SVerticalBox::Slot()
					.MaxHeight(500.0f)
					[
						CreateBoneTree()
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(FMargin(0.0f, 4.0f))
					[
						FinalExtraWidget.ToSharedRef()
					]
				]
			]
		);
	}
}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
