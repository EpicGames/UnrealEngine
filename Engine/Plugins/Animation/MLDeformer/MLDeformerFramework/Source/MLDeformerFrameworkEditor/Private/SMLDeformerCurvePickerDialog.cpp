// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMLDeformerCurvePickerDialog.h"
#include "MLDeformerInputInfo.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/SListView.h"
#include "Dialog/SCustomDialog.h"
#include "Animation/Skeleton.h"
#include "Animation/AnimCurveMetadata.h"
#include "Engine/SkeletalMesh.h"

#define LOCTEXT_NAMESPACE "MLDeformerCurvePickerWidget"

namespace UE::MLDeformer
{
	FMLDeformerCurvePickerElement::FMLDeformerCurvePickerElement(const FName InName, const FSlateColor& InTextColor)
	{
		Name = InName;
		TextColor = InTextColor;
	}

	TSharedRef<ITableRow> FMLDeformerCurvePickerElement::MakeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FMLDeformerCurvePickerElement> InElement, TSharedPtr<SMLDeformerCurvePickerListView> InListView)
	{
		return SNew(SMLDeformerCurvePickerRowWidget, InOwnerTable, InElement, InListView);
	}

	void SMLDeformerCurvePickerListView::Construct(const FArguments& InArgs)
	{
		PickerDialog = InArgs._PickerDialog;

		SListView<TSharedPtr<FMLDeformerCurvePickerElement>>::FArguments SuperArgs;
		SuperArgs.ListItemsSource(&Elements);
		SuperArgs.SelectionMode(InArgs._AllowMultiSelect ? ESelectionMode::Multi : ESelectionMode::Single);
		SuperArgs.OnGenerateRow(this, &SMLDeformerCurvePickerListView::MakeTableRowWidget);
		SuperArgs.OnMouseButtonDoubleClick(this, &SMLDeformerCurvePickerListView::OnMouseDoubleClicked);

		SListView<TSharedPtr<FMLDeformerCurvePickerElement>>::Construct(SuperArgs);
	}

	void SMLDeformerCurvePickerListView::Clear()
	{
		Elements.Reset();
	}

	void SMLDeformerCurvePickerListView::OnMouseDoubleClicked(TSharedPtr<FMLDeformerCurvePickerElement> ClickedItem)
	{
		if (ClickedItem.IsValid() && PickerDialog.IsValid())
		{
			PickerDialog->OnOkClicked();
			PickerDialog->RequestDestroyWindow();
		}
	}

	void SMLDeformerCurvePickerListView::AddElement(TSharedPtr<FMLDeformerCurvePickerElement> Element)
	{
		Elements.Add(Element);
	}

	TSharedRef<ITableRow> SMLDeformerCurvePickerListView::MakeTableRowWidget(TSharedPtr<FMLDeformerCurvePickerElement> InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return InItem->MakeRowWidget(OwnerTable, InItem.ToSharedRef(), SharedThis(this));
	}

	void SMLDeformerCurvePickerRowWidget::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FMLDeformerCurvePickerElement> InElement, TSharedPtr<SMLDeformerCurvePickerListView> InListView)
	{
		WeakElement = InElement;

		STableRow<TSharedPtr<FMLDeformerCurvePickerElement>>::Construct
		(
			STableRow<TSharedPtr<FMLDeformerCurvePickerElement>>::FArguments()
			.Padding(FMargin(4.0f, 0.0f))
			.Content()
			[
				SNew(STextBlock)
				.Text(this, &SMLDeformerCurvePickerRowWidget::GetName)
				.ColorAndOpacity_Lambda
				(
					[this]()
					{
						return WeakElement.IsValid() ? WeakElement.Pin()->TextColor : FSlateColor::UseForeground();
					}
				)
			], 
			OwnerTable
		);
	}

	FText SMLDeformerCurvePickerRowWidget::GetName() const
	{
		if (WeakElement.IsValid())
		{
			return FText::FromName(WeakElement.Pin()->Name);
		}
		return FText();
	}

	TSharedRef<SMLDeformerCurvePickerListView> SMLDeformerCurvePickerDialog::CreateListWidget()
	{
		TSharedRef<SMLDeformerCurvePickerListView> ListView = 
			SAssignNew(CurveListView, SMLDeformerCurvePickerListView)
			.AllowMultiSelect(bAllowMultiSelect)
			.PickerDialog(SharedThis(this));

		if (SkeletalMesh == nullptr)
		{
			return ListView;
		}

		RefreshListElements();
		return ListView;
	}

	const TArray<FName>& SMLDeformerCurvePickerDialog::GetPickedCurveNames() const
	{
		return PickedCurveNames;
	}

	void SMLDeformerCurvePickerDialog::OnOkClicked()
	{
		const TArray<TSharedPtr<FMLDeformerCurvePickerElement>> SelectedElements = CurveListView->GetSelectedItems();
		PickedCurveNames.Reset();
		PickedCurveNames.SetNumUninitialized(SelectedElements.Num());

		for (int32 Index = 0; Index < SelectedElements.Num(); ++Index)
		{
			PickedCurveNames[Index] = SelectedElements[Index]->Name;
		}
	}

	void SMLDeformerCurvePickerDialog::OnCancelClicked()
	{
		PickedCurveNames.Reset();
	}

	FReply SMLDeformerCurvePickerDialog::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
	{
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			PickedCurveNames.Reset();
			RequestDestroyWindow();
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}
	
	void SMLDeformerCurvePickerDialog::RefreshListElements() const 
	{
		CurveListView->Clear();

		TArray<FName> CurveNames;
		if (SkeletalMesh.IsValid())
		{
			UMLDeformerInputInfo::BuildCurveList(SkeletalMesh.Get(), CurveNames);
		}

		for (const FName CurveName : CurveNames)
		{
			// Check if we want to include this bone.
			bool bIncludeCurve = (FilterText.IsEmpty() || CurveName.ToString().Contains(FilterText));
			if (!IncludeList.IsEmpty())
			{
				bIncludeCurve &= IncludeList.Contains(CurveName);
			}

			if (bIncludeCurve)
			{
				TSharedPtr<FMLDeformerCurvePickerElement> Element = MakeShared<FMLDeformerCurvePickerElement>(CurveName, FSlateColor::UseForeground());
				Element->TextColor = (HighlightCurveNames.Contains(CurveName)) ? HighlightColor : FSlateColor::UseForeground();
				CurveListView->AddElement(Element);
			}
		}
	}

	void SMLDeformerCurvePickerDialog::OnFilterTextChanged(const FText& InFilterText)
	{
		FilterText = InFilterText.ToString();
		RefreshListElements();
		CurveListView->RequestListRefresh();
	}

	void SMLDeformerCurvePickerDialog::Construct(const FArguments& InArgs)
	{
		HighlightCurveNames = InArgs._HighlightCurveNames;
		SkeletalMesh = InArgs._SkeletalMesh;
		HighlightColor = InArgs._HighlightCurveNamesColor;
		bAllowMultiSelect = InArgs._AllowMultiSelect;
		IncludeList = InArgs._IncludeList;

		FText DialogTitle;
		if (InArgs._Title.ToString().IsEmpty())
		{
			if (InArgs._AllowMultiSelect)
			{
				DialogTitle = LOCTEXT("DefaultTitleMulti", "Please select curves");
			}
			else
			{
				DialogTitle = LOCTEXT("DefaultTitleSingle", "Please select a single curve");
			}
		}
		else
		{
			DialogTitle = InArgs._Title;
		}

		SCustomDialog::Construct
		(
			SCustomDialog::FArguments()
			.AutoCloseOnButtonPress(true)
			.Title(DialogTitle)
			.UseScrollBox(false)
			.Buttons(
			{
				SCustomDialog::FButton(LOCTEXT("OKText", "OK"))
				.SetPrimary(true)
				.SetOnClicked(FSimpleDelegate::CreateSP(this, &SMLDeformerCurvePickerDialog::OnOkClicked))
				.SetFocus(),
				SCustomDialog::FButton(LOCTEXT("CancelText", "Cancel"))
				.SetOnClicked(FSimpleDelegate::CreateSP(this, &SMLDeformerCurvePickerDialog::OnCancelClicked))
			})
			.Content()
			[
				SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				.MinDesiredWidth(400.0f)
				.MinDesiredHeight(500.0f)
				.HAlign(EHorizontalAlignment::HAlign_Fill)
				.VAlign(EVerticalAlignment::VAlign_Fill)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SSearchBox)
						.HintText(LOCTEXT("SearchCurvesHint", "Search Curves"))
						.OnTextChanged(this, &SMLDeformerCurvePickerDialog::OnFilterTextChanged)
					]
					+SVerticalBox::Slot()
					.MaxHeight(500.0f)
					[
						CreateListWidget()
					]
				]
			]
		);
	}
}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
