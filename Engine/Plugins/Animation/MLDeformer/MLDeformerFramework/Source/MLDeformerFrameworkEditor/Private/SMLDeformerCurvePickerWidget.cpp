// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMLDeformerCurvePickerWidget.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"

#define LOCTEXT_NAMESPACE "SCurvePickerWidget"

namespace UE::MLDeformer
{
	void SCurvePickerWidget::Construct(const FArguments& InArgs)
	{
		OnCurveNamePicked = InArgs._OnCurveNamePicked;
		OnGetSkeletalMesh = InArgs._OnGetSkeletalMesh;

		if (OnGetSkeletalMesh.IsBound())
		{
			SkeletalMesh = OnGetSkeletalMesh.Execute();
		}

		ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(SearchBox, SSearchBox)
					.HintText(LOCTEXT("SearchBoxHint", "Search Curves"))
					.OnTextChanged(this, &SCurvePickerWidget::HandleFilterTextChanged)
			]
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SAssignNew(NameListView, SListView<TSharedPtr<FString>>)
				.SelectionMode(ESelectionMode::Single)
				.ListItemsSource(&CurveNames)
				.OnSelectionChanged(this, &SCurvePickerWidget::HandleSelectionChanged)
				.OnGenerateRow(this, &SCurvePickerWidget::HandleGenerateRow)
			]
		];

		RefreshListItems();
	}

	void SCurvePickerWidget::HandleSelectionChanged(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectionType)
	{
		OnCurveNamePicked.ExecuteIfBound(**InItem.Get());
	}

	TSharedRef<ITableRow> SCurvePickerWidget::HandleGenerateRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& InOwnerTable)
	{
		return 
			SNew(STableRow<TSharedPtr<FString>>, InOwnerTable)
			[
				SNew(SBox)
				.MinDesiredHeight(20.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString(*InItem.Get()))
					.HighlightText_Lambda([this]() { return FText::FromString(FilterText); })
				]
			];
	}

	void SCurvePickerWidget::RefreshListItems()
	{
		CurveNames.Empty();
		UniqueCurveNames.Empty();

		const UAnimCurveMetaData* SkelMeshCurveData = SkeletalMesh->GetAssetUserData<UAnimCurveMetaData>();
		if (SkelMeshCurveData)
		{
			TArray<FName> SkelMeshCurveNames;
			SkelMeshCurveData->GetCurveMetaDataNames(SkelMeshCurveNames);
			for (const FName& CurveName : SkelMeshCurveNames)
			{
				UniqueCurveNames.Add(CurveName.ToString());
			}
		}
		
		if (SkeletalMesh->GetSkeleton())
		{
			TArray<FName> SkeletonCurveNames;
			SkeletalMesh->GetSkeleton()->GetCurveMetaDataNames(SkeletonCurveNames);
			for (const FName& CurveName : SkeletonCurveNames)
			{
				UniqueCurveNames.Add(CurveName.ToString());
			}
		}

		FilterAvailableCurves();
	}

	void SCurvePickerWidget::FilterAvailableCurves()
	{
		CurveNames.Empty();

		for (const FString& UniqueCurveName : UniqueCurveNames)
		{
			if (FilterText.IsEmpty() || UniqueCurveName.Contains(FilterText))
			{
				CurveNames.Add(MakeShared<FString>(UniqueCurveName));
			}
		}

		NameListView->RequestListRefresh();
	}

	void SCurvePickerWidget::HandleFilterTextChanged(const FText& InFilterText)
	{
		FilterText = InFilterText.ToString();
		FilterAvailableCurves();
	}

	/////////////////////////////////////////////////////

	void SCurveSelectionWidget::Construct(const FArguments& InArgs)
	{
		OnCurveSelectionChanged = InArgs._OnCurveSelectionChanged;
		OnGetSelectedCurve = InArgs._OnGetSelectedCurve;
		OnGetSkeletalMesh = InArgs._OnGetSkeletalMesh;

		SuppliedToolTip = InArgs._ToolTipText.Get();

		this->ChildSlot
		[
			SAssignNew(CurvePickerButton, SComboButton)
			.OnGetMenuContent(FOnGetContent::CreateSP(this, &SCurveSelectionWidget::CreateSkeletonWidgetMenu))
			.ContentPadding(FMargin(4.0f, 2.0f, 4.0f, 2.0f))
			.ButtonContent()
			[
				SNew(STextBlock)
					.Text(this, &SCurveSelectionWidget::GetCurrentCurveName)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ToolTipText(this, &SCurveSelectionWidget::GetFinalToolTip)
			]
		];
	}

	TSharedRef<SWidget> SCurveSelectionWidget::CreateSkeletonWidgetMenu()
	{
		bool bMultipleValues = false;

		TSharedRef<SCurvePickerWidget> ListWidget = SNew(SCurvePickerWidget)
			.OnCurveNamePicked(this, &SCurveSelectionWidget::OnSelectionChanged)
			.OnGetSkeletalMesh(OnGetSkeletalMesh);

		CurvePickerButton->SetMenuContentWidgetToFocus(ListWidget->GetFilterTextWidget());

		return ListWidget;
	}

	void SCurveSelectionWidget::OnSelectionChanged(const FString& CurveName)
	{
		SelectedCurve = FText::FromString(CurveName);
		if (OnCurveSelectionChanged.IsBound())
		{
			OnCurveSelectionChanged.Execute(CurveName);
		}
		CurvePickerButton->SetIsOpen(false);
	}

	FText SCurveSelectionWidget::GetCurrentCurveName() const
	{
		if (OnGetSelectedCurve.IsBound())
		{
			const FString Name = OnGetSelectedCurve.Execute();
			return FText::FromString(Name);
		}

		return FText::GetEmpty();
	}

	FText SCurveSelectionWidget::GetFinalToolTip() const
	{
		return FText::Format(LOCTEXT("CurveClickToolTip", "Curve: {0}\n\n{1}"), GetCurrentCurveName(), SuppliedToolTip);
	}

	USkeletalMesh* SCurveSelectionWidget::GetSkeletalMesh() const
	{
		if (OnGetSkeletalMesh.IsBound())
		{
			return OnGetSkeletalMesh.Execute();
		}

		return nullptr;
	}

}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
