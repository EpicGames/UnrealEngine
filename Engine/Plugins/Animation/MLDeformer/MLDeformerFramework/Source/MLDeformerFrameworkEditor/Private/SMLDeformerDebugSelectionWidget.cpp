// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMLDeformerDebugSelectionWidget.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SGridPanel.h"
#include "MLDeformerComponent.h"
#include "MLDeformerEditorToolkit.h"
#include "MLDeformerModel.h"
#include "MLDeformerEditorModel.h"
#include "MLDeformerEditorStyle.h"
#include "Components/SkeletalMeshComponent.h"
#include "Selection.h"
#include "Engine/World.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "MLDeformerDebugSelectionWidget"

namespace
{
	bool IsValidDebugWorld(UWorld* World)
	{
		return 
			IsValid(World) &&
			World->WorldType == EWorldType::PIE &&
			World->PersistentLevel != nullptr &&
			World->PersistentLevel->OwningWorld == World;
	}

	TArray<TObjectPtr<UWorld>> GetDebugWorlds()
	{
		TArray<TObjectPtr<UWorld>> Results;

		for (TObjectIterator<UWorld> It; It; ++It)
		{
			UWorld* World = *It;
			if (!IsValidDebugWorld(World))
			{
				continue;
			}

			Results.Emplace(TObjectPtr<UWorld>(World));
		}

		return MoveTemp(Results);
	}

	FString GetWorldName(UWorld* World)
	{
		FString WorldName;

		ENetMode NetMode = World->GetNetMode();
		switch (NetMode)
		{
			case NM_Standalone:
				WorldName = LOCTEXT("DebugWorldStandalone", "Standalone").ToString();
				break;

			case NM_ListenServer:
				WorldName = LOCTEXT("DebugWorldListenServer", "Listen Server").ToString();
				break;

			case NM_DedicatedServer:
				WorldName = LOCTEXT("DebugWorldDedicatedServer", "Dedicated Server").ToString();
				break;

			case NM_Client:
				if (FWorldContext* PieContext = GEngine->GetWorldContextFromWorld(World))
				{
					WorldName = FString::Printf(TEXT("%s %d"), *LOCTEXT("DebugWorldClient", "Client").ToString(), PieContext->PIEInstance - 1);
				}
				break;

			default:
				WorldName = LOCTEXT("UnknownNetModeName", "Unknown Net Mode").ToString();
		};

		if (FWorldContext* PieContext = GEngine->GetWorldContextFromWorld(World))
		{
			if (!PieContext->CustomDescription.IsEmpty())
			{
				WorldName += TEXT(" ") + PieContext->CustomDescription;
			}
		}

		return WorldName;
	}
}

namespace UE::MLDeformer
{
	void SMLDeformerDebugSelectionWidget::Construct(const FArguments& InArgs)
	{
		MLDeformerEditor = InArgs._MLDeformerEditor;

		ChildSlot
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(4, 0, 4, 0))
			.HAlign(EHorizontalAlignment::HAlign_Left)
			.VAlign(EVerticalAlignment::VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DebugActorText", "Debug Actor:"))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(2, 0, 0, 0))
			.HAlign(EHorizontalAlignment::HAlign_Left)
			.VAlign(EVerticalAlignment::VAlign_Center)
			[
				SAssignNew(ActorComboBox, SComboBox<TSharedPtr<FMLDeformerDebugActor>>)
				.OptionsSource(&Actors)
				.OnGenerateWidget(this, &SMLDeformerDebugSelectionWidget::OnGenerateActorComboBoxItemWidget)
				.OnSelectionChanged(this, &SMLDeformerDebugSelectionWidget::OnActorSelectionChanged)
				[
					SNew(STextBlock)
					.Text(this, &SMLDeformerDebugSelectionWidget::GetComboBoxText)
					.ColorAndOpacity_Lambda([this]()
					{
						const FLinearColor ActiveColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.Debug.ActiveDebugColor");
						return (IsDebuggingDisabled() || DebugActorName.IsEmpty()) ? FSlateColor::UseForeground() : FSlateColor(ActiveColor);
					})
				]
			]
		];

		FWorldDelegates::OnPostWorldInitialization.AddSPLambda(this, [this](UWorld* World, const UWorld::InitializationValues IVS) { Refresh(); });
		FWorldDelegates::OnWorldCleanup.AddSPLambda(this, [this](UWorld* World, bool bSessionEnded, bool bCleanupResources) { Refresh(); } );
		FWorldDelegates::OnPostDuplicate.AddSPLambda(this, [this](UWorld* World, bool bDuplicateForPIE, FWorldDelegates::FReplacementMap& ReplacementMap, TArray<UObject*>& ObjectsToFixReferences) { Refresh(); });
		FWorldDelegates::OnPostWorldRename.AddSPLambda(this, [this](UWorld* World) { Refresh(); } );
		
		FWorldDelegates::OnPIEReady.AddSPLambda(this, [this](UGameInstance*) { Refresh(); });
		FEditorDelegates::PostPIEStarted.AddSPLambda(this, [this](bool bSimulating) { Refresh(); });
		FEditorDelegates::PausePIE.AddSPLambda(this, [this](bool bSimulating) { Refresh(); });
		FEditorDelegates::ResumePIE.AddSPLambda(this, [this](bool bSimulating) { Refresh(); });
		FEditorDelegates::SingleStepPIE.AddSPLambda(this, [this](bool bSimulating) { Refresh(); });
		FEditorDelegates::EndPIE.AddSPLambda(this, [this](bool bSimulating) { Refresh(); });
		FEditorDelegates::CancelPIE.AddSPLambda(this, [this]() { Refresh(); });
		FEditorDelegates::OnNewActorsPlaced.AddSPLambda(this, [this](UObject*, const TArray<AActor*>&) { Refresh(); });
		FEditorDelegates::OnDeleteActorsBegin.AddSPLambda(this, [this]() { Refresh(); });
		FEditorDelegates::OnSwitchBeginPIEAndSIE.AddSPLambda(this, [this](bool bSimulating) { Refresh(); });

		USelection::SelectObjectEvent.AddSPLambda(this, [this](UObject* NewSelection) { Refresh(); });
		USelection::SelectionChangedEvent.AddSPLambda(this, [this](UObject* NewSelection) { Refresh(); });
		USelection::SelectNoneEvent.AddSPLambda(this, [this]() { Refresh(); });
	}

	FText SMLDeformerDebugSelectionWidget::GetComboBoxText() const
	{
		if (!IsDebuggingDisabled())
		{
			if (ActiveDebugActor.IsValid() && !DebugActorName.IsEmpty())
			{
				return DebugActorName;
			}
		}
		return LOCTEXT("EditorPreviewText", "Editor Preview (Debug Disabled)");
	}

	bool SMLDeformerDebugSelectionWidget::IsDebuggingDisabled() const
	{
		return Actors.IsEmpty() || !ActiveDebugActor.IsValid() || !IsValid(ActiveDebugActor->Actor);
	}

	void SMLDeformerDebugSelectionWidget::Refresh()
	{
		DebugActorName = FText();
		RefreshActorList();
	}

	AActor* SMLDeformerDebugSelectionWidget::GetDebugActor() const
	{
		return !DebugActorName.IsEmpty() && ActiveDebugActor.IsValid() && IsValid(ActiveDebugActor->Actor) ? ActiveDebugActor->Actor.Get() : nullptr;
	}

	TSharedRef<SWidget> SMLDeformerDebugSelectionWidget::OnGenerateActorComboBoxItemWidget(TSharedPtr<FMLDeformerDebugActor> Item)
	{
		// If we have the first item in the actor list, generate a special widget.
		const AActor* Actor = Item.IsValid() ? Item->Actor.Get () : nullptr;
		FText ActorName = Actor ? FText::FromString(Actor->GetName()) : LOCTEXT("DestroyedActorText", "<Destroyed>");

		const UWorld* World = Actor ? Actor->GetWorld() : nullptr;
		FText WorldName = World ? FText::FromString(GetDebugStringForWorld(World)) : LOCTEXT("DestroyedWorldText", "<Destroyed>");

		const FLinearColor ActiveColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.Debug.ActiveDebugColor");
		const FLinearColor EditorSelectedMarkColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.Debug.EditorSelectedMarkColor");

		// Special item to disable debugging.
		if (Actor == nullptr)
		{
			ActorName = LOCTEXT("DebugDisabledActorText", "Editor Preview (Disable Debug)");
			WorldName = LOCTEXT("DebugDisabledWorldText", "Editor Preview World");
		}

		TSharedPtr<SWidget> ItemWidget = 
			SNew(SGridPanel)
			+SGridPanel::Slot(0, 0)
			.Padding(2.0f)
			.HAlign(EHorizontalAlignment::HAlign_Right)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ActorName", "Actor:"))
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			]
			+SGridPanel::Slot(1, 0)
			.Padding(2.0f)
			.HAlign(EHorizontalAlignment::HAlign_Left)
			[
				SNew(STextBlock)
				.Text(ActorName)
				.Font_Lambda
				(
					[Item]()
					{
						if (Item.IsValid() && Item->bSelectedInEngine)
						{
							return FAppStyle::GetFontStyle(TEXT("NormalFontBold"));
						}
						return FAppStyle::GetFontStyle(TEXT("NormalFont"));
					}
				)
				.ColorAndOpacity_Lambda
				(
					[this, Item, ActiveColor]() 
					{ 
						const AActor* ItemActor = Item.IsValid() ? Item->Actor.Get() : nullptr;
						return (ActiveDebugActor.IsValid() && ActiveDebugActor->Actor == ItemActor) || (!ItemActor && DebugActorName.IsEmpty()) ? ActiveColor : FSlateColor::UseForeground();
					}
				)
			]
			+SGridPanel::Slot(2, 0)
			.Padding(2.0f)
			.HAlign(EHorizontalAlignment::HAlign_Left)
			[
				SNew(STextBlock)
				.Text_Lambda
				(
					[Item]()
					{
						if (Item.IsValid() && Item->bSelectedInEngine)
						{
							return LOCTEXT("SelectedText", "(Selected)");
						}
						return FText();
					}
				)
				.TextStyle(FAppStyle::Get(), "RichTextBlock.Bold")
				.ColorAndOpacity(EditorSelectedMarkColor)
			]
			+SGridPanel::Slot(0, 1)
			.Padding(2.0f)
			.HAlign(EHorizontalAlignment::HAlign_Right)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("WorldName", "World:"))
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			]
			+SGridPanel::Slot(1, 1)
			.Padding(2.0f)
			.HAlign(EHorizontalAlignment::HAlign_Left)
			[
				SNew(STextBlock)
				.Text(WorldName)
				.ColorAndOpacity_Lambda
				(
					[this, Item, ActiveColor]() 
					{ 
						const TObjectPtr<AActor> ItemActor = Item.IsValid() ? Item->Actor : nullptr;
						return (ActiveDebugActor.IsValid() && ActiveDebugActor->Actor == ItemActor) || (!ItemActor && DebugActorName.IsEmpty()) ? ActiveColor : FSlateColor::UseForeground();
					}
				)
			];

		return ItemWidget.ToSharedRef();
	}

	void SMLDeformerDebugSelectionWidget::OnActorSelectionChanged(TSharedPtr<FMLDeformerDebugActor> Item, ESelectInfo::Type SelectInfo)
	{
		ActiveDebugActor = Item;
		if (!Item.IsValid() || !IsValid(Item->Actor))
		{
			DebugActorName = FText();
		}
		else
		{
			if (IsValid(Item->Actor))
			{
				DebugActorName = IsValid(Item->Actor) ? FText::FromString(Item->Actor->GetName()) : FText();
			}
		}

		if (MLDeformerEditor && MLDeformerEditor->GetActiveModel())
		{
			MLDeformerEditor->GetActiveModel()->OnDebugActorChanged(ActiveDebugActor.IsValid() ? ActiveDebugActor->Actor : nullptr);
		}
	}

	TArray<TSharedPtr<FMLDeformerDebugActor>> SMLDeformerDebugSelectionWidget::GetDebugActorsForWorld(TObjectPtr<UWorld> World) const
	{
		TArray<TSharedPtr<FMLDeformerDebugActor>> Results;

		FMLDeformerEditorModel* ActiveModel = MLDeformerEditor->GetActiveModel();
		UMLDeformerModel* Model = ActiveModel ? ActiveModel->GetModel() : nullptr;
		USkeletalMesh* ModelSkeletalMesh = Model ? Model->GetSkeletalMesh() : nullptr;
		if (!Model || !ModelSkeletalMesh)
		{
			return Results;
		}

		// Iterate over all actors.
		for (TObjectIterator<AActor> It; It; ++It)
		{
			// Make sure the actor is alive and is using the world we're interested in.
			AActor* Actor = *It;
			if (IsValid(Actor) && Actor->GetOuter() && Actor->GetWorld() == World)
			{	
				if (Model->IsCompatibleDebugActor(Actor))
				{
					TSharedPtr<FMLDeformerDebugActor> NewActor = MakeShared<FMLDeformerDebugActor>();
					NewActor->Actor = Actor;
					NewActor->bSelectedInEngine = false;
					Results.Emplace(NewActor);
				}
			}
		}

		return MoveTemp(Results);
	}

	void SMLDeformerDebugSelectionWidget::RefreshActorList()
	{
		// Create an "Editor Preview (Disable Debug)" item, which is always the first item.
		// The preview item's Actor member is a nullptr.
		Actors.Reset();
		TSharedPtr<FMLDeformerDebugActor> PreviewItem = MakeShared<FMLDeformerDebugActor>();
		Actors.Emplace(PreviewItem);

		// Add actors of all the worlds we are interested in.
		const TArray<TObjectPtr<UWorld>> Worlds = GetDebugWorlds();
		for (UWorld* World : Worlds)
		{
			if (IsValidDebugWorld(World))
			{
				Actors.Append(GetDebugActorsForWorld(World));
			}
		}

		// Update the selected flags.
		UpdateDebugActorSelectedFlags();

		// Reselect the right item.
		for (const TSharedPtr<FMLDeformerDebugActor>& CurActor : Actors)
		{
			const AActor* ActiveActorObject = ActiveDebugActor.IsValid() ? ActiveDebugActor->Actor.Get() : nullptr;
			if (CurActor.IsValid() && CurActor->Actor.Get() == ActiveActorObject)
			{
				ActorComboBox->SetSelectedItem(CurActor);
				return;
			}
		}

		DebugActorName = FText();
		ActorComboBox->SetSelectedItem(PreviewItem);
	}

	void SMLDeformerDebugSelectionWidget::UpdateDebugActorSelectedFlags()
	{
		// First set all selected flags to false.
		for (TSharedPtr<FMLDeformerDebugActor>& DebugActor : Actors)
		{
			if (DebugActor.IsValid())
			{
				DebugActor->bSelectedInEngine = false;
			}
		}

		// Get the selected actors in the editor.
		const USelection* ActiveDebugActors = GEditor->GetSelectedActors();
		if (ActiveDebugActors != nullptr)
		{
			// Processed in reverse order, as we want the last selected item to be the one we pick.
			// There can only be one actor selected in the MLD editor to debug, while many can be selected in the editor itself.
			for (int32 Index = ActiveDebugActors->Num() - 1; Index >= 0; --Index)
			{
				// Get the object.
				const UObject* Object = ActiveDebugActors->GetSelectedObject(Index);
				if (Object == nullptr)
				{
					continue;
				}

				// Are we an Actor?
				const AActor* Actor = Cast<AActor>(Object);
				if (Actor == nullptr)
				{
					continue;
				}

				// Find the actor in the list.
				const TSharedPtr<FMLDeformerDebugActor>* DebugActor = Actors.FindByPredicate
				(
					[Actor](const TSharedPtr<FMLDeformerDebugActor>& DebugActor)
					{
						if (DebugActor.IsValid())
						{
							return DebugActor->Actor == Actor;
						}
						return false;
					} 
				);

				// Make sure this actor also is inside our debugging list.
				if (!DebugActor)
				{
					continue;
				}

				// Mark it as selected.
				if ((*DebugActor).IsValid())
				{
					(*DebugActor)->bSelectedInEngine = true;
					break;	// There can only be one selected actor, as we can't debug multiple at the same time.
				}
			}	// For all selected actors.
		}
	}

}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
