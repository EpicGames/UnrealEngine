// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboBox.h"
#include "Containers/UnrealString.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class UWorld;
class AActor;

namespace UE::MLDeformer
{
	class FMLDeformerEditorToolkit;
}

namespace UE::MLDeformer
{
	struct FMLDeformerDebugActor
	{
		/** The actor inside the PIE viewport, or nullptr when debugging is disabled. */
		TObjectPtr<AActor> Actor;

		/**
		 * Is this actor selected in the editor / engine?
		 * This is different from the Debug Actor we picked, we call that Active debug actor. 
		 */
		bool bSelectedInEngine = false;
	};

	/**
	 * 
	 */
	class SMLDeformerDebugSelectionWidget
		: public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SMLDeformerDebugSelectionWidget) {}
		SLATE_ARGUMENT(FMLDeformerEditorToolkit*, MLDeformerEditor)
		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs);

		/** Refresh the list of debuggable actors in the combobox. */
		UE_API void Refresh();

		/** Get the current actor we're debugging, or a nullptr when debugging is disabled. */
		UE_API AActor* GetDebugActor() const;

		const TArray<TSharedPtr<FMLDeformerDebugActor>>& GetActors() const { return Actors; }

	private:
		UE_API TSharedRef<SWidget> OnGenerateActorComboBoxItemWidget(TSharedPtr<FMLDeformerDebugActor> Item);
		UE_API TArray<TSharedPtr<FMLDeformerDebugActor>> GetDebugActorsForWorld(TObjectPtr<UWorld> World) const;
		UE_API FText GetComboBoxText() const;
		UE_API void OnActorSelectionChanged(TSharedPtr<FMLDeformerDebugActor> Item, ESelectInfo::Type SelectInfo);
		UE_API void RefreshActorList();
		UE_API void UpdateDebugActorSelectedFlags();
		UE_API bool IsDebuggingDisabled() const;

	private:
		/** The combobox that contains the actor names. */
		TSharedPtr<SComboBox<TSharedPtr<FMLDeformerDebugActor>>> ActorComboBox;

		/** The actors we can debug. */
		TArray<TSharedPtr<FMLDeformerDebugActor>> Actors;

		/** The selected actor. */
		TSharedPtr<FMLDeformerDebugActor> ActiveDebugActor;

		/** The name of the actor we are debugging. */
		FText DebugActorName;

		/** A pointer to our asset editor. */
		FMLDeformerEditorToolkit* MLDeformerEditor = nullptr;
	};

}	// namespace UE::MLDeformer

#undef UE_API
