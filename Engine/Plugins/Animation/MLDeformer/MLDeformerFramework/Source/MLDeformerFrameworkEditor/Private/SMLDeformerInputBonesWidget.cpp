// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMLDeformerInputBonesWidget.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Views/STreeView.h"
#include "MLDeformerEditorModel.h"
#include "MLDeformerInputInfo.h"
#include "MLDeformerModel.h"
#include "MLDeformerEditorStyle.h"
#include "SMLDeformerInputWidget.h"
#include "SMLDeformerBonePickerDialog.h"
#include "Engine/SkeletalMesh.h"
#include "ScopedTransaction.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"
#include "HAL/PlatformApplicationMisc.h"

#define LOCTEXT_NAMESPACE "MLDeformerInputBonesWidget"

namespace UE::MLDeformer
{
	TSharedRef<ITableRow> FMLDeformerInputBoneTreeElement::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FMLDeformerInputBoneTreeElement> InTreeElement, TSharedPtr<SMLDeformerInputBoneTreeWidget> InTreeWidget)
	{
		return SNew(SMLDeformerInputBoneTreeRowWidget, InOwnerTable, InTreeElement, InTreeWidget);
	}

	void SMLDeformerInputBoneTreeWidget::Construct(const FArguments& InArgs)
	{
		InputBonesWidget = InArgs._InputBonesWidget;

		STreeView<TSharedPtr<FMLDeformerInputBoneTreeElement>>::FArguments SuperArgs;
		SuperArgs.TreeItemsSource(&RootElements);
		SuperArgs.SelectionMode(ESelectionMode::Multi);
		SuperArgs.OnGenerateRow(this, &SMLDeformerInputBoneTreeWidget::MakeTableRowWidget);
		SuperArgs.OnGetChildren(this, &SMLDeformerInputBoneTreeWidget::HandleGetChildrenForTree);
		SuperArgs.OnSelectionChanged(this, &SMLDeformerInputBoneTreeWidget::OnSelectionChanged);
		SuperArgs.OnContextMenuOpening(this, &SMLDeformerInputBoneTreeWidget::OnContextMenuOpening);
		SuperArgs.HighlightParentNodesForSelection(false);
		SuperArgs.AllowInvisibleItemSelection(true);

		STreeView<TSharedPtr<FMLDeformerInputBoneTreeElement>>::Construct(SuperArgs);
	}

	void SMLDeformerInputBoneTreeWidget::OnSelectionChanged(TSharedPtr<FMLDeformerInputBoneTreeElement> Selection, ESelectInfo::Type SelectInfo)
	{
		TSharedPtr<SMLDeformerInputWidget> InputWidget = InputBonesWidget->GetInputWidget();
		check(InputWidget.IsValid());
		if (Selection.IsValid())
		{
			InputWidget->OnSelectInputBone(Selection.IsValid() ? Selection->Name : NAME_None);
		}
	}

	TSharedPtr<SWidget> SMLDeformerInputBoneTreeWidget::OnContextMenuOpening() const
	{
		const FMLDeformerInputBonesWidgetCommands& Actions = FMLDeformerInputBonesWidgetCommands::Get();
		FMenuBuilder Menu(true, InputBonesWidget->GetInputWidget()->GetBonesCommandList());
		Menu.BeginSection("BoneActions", LOCTEXT("BoneActionsHeading", "Bone Actions"));
		{
			Menu.AddMenuEntry(Actions.AddInputBones);
			Menu.AddMenuEntry(Actions.AddAnimatedBones);
			
			if (!GetSelectedItems().IsEmpty())
			{
				Menu.AddSeparator();
				Menu.AddMenuEntry(Actions.DeleteInputBones);
			}

			if (!GetRootElements().IsEmpty())
			{
				Menu.AddMenuEntry(Actions.ClearInputBones);
			}
		}
		Menu.EndSection();

		const FMLDeformerEditorModel* EditorModel = InputBonesWidget->EditorModel;
		InputBonesWidget->GetInputWidget()->AddInputBonesMenuItems(Menu);

		Menu.BeginSection("CopyPasteActions", LOCTEXT("CopyPasteActionsHeading", "Copy/Paste Actions"));
		{
			if (!RootElements.IsEmpty())
			{
				Menu.AddMenuEntry(Actions.CopyInputBones);
			}

			TSharedPtr<FJsonObject> JsonObject;
			if (InputBonesWidget->GetInputWidget()->HasValidClipBoardData(TEXT("InputBones"), JsonObject))
			{
				Menu.AddMenuEntry(Actions.PasteInputBones);
			}
		}
		Menu.EndSection();
		
		return Menu.MakeWidget();
	}

	void SMLDeformerInputBoneTreeWidget::HandleGetChildrenForTree(TSharedPtr<FMLDeformerInputBoneTreeElement> InItem, TArray<TSharedPtr<FMLDeformerInputBoneTreeElement>>& OutChildren)
	{
		OutChildren = InItem.Get()->Children;
	}

	void SMLDeformerInputBoneTreeWidget::AddElement(TSharedPtr<FMLDeformerInputBoneTreeElement> Element, TSharedPtr<FMLDeformerInputBoneTreeElement> ParentElement)
	{
		if (!ParentElement)
		{
			RootElements.Add(Element);
		}
		else
		{
			ParentElement->Children.Add(Element);
		}
	}

	void SMLDeformerInputBoneTreeWidget::RecursiveSortElements(TSharedPtr<FMLDeformerInputBoneTreeElement> Element)
	{
		if (Element.IsValid())
		{
			Element->Children.Sort(
				[](const TSharedPtr<FMLDeformerInputBoneTreeElement>& ItemA, const TSharedPtr<FMLDeformerInputBoneTreeElement>& ItemB)
				{
					return (ItemA->Name.ToString() < ItemB->Name.ToString());
				});

			for (const TSharedPtr<FMLDeformerInputBoneTreeElement>& Child : Element->Children)
			{
				RecursiveSortElements(Child);
			}
		}
		else // We need to sort the root level.
		{
			RootElements.Sort(
				[](const TSharedPtr<FMLDeformerInputBoneTreeElement>& ItemA, const TSharedPtr<FMLDeformerInputBoneTreeElement>& ItemB)
				{
					return (ItemA->Name.ToString() < ItemB->Name.ToString());
				});

			for (const TSharedPtr<FMLDeformerInputBoneTreeElement>& Child : RootElements)
			{
				RecursiveSortElements(Child);
			}
		}
	}

	void SMLDeformerInputBoneTreeWidget::RefreshElements(const TArray<FName>& BoneNames, const FReferenceSkeleton* RefSkeleton)
	{
		RootElements.Reset();

		const FSlateColor ErrorColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.InputsWidget.ErrorColor");

		// If we have no reference skeleton, just add everything as flat list as we don't have hierarchy data.
		const FString& FilterText = InputBonesWidget->FilterText;
		if (!RefSkeleton)
		{
			for (const FName BoneName : BoneNames)
			{
				if (FilterText.IsEmpty() || BoneName.ToString().Contains(FilterText))
				{
					TSharedPtr<FMLDeformerInputBoneTreeElement> Element = MakeShared<FMLDeformerInputBoneTreeElement>();
					Element->Name = BoneName;
					Element->TextColor = ErrorColor;
					RootElements.Add(Element);
				}
			}
		}
		else
		{
			// Add all the bones to some element map so we know the tree element for a given bone name.
			TMap<FName, TSharedPtr<FMLDeformerInputBoneTreeElement>> NameToElementMap;
			for (const FName BoneName : BoneNames)
			{
				if (!BoneName.ToString().Contains(FilterText))
				{
					continue;
				}

				const int32 RefSkelBoneIndex = RefSkeleton->FindBoneIndex(BoneName);			
				TSharedPtr<FMLDeformerInputBoneTreeElement> Element = MakeShared<FMLDeformerInputBoneTreeElement>();
				Element->Name = BoneName;
				Element->TextColor = (RefSkelBoneIndex != INDEX_NONE) ? FSlateColor::UseForeground() : ErrorColor;
				NameToElementMap.Add(BoneName, Element);
			}

			// Handle parents and specify root items.
			for (auto& Element : NameToElementMap)
			{
				TSharedPtr<FMLDeformerInputBoneTreeElement> ParentElement = FindParentElementForBone(Element.Key, *RefSkeleton, NameToElementMap);
				if (ParentElement.IsValid())
				{
					ParentElement->Children.Add(Element.Value);
				}
				else
				{
					RootElements.Add(Element.Value);
				}

				SetItemExpansion(Element.Value, true);
			}
		}

		RecursiveSortElements(TSharedPtr<FMLDeformerInputBoneTreeElement>());
	}

	TSharedPtr<FMLDeformerInputBoneTreeElement> SMLDeformerInputBoneTreeWidget::FindParentElementForBone(FName BoneName, const FReferenceSkeleton& RefSkeleton, const TMap<FName, TSharedPtr<FMLDeformerInputBoneTreeElement>>& NameToElementMap) const
	{
		const int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);
		if (BoneIndex != INDEX_NONE)
		{
			int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
			while (ParentIndex != INDEX_NONE)
			{
				const FName ParentName = RefSkeleton.GetBoneName(ParentIndex);
				const TSharedPtr<FMLDeformerInputBoneTreeElement>* ParentElementPtr = NameToElementMap.Find(ParentName);
				if (ParentElementPtr)
				{
					return *ParentElementPtr;
				}

				ParentIndex = RefSkeleton.GetParentIndex(ParentIndex);
			}
		}

		return TSharedPtr<FMLDeformerInputBoneTreeElement>();
	}

	TSharedRef<ITableRow> SMLDeformerInputBoneTreeWidget::MakeTableRowWidget(TSharedPtr<FMLDeformerInputBoneTreeElement> InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return InItem->MakeTreeRowWidget(OwnerTable, InItem.ToSharedRef(), SharedThis(this));
	}

	FReply SMLDeformerInputBoneTreeWidget::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
	{		
		TSharedPtr<FUICommandList> BonesCommandList = InputBonesWidget->GetInputWidget()->GetBonesCommandList();
		if (InputBonesWidget.IsValid() && BonesCommandList.IsValid() && BonesCommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}

		return STreeView<TSharedPtr<FMLDeformerInputBoneTreeElement>>::OnKeyDown(InGeometry, InKeyEvent);
	}

	void SMLDeformerInputBoneTreeWidget::RecursiveAddNames(const FMLDeformerInputBoneTreeElement& Element, TArray<FName>& OutNames) const
	{
		OutNames.Add(Element.Name);
		for (const TSharedPtr<FMLDeformerInputBoneTreeElement>& ChildElement : Element.Children)
		{
			RecursiveAddNames(*ChildElement, OutNames);
		}
	}

	TArray<FName> SMLDeformerInputBoneTreeWidget::ExtractAllElementNames() const
	{
		TArray<FName> Names;
		for (const TSharedPtr<FMLDeformerInputBoneTreeElement>& Element : RootElements)
		{
			RecursiveAddNames(*Element, Names);
		}
		return MoveTemp(Names);
	}

	void SMLDeformerInputBoneTreeRowWidget::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FMLDeformerInputBoneTreeElement> InTreeElement, TSharedPtr<SMLDeformerInputBoneTreeWidget> InTreeView)
	{
		WeakTreeElement = InTreeElement;

		STableRow<TSharedPtr<FMLDeformerInputBoneTreeElement>>::Construct
		(
			STableRow<TSharedPtr<FMLDeformerInputBoneTreeElement>>::FArguments()
			.ShowWires(true)
			.Content()
			[
				SNew(STextBlock)
				.Text(this, &SMLDeformerInputBoneTreeRowWidget::GetName)
				.ColorAndOpacity_Lambda
				(
					[this]()
					{
						return WeakTreeElement.IsValid() ? WeakTreeElement.Pin()->TextColor : FSlateColor::UseForeground();
					}
				)
			], 
			OwnerTable
		);
	}

	FText SMLDeformerInputBoneTreeRowWidget::GetName() const
	{
		if (WeakTreeElement.IsValid())
		{
			return FText::FromName(WeakTreeElement.Pin()->Name);
		}
		return FText();
	}

	void SMLDeformerInputBonesWidget::Construct(const FArguments& InArgs)
	{
		EditorModel = InArgs._EditorModel;
		InputWidget = InArgs._InputWidget;

		ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SSearchBox)
				.HintText(LOCTEXT("BonesSearchBoxHint", "Search Bones"))
				.OnTextChanged(this, &SMLDeformerInputBonesWidget::OnFilterTextChanged)
				.Visibility_Lambda(
					[this]() 
					{
						return (EditorModel->GetEditorInputInfo()->GetNumBones() > 0) ? EVisibility::Visible : EVisibility::Collapsed;
					})
			]
			+SVerticalBox::Slot()
			.MinHeight(75.0f)
			[
				SAssignNew(TreeWidget, SMLDeformerInputBoneTreeWidget)
				.InputBonesWidget(SharedThis(this))
			]
		];

		RefreshTree(false);
	}

	void SMLDeformerInputBonesWidget::OnFilterTextChanged(const FText& InFilterText)
	{
		FilterText = InFilterText.ToString();
		RefreshTree(false);
	}

	void SMLDeformerInputBonesWidget::BindCommands(TSharedPtr<FUICommandList> CommandList)
	{
		const FMLDeformerInputBonesWidgetCommands& Commands = FMLDeformerInputBonesWidgetCommands::Get();
		CommandList->MapAction(Commands.AddInputBones, FExecuteAction::CreateSP(this, &SMLDeformerInputBonesWidget::OnAddInputBones));
		CommandList->MapAction(Commands.DeleteInputBones, FExecuteAction::CreateSP(this, &SMLDeformerInputBonesWidget::OnDeleteInputBones));
		CommandList->MapAction(Commands.ClearInputBones, FExecuteAction::CreateSP(this, &SMLDeformerInputBonesWidget::OnClearInputBones));
		CommandList->MapAction(Commands.AddAnimatedBones, FExecuteAction::CreateSP(this, &SMLDeformerInputBonesWidget::OnAddAnimatedBones));
		CommandList->MapAction(Commands.CopyInputBones, FExecuteAction::CreateSP(this, &SMLDeformerInputBonesWidget::OnCopyInputBones));
		CommandList->MapAction(Commands.PasteInputBones, FExecuteAction::CreateSP(this, &SMLDeformerInputBonesWidget::OnPasteInputBones));
	}

	void SMLDeformerInputBonesWidget::OnCopyInputBones() const
	{
		const TSharedRef<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		JsonObject->SetStringField(TEXT("DataId"), TEXT("InputBones"));

		// Build the names array.
		TArray<TSharedPtr<FJsonValue>> JsonNameArray;
		for (const FName Name : EditorModel->GetEditorInputInfo()->GetBoneNames())
		{
			if (Name.IsValid() && !Name.IsNone())
			{
				JsonNameArray.Add(MakeShared<FJsonValueString>(Name.ToString()));
			}
		}		
		JsonObject->SetArrayField(TEXT("Names"), JsonNameArray);

		// Convert the JSON to a string and copy it to the clipboard.
		FString OutputString;
		const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
		if (!FJsonSerializer::Serialize(JsonObject, Writer))
		{
			UE_LOGF(LogMLDeformer, Error, "Failed to serialize JSON on Copy action in the Input Bones widget.");
			return;
		}
		FPlatformApplicationMisc::ClipboardCopy(*OutputString);
	}

	void SMLDeformerInputBonesWidget::OnPasteInputBones()
	{
		FString Text;
		FPlatformApplicationMisc::ClipboardPaste(Text);

		TSharedPtr<FJsonObject> JsonObject;
		if (GetInputWidget()->HasValidClipBoardData(TEXT("InputBones"), JsonObject) && JsonObject.IsValid())
		{
			TArray<FString> Names;
			if (!JsonObject->TryGetStringArrayField(TEXT("Names"), Names))
			{
				UE_LOGF(LogMLDeformer, Error, "Failed to get bone names during paste operation.");
			}
				
			FScopedTransaction Transaction(TEXT("SMLDeformerInputBonesWidget"), LOCTEXT("PasteInputBonesText", "Paste Input Bones"), EditorModel->GetModel());
			EditorModel->GetModel()->Modify();
		
			TArray<FName> AddedBones;
			TArray<FBoneReference>& CurrentBoneNames = EditorModel->GetModel()->GetBoneIncludeList();
			for (const FString& BoneNameString : Names)
			{
				if (!BoneNameString.IsEmpty())
				{
					const FName BoneName(BoneNameString);
					CurrentBoneNames.AddUnique(BoneName);
					AddedBones.Add(BoneName);
				}
			}
			
			EditorModel->UpdateEditorInputInfo();
			Refresh();

			// Trigger the input widgets events.
			// This is done AFTER the RefreshTree call, because that updates the editor input info.
			// Some handler code might depend on that to be updated first.
			GetInputWidget()->OnAddInputBones(AddedBones);
		}
	}

	void SMLDeformerInputBonesWidget::OnAddInputBones()
	{
		USkeletalMesh* SkelMesh = EditorModel->GetModel()->GetSkeletalMesh();
		if (SkelMesh)
		{
			const FSlateColor HighlightColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.InputsWidget.HighlightColor");

			TSharedPtr<SMLDeformerBonePickerDialog> Dialog = 
				SNew(SMLDeformerBonePickerDialog)
				.RefSkeleton(&SkelMesh->GetRefSkeleton())
				.AllowMultiSelect(true)
				.HighlightBoneNamesColor(HighlightColor)
				.HighlightBoneNames(TreeWidget->ExtractAllElementNames())
				.ExtraWidget(InputWidget->GetExtraBonePickerWidget());

			Dialog->ShowModal();

			const TArray<FName>& BoneNames = Dialog->GetPickedBoneNames();
			if (!BoneNames.IsEmpty())
			{
				UMLDeformerModel* Model = EditorModel->GetModel();
				FScopedTransaction Transaction(LOCTEXT("AddInputBonesText", "Add Input Bones"));
				Model->Modify();

				TArray<FName> BonesAdded;
				for (const FName BoneName : BoneNames)
				{
					TArray<FBoneReference>& ModelBoneIncludeList = Model->GetBoneIncludeList();
					
					if (!ModelBoneIncludeList.Contains(BoneName))
					{
						ModelBoneIncludeList.AddUnique(BoneName);
						BonesAdded.Add(BoneName);
					}
				}

				EditorModel->UpdateEditorInputInfo();
				RefreshTree();

				// Trigger the input widgets events.
				// This is done AFTER the RefreshTree call, because that updates the editor input info.
				// Some handler code might depend on that to be updated first.
				InputWidget->OnAddInputBones(BonesAdded);
			}
		}
		else
		{
			UE_LOGF(LogMLDeformer, Warning, "No skeleton is available to pick bones from");
		}
	}

	void SMLDeformerInputBonesWidget::OnClearInputBones()
	{
		UMLDeformerModel* Model = EditorModel->GetModel();
		FScopedTransaction Transaction(LOCTEXT("ClearInputBonesText", "Clear Input Bones"));
		Model->Modify();
		Model->GetBoneIncludeList().Empty();
		EditorModel->UpdateEditorInputInfo();
		RefreshTree();
		TreeWidget->ClearSelection();
		InputWidget->OnClearInputBones();
	}

	void SMLDeformerInputBonesWidget::OnDeleteInputBones()
	{
		const TArray<TSharedPtr<FMLDeformerInputBoneTreeElement>> SelectedItems = TreeWidget->GetSelectedItems();
		if (SelectedItems.IsEmpty())
		{
			return;
		}

		UMLDeformerModel* Model = EditorModel->GetModel();
		FScopedTransaction Transaction(LOCTEXT("RemoveInputBonesText", "Remove Input Bones"));
		Model->Modify();

		TArray<FBoneReference>& ModelBoneIncludeList = Model->GetBoneIncludeList();
		TArray<FName> BoneNamesToRemove;
		BoneNamesToRemove.Reserve(SelectedItems.Num());
		for (const TSharedPtr<FMLDeformerInputBoneTreeElement>& Item : SelectedItems)
		{
			check(Item.IsValid());
			ModelBoneIncludeList.Remove(Item->Name);
			BoneNamesToRemove.Add(Item->Name);
		}

		EditorModel->UpdateEditorInputInfo();
		RefreshTree();
		TreeWidget->ClearSelection();

		// Call the OnDeleteInputBone events after we informed the model about the bone removal.
		InputWidget->OnDeleteInputBones(BoneNamesToRemove);
	}

	void SMLDeformerInputBonesWidget::OnAddAnimatedBones()
	{
		UMLDeformerModel* Model = EditorModel->GetModel();
		FScopedTransaction Transaction(LOCTEXT("AddAnimatedInputBonesText", "Add Animated Input Bones"));
		Model->Modify();

		EditorModel->AddAnimatedBonesToBonesIncludeList();
		EditorModel->SetResamplingInputOutputsNeeded(true);
		EditorModel->UpdateEditorInputInfo();
		RefreshTree();
		InputWidget->OnAddAnimatedBones();
	}

	void SMLDeformerInputBonesWidget::RefreshTree(bool bBroadCastPropertyChanged)
	{
		UMLDeformerModel* Model = EditorModel->GetModel();

		if (bBroadCastPropertyChanged)
		{
			BroadcastModelPropertyChanged(Model->GetBoneIncludeListPropertyName());
		}

		USkeletalMesh* SkelMesh = Model->GetSkeletalMesh();
		const FReferenceSkeleton* RefSkeleton = SkelMesh ? &SkelMesh->GetRefSkeleton() : nullptr;
		UMLDeformerInputInfo* InputInfo = EditorModel->GetEditorInputInfo();

		TArray<FName> BoneNames;
		BoneNames.Reserve(Model->GetBoneIncludeList().Num());
		for (const FBoneReference& BoneRef : Model->GetBoneIncludeList())
		{
			BoneNames.Add(BoneRef.BoneName);
		}

		TreeWidget->RefreshElements(BoneNames, RefSkeleton);
		TreeWidget->RequestTreeRefresh();
		const int32 NumBonesIncluded = InputInfo ? InputInfo->GetNumBones() : 0;
		SectionTitle = FText::Format(FTextFormat(LOCTEXT("BonesTitle", "Bones ({0} / {1})")), NumBonesIncluded, BoneNames.Num());
	}

	bool SMLDeformerInputBonesWidget::BroadcastModelPropertyChanged(const FName PropertyName)
	{
		UMLDeformerModel* Model = EditorModel->GetModel();

		FProperty* Property = Model->GetClass()->FindPropertyByName(PropertyName);
		if (Property == nullptr)
		{
			UE_LOGF(LogMLDeformer, Error, "Failed to find property '%ls' in class '%ls'", *PropertyName.ToString(), *Model->GetName());
			return false;
		}

		FPropertyChangedEvent Event(Property, EPropertyChangeType::ValueSet);
		Model->PostEditChangeProperty(Event);
		return true;
	}

	FMLDeformerInputBonesWidgetCommands::FMLDeformerInputBonesWidgetCommands() 
		: TCommands<FMLDeformerInputBonesWidgetCommands>
	(	"ML Deformer Bone Inputs",
		NSLOCTEXT("MLDeformerInputBonesWidget", "MLDeformerInputsBonesDesc", "MLDeformer Bone Inputs"),
		NAME_None,
		FMLDeformerEditorStyle::Get().GetStyleSetName())
	{
	}

	void FMLDeformerInputBonesWidgetCommands::RegisterCommands()
	{
		UI_COMMAND(AddInputBones, "Add Bones", "Add bones to the list.", EUserInterfaceActionType::Button, FInputChord(EKeys::Insert));
		UI_COMMAND(DeleteInputBones, "Delete Selected", "Deletes the selected input bones.", EUserInterfaceActionType::Button, FInputChord(EKeys::Delete));
		UI_COMMAND(ClearInputBones, "Clear List", "Clears the entire list of input bones.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(AddAnimatedBones, "Add All Animated Bones", "Add all animated bones to the list.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(CopyInputBones, "Copy Input Bones List", "Copies all bones, so that you can paste them into another model's input bones list.", EUserInterfaceActionType::Button, FInputChord(EKeys::C, EModifierKey::Control));
		UI_COMMAND(PasteInputBones, "Paste Input Bones List", "Pastes a previously copied bone list. This will add bones to the existing list, not replace it fully.", EUserInterfaceActionType::Button, FInputChord(EKeys::V, EModifierKey::Control));
	}

	FText SMLDeformerInputBonesWidget::GetSectionTitle() const
	{ 
		return SectionTitle;
	}

	void SMLDeformerInputBonesWidget::Refresh()
	{ 
		RefreshTree(false);
	}

	TSharedPtr<SMLDeformerInputBoneTreeWidget> SMLDeformerInputBonesWidget::GetTreeWidget() const
	{ 
		return TreeWidget;
	}
	
	TSharedPtr<SMLDeformerInputWidget> SMLDeformerInputBonesWidget::GetInputWidget() const
	{ 
		return InputWidget;
	}

}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
