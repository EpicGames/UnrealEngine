// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMLDeformerInputCurvesWidget.h"
#include "SMLDeformerInputWidget.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Views/SListView.h"
#include "MLDeformerEditorModel.h"
#include "MLDeformerInputInfo.h"
#include "MLDeformerModel.h"
#include "MLDeformerEditorStyle.h"
#include "SMLDeformerCurvePickerDialog.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/Skeleton.h"
#include "ScopedTransaction.h"
#include "Animation/AnimCurveMetadata.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"
#include "HAL/PlatformApplicationMisc.h"


#define LOCTEXT_NAMESPACE "MLDeformerInputCurvesWidget"

namespace UE::MLDeformer
{
	TSharedRef<ITableRow> FMLDeformerInputCurveListElement::MakeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FMLDeformerInputCurveListElement> InElement, TSharedPtr<SMLDeformerInputCurveListWidget> InListWidget)
	{
		return SNew(SMLDeformerInputCurveListRowWidget, InOwnerTable, InElement, InListWidget);
	}

	void SMLDeformerInputCurveListWidget::Construct(const FArguments& InArgs)
	{
		InputCurvesWidget = InArgs._InputCurvesWidget;

		SListView<TSharedPtr<FMLDeformerInputCurveListElement>>::FArguments SuperArgs;
		SuperArgs.ListItemsSource(&Elements);
		SuperArgs.SelectionMode(ESelectionMode::Multi);
		SuperArgs.OnGenerateRow(this, &SMLDeformerInputCurveListWidget::MakeTableRowWidget);
		SuperArgs.OnContextMenuOpening(this, &SMLDeformerInputCurveListWidget::OnContextMenuOpening);
		SuperArgs.OnSelectionChanged(this, &SMLDeformerInputCurveListWidget::OnSelectionChanged);

		SListView<TSharedPtr<FMLDeformerInputCurveListElement>>::Construct(SuperArgs);
	}

	void SMLDeformerInputCurveListWidget::OnSelectionChanged(TSharedPtr<FMLDeformerInputCurveListElement> Selection, ESelectInfo::Type SelectInfo)
	{
		TSharedPtr<SMLDeformerInputWidget> InputWidget = InputCurvesWidget->GetInputWidget();
		check(InputWidget.IsValid());
		InputWidget->OnSelectInputCurve(Selection.IsValid() ? Selection->Name : NAME_None);
	}

	TSharedPtr<SWidget> SMLDeformerInputCurveListWidget::OnContextMenuOpening() const
	{
		const FMLDeformerInputCurvesWidgetCommands& Actions = FMLDeformerInputCurvesWidgetCommands::Get();
		FMenuBuilder Menu(true, InputCurvesWidget->GetInputWidget()->GetCurvesCommandList());
		Menu.BeginSection("CurveActions", LOCTEXT("CurveActionsHeading", "Curve Actions"));
		{
			Menu.AddMenuEntry(Actions.AddInputCurves);
			Menu.AddMenuEntry(Actions.AddAnimatedCurves);
			
			if (!GetSelectedItems().IsEmpty())
			{
				Menu.AddSeparator();
				Menu.AddMenuEntry(Actions.DeleteInputCurves);
			}

			if (!GetElements().IsEmpty())
			{
				Menu.AddMenuEntry(Actions.ClearInputCurves);
			}
		}
		Menu.EndSection();

		const FMLDeformerEditorModel* EditorModel = InputCurvesWidget->EditorModel;
		InputCurvesWidget->GetInputWidget()->AddInputCurvesMenuItems(Menu);

		Menu.BeginSection("CopyPasteActions", LOCTEXT("CopyPasteActionsHeading", "Copy/Paste Actions"));
		{
			if (!Elements.IsEmpty())
			{
				Menu.AddMenuEntry(Actions.CopyInputCurves);
			}

			TSharedPtr<FJsonObject> JsonObject;
			if (InputCurvesWidget->GetInputWidget()->HasValidClipBoardData(TEXT("InputCurves"), JsonObject))
			{
				Menu.AddMenuEntry(Actions.PasteInputCurves);
			}
		}
		Menu.EndSection();

		
		return Menu.MakeWidget();
	}

	void SMLDeformerInputCurveListWidget::AddElement(TSharedPtr<FMLDeformerInputCurveListElement> Element)
	{
		Elements.Add(Element);
	}

	void SMLDeformerInputCurveListWidget::RefreshElements(const TArray<FName>& CurveNames, USkeletalMesh* SkeletalMesh)
	{
		Elements.Reset();

		const FLinearColor ErrorColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.InputsWidget.ErrorColor");

		// If we have no skeletal mesh, just add everything as flat list as we don't have hierarchy data.
		if (SkeletalMesh == nullptr)
		{
			for (const FName CurveName : CurveNames)
			{
				if (InputCurvesWidget->FilterText.IsEmpty() || CurveName.ToString().Contains(InputCurvesWidget->FilterText))
				{
					TSharedPtr<FMLDeformerInputCurveListElement> Element = MakeShared<FMLDeformerInputCurveListElement>();
					Element->Name = CurveName;
					Element->TextColor = FSlateColor(ErrorColor);
					AddElement(Element);
				}
			}
		}
		else
		{
			TArray<FName> AssetCurveNames;
			UMLDeformerInputInfo::BuildCurveList(SkeletalMesh, AssetCurveNames);

			for (int32 CurveIndex = 0; CurveIndex < CurveNames.Num(); ++CurveIndex)
			{
				const FName CurveName = CurveNames[CurveIndex];
				const bool bExistsInSkel = AssetCurveNames.Contains(CurveName);

				if (InputCurvesWidget->FilterText.IsEmpty() || CurveName.ToString().Contains(InputCurvesWidget->FilterText))
				{
					TSharedPtr<FMLDeformerInputCurveListElement> Element = MakeShared<FMLDeformerInputCurveListElement>();
					Element->Name = CurveName;
					Element->TextColor = !bExistsInSkel ? FSlateColor(ErrorColor) : FSlateColor::UseForeground();
					AddElement(Element);
				}
			}
		}

		SortElements();
	}

	void SMLDeformerInputCurveListWidget::SortElements()
	{
		Elements.Sort(
			[](const TSharedPtr<FMLDeformerInputCurveListElement>& ItemA, const TSharedPtr<FMLDeformerInputCurveListElement>& ItemB)
			{
				return (ItemA->Name.ToString() < ItemB->Name.ToString());
			});
	}

	TSharedRef<ITableRow> SMLDeformerInputCurveListWidget::MakeTableRowWidget(TSharedPtr<FMLDeformerInputCurveListElement> InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return InItem->MakeRowWidget(OwnerTable, InItem.ToSharedRef(), SharedThis(this));
	}

	FReply SMLDeformerInputCurveListWidget::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
	{
		TSharedPtr<FUICommandList> CurvesCommandList = InputCurvesWidget->GetInputWidget()->GetCurvesCommandList();

		if (InputCurvesWidget.IsValid() && CurvesCommandList.IsValid() && CurvesCommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}

		return SListView<TSharedPtr<FMLDeformerInputCurveListElement>>::OnKeyDown(InGeometry, InKeyEvent);
	}

	void SMLDeformerInputCurveListRowWidget::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FMLDeformerInputCurveListElement> InElement, TSharedPtr<SMLDeformerInputCurveListWidget> InListView)
	{
		WeakElement = InElement;

		STableRow<TSharedPtr<FMLDeformerInputCurveListElement>>::Construct
		(
			STableRow<TSharedPtr<FMLDeformerInputCurveListElement>>::FArguments()
			.Padding(FMargin(4.0f, 0.0f))
			.Content()
			[
				SNew(STextBlock)
				.Text(this, &SMLDeformerInputCurveListRowWidget::GetName)
				.ColorAndOpacity_Lambda
				(
					[this]()
					{
						return WeakElement.IsValid() ? WeakElement.Pin()->TextColor : FSlateColor::UseForeground();
					}
				)
			], 
			OwnerTable
		);
	}

	FText SMLDeformerInputCurveListRowWidget::GetName() const
	{
		if (WeakElement.IsValid())
		{
			return FText::FromName(WeakElement.Pin()->Name);
		}
		return FText();
	}

	void SMLDeformerInputCurvesWidget::Construct(const FArguments& InArgs)
	{
		EditorModel = InArgs._EditorModel;
		InputWidget = InArgs._InputWidget;

		ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SSearchBox)
				.HintText(LOCTEXT("CurvesSearchBoxHint", "Search Curves"))
				.OnTextChanged(this, &SMLDeformerInputCurvesWidget::OnFilterTextChanged)
				.Visibility_Lambda(
					[this]() 
					{
						return (EditorModel->GetEditorInputInfo()->GetNumCurves() > 0) ? EVisibility::Visible : EVisibility::Collapsed;
					})
			]
			+SVerticalBox::Slot()
			.MinHeight(75.0f)
			[
				SAssignNew(ListWidget, SMLDeformerInputCurveListWidget)
				.InputCurvesWidget(SharedThis(this))
			]
		];

		RefreshList(false);
	}

	void SMLDeformerInputCurvesWidget::OnFilterTextChanged(const FText& InFilterText)
	{
		FilterText = InFilterText.ToString();
		RefreshList(false);
	}

	void SMLDeformerInputCurvesWidget::BindCommands(TSharedPtr<FUICommandList> CommandList)
	{
		const FMLDeformerInputCurvesWidgetCommands& Commands = FMLDeformerInputCurvesWidgetCommands::Get();
		CommandList->MapAction(Commands.AddInputCurves, FExecuteAction::CreateSP(this, &SMLDeformerInputCurvesWidget::OnAddInputCurves));
		CommandList->MapAction(Commands.DeleteInputCurves, FExecuteAction::CreateSP(this, &SMLDeformerInputCurvesWidget::OnDeleteInputCurves));
		CommandList->MapAction(Commands.ClearInputCurves, FExecuteAction::CreateSP(this, &SMLDeformerInputCurvesWidget::OnClearInputCurves));
		CommandList->MapAction(Commands.AddAnimatedCurves, FExecuteAction::CreateSP(this, &SMLDeformerInputCurvesWidget::OnAddAnimatedCurves));
		CommandList->MapAction(Commands.CopyInputCurves, FExecuteAction::CreateSP(this, &SMLDeformerInputCurvesWidget::OnCopyInputCurves));
		CommandList->MapAction(Commands.PasteInputCurves, FExecuteAction::CreateSP(this, &SMLDeformerInputCurvesWidget::OnPasteInputCurves));
	}


	void SMLDeformerInputCurvesWidget::OnCopyInputCurves() const
	{
		const TSharedRef<FJsonObject> JsonObject = MakeShared<FJsonObject>();
		JsonObject->SetStringField(TEXT("DataId"), TEXT("InputCurves"));

		// Build the names array.
		TArray<TSharedPtr<FJsonValue>> JsonNameArray;
		for (const FName Name : EditorModel->GetEditorInputInfo()->GetCurveNames())
		{
			if (Name.IsValid() && !Name.IsNone())
			{
				JsonNameArray.Add(MakeShared<FJsonValueString>(Name.ToString()));
			}
		}		
		JsonObject->SetArrayField(TEXT("Names"), JsonNameArray);

		// Convert the JSON to a string and copy it to the clipboard.
		FString OutputString;
		const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
		if (!FJsonSerializer::Serialize(JsonObject, Writer))
		{
			UE_LOGF(LogMLDeformer, Error, "Failed to serialize JSON on Copy action in the Input Curves widget.");
			return;
		}
		FPlatformApplicationMisc::ClipboardCopy(*OutputString);
	}

	void SMLDeformerInputCurvesWidget::OnPasteInputCurves()
	{
		FString Text;
		FPlatformApplicationMisc::ClipboardPaste(Text);

		TSharedPtr<FJsonObject> JsonObject;
		if (GetInputWidget()->HasValidClipBoardData(TEXT("InputCurves"), JsonObject) && JsonObject.IsValid())
		{
			TArray<FString> Names;
			if (!JsonObject->TryGetStringArrayField(TEXT("Names"), Names))
			{
				UE_LOGF(LogMLDeformer, Error, "Failed to get curve names during paste operation.");
			}
				
			FScopedTransaction Transaction(TEXT("SMLDeformerInputCurvesWidget"), LOCTEXT("PasteInputCurvesText", "Paste Input Curves"), EditorModel->GetModel());
			EditorModel->GetModel()->Modify();
		
			TArray<FName> AddedCurves;
			TArray<FMLDeformerCurveReference>& CurrentCurveNames = EditorModel->GetModel()->GetCurveIncludeList();
			for (const FString& CurveNameString : Names)
			{
				if (!CurveNameString.IsEmpty())
				{
					const FName CurveName(CurveNameString);
					CurrentCurveNames.AddUnique(CurveName);
					AddedCurves.Add(CurveName);
				}
			}
		
			EditorModel->UpdateEditorInputInfo();
			Refresh();

			// Trigger the input widgets events.
			// This is done AFTER the RefreshTree call, because that updates the editor input info.
			// Some handler code might depend on that to be updated first.
			GetInputWidget()->OnAddInputCurves(AddedCurves);
		}
	}

	void SMLDeformerInputCurvesWidget::OnAddInputCurves()
	{
		USkeletalMesh* SkelMesh = EditorModel->GetModel()->GetSkeletalMesh();
		if (SkelMesh)
		{
			const FLinearColor HighlightColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.InputsWidget.HighlightColor");

			TArray<FName> HighlightCurveNames;
			HighlightCurveNames.Reserve(ListWidget->GetElements().Num());
			for (const TSharedPtr<FMLDeformerInputCurveListElement>& Element : ListWidget->GetElements())
			{
				HighlightCurveNames.Add(Element->Name);
			}

			TSharedPtr<SMLDeformerCurvePickerDialog> Dialog = 
				SNew(SMLDeformerCurvePickerDialog)
				.SkeletalMesh(SkelMesh)
				.AllowMultiSelect(true)
				.HighlightCurveNamesColor(FSlateColor(HighlightColor))
				.HighlightCurveNames(HighlightCurveNames);

			Dialog->ShowModal();

			const TArray<FName>& PickerCurveNames = Dialog->GetPickedCurveNames();
			if (!PickerCurveNames.IsEmpty())
			{
				UMLDeformerModel* Model = EditorModel->GetModel();
				FScopedTransaction Transaction(TEXT("SMLDeformerInputCurvesWidget"), LOCTEXT("AddInputCurvesText", "Add Input Curves"), Model);
				Model->Modify();

				TArray<FName> CurvesAdded;
				CurvesAdded.Reserve(PickerCurveNames.Num());
				for (const FName CurveName : PickerCurveNames)
				{
					// Only add when the curve isn't already int he list.
					TArray<FMLDeformerCurveReference>& ModelCurveIncludeList = Model->GetCurveIncludeList();
					if (ModelCurveIncludeList.Find(CurveName) == INDEX_NONE)
					{
						ModelCurveIncludeList.Add(FMLDeformerCurveReference(CurveName));
						CurvesAdded.Add(CurveName);
					}
				}

				EditorModel->UpdateEditorInputInfo();
				RefreshList();

				// Trigger the input widgets events.
				// This is done AFTER the RefreshList call, because that updates the editor input info.
				// Some handler code might depend on that to be updated first.
				InputWidget->OnAddInputCurves(CurvesAdded);
			}
		}
		else
		{
			UE_LOGF(LogMLDeformer, Warning, "No skeletal mesh is available to pick curves from");
		}
	}

	void SMLDeformerInputCurvesWidget::OnClearInputCurves()
	{
		UMLDeformerModel* Model = EditorModel->GetModel();
		FScopedTransaction Transaction(LOCTEXT("ClearInputCurvesText", "Clear Input Curves"));
		Model->Modify();

		Model->GetCurveIncludeList().Empty();
		EditorModel->UpdateEditorInputInfo();
		RefreshList();
		ListWidget->ClearSelection();
		InputWidget->OnClearInputCurves();
	}

	void SMLDeformerInputCurvesWidget::OnDeleteInputCurves()
	{
		const TArray<TSharedPtr<FMLDeformerInputCurveListElement>> SelectedItems = ListWidget->GetSelectedItems();
		if (SelectedItems.IsEmpty())
		{
			return;
		}

		UMLDeformerModel* Model = EditorModel->GetModel();
		FScopedTransaction Transaction(LOCTEXT("RemoveInputCurvesText", "Remove Input Curves"));
		Model->Modify();

		TArray<FMLDeformerCurveReference>& ModelCurveIncludeList = Model->GetCurveIncludeList();
		TArray<FName> CurvesToRemove;
		CurvesToRemove.Reserve(SelectedItems.Num());
		for (const TSharedPtr<FMLDeformerInputCurveListElement>& Item : SelectedItems)
		{
			check(Item.IsValid());
			ModelCurveIncludeList.Remove(Item->Name);
			CurvesToRemove.Add(Item->Name);
		}

		EditorModel->UpdateEditorInputInfo();
		RefreshList();
		ListWidget->ClearSelection();

		// Call the OnDeleteInputCurve events after we informed the model about the curve removal.
		InputWidget->OnDeleteInputCurves(CurvesToRemove);
	}

	void SMLDeformerInputCurvesWidget::OnAddAnimatedCurves()
	{
		UMLDeformerModel* Model = EditorModel->GetModel();
		FScopedTransaction Transaction(LOCTEXT("AddAnimatedInputCurvesText", "Add Animated Input Curves"));
		Model->Modify();

		EditorModel->AddAnimatedCurvesToCurvesIncludeList();
		EditorModel->SetResamplingInputOutputsNeeded(true);
		EditorModel->UpdateEditorInputInfo();
		RefreshList();
		InputWidget->OnAddAnimatedCurves();
	}

	void SMLDeformerInputCurvesWidget::RefreshList(bool bBroadCastPropertyChanged)
	{
		UMLDeformerModel* Model = EditorModel->GetModel();

		if (bBroadCastPropertyChanged)
		{
			BroadcastModelPropertyChanged(Model->GetCurveIncludeListPropertyName());
		}

		TArray<FName> CurveNames;
		CurveNames.Reserve(Model->GetCurveIncludeList().Num());
		for (const FMLDeformerCurveReference& CurveRef : Model->GetCurveIncludeList())
		{
			CurveNames.Add(CurveRef.CurveName);		
		}

		ListWidget->RefreshElements(CurveNames, Model->GetSkeletalMesh());
		ListWidget->RequestListRefresh();

		UMLDeformerInputInfo* InputInfo = EditorModel->GetEditorInputInfo();
		const int32 NumCurvesIncluded = InputInfo ? InputInfo->GetNumCurves() : 0;
		SectionTitle = FText::Format(FTextFormat(LOCTEXT("CurvesTitle", "Curves ({0} / {1})")), NumCurvesIncluded, CurveNames.Num());
	}

	bool SMLDeformerInputCurvesWidget::BroadcastModelPropertyChanged(const FName PropertyName)
	{
		UMLDeformerModel* Model = EditorModel->GetModel();

		FProperty* Property = Model->GetClass()->FindPropertyByName(PropertyName);
		if (Property == nullptr)
		{
			UE_LOGF(LogMLDeformer, Error, "Failed to find property '%ls' in class '%ls'", *PropertyName.ToString(), *Model->GetName());
			return false;
		}

		FPropertyChangedEvent Event(Property, EPropertyChangeType::ValueSet);
		Model->PostEditChangeProperty(Event);
		return true;
	}

	FMLDeformerInputCurvesWidgetCommands::FMLDeformerInputCurvesWidgetCommands() 
		: TCommands<FMLDeformerInputCurvesWidgetCommands>
	(	"ML Deformer Curve Inputs",
		NSLOCTEXT("MLDeformerInputCurvesWidget", "MLDeformerInputsCurvesDesc", "MLDeformer Curve Inputs"),
		NAME_None,
		FMLDeformerEditorStyle::Get().GetStyleSetName())
	{
	}

	void FMLDeformerInputCurvesWidgetCommands::RegisterCommands()
	{
		UI_COMMAND(AddInputCurves, "Add Curves", "Add curves to the list.", EUserInterfaceActionType::Button, FInputChord(EKeys::Insert));
		UI_COMMAND(DeleteInputCurves, "Delete Selected", "Deletes the selected input curves.", EUserInterfaceActionType::Button, FInputChord(EKeys::Delete));
		UI_COMMAND(ClearInputCurves, "Clear List", "Clears the entire list of input curves.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(AddAnimatedCurves, "Add All Animated Curves", "Add all animated curves to the list.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(CopyInputCurves, "Copy Input Curves List", "Copies all curves, so that you can paste them into another model's input curves list.", EUserInterfaceActionType::Button, FInputChord(EKeys::C, EModifierKey::Control));
		UI_COMMAND(PasteInputCurves, "Paste Input Curves List", "Pastes a previously copied curves list. This will add curves to the existing list, not replace it fully.", EUserInterfaceActionType::Button, FInputChord(EKeys::V, EModifierKey::Control));
	}

	FText SMLDeformerInputCurvesWidget::GetSectionTitle() const
	{ 
		return SectionTitle;
	}

	void SMLDeformerInputCurvesWidget::Refresh()
	{ 
		RefreshList(false);
	}

	TSharedPtr<SMLDeformerInputCurveListWidget> SMLDeformerInputCurvesWidget::GetListWidget() const
	{ 
		return ListWidget;
	}

	TSharedPtr<SMLDeformerInputWidget> SMLDeformerInputCurvesWidget::GetInputWidget() const
	{ 
		return InputWidget;
	}

}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
