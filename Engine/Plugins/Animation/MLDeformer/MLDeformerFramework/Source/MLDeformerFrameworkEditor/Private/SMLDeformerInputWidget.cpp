// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMLDeformerInputWidget.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Views/SListView.h"
#include "Framework/Application/SlateApplication.h"
#include "Dialog/SCustomDialog.h"
#include "MLDeformerEditorModel.h"
#include "MLDeformerInputInfo.h"
#include "MLDeformerModel.h"
#include "MLDeformerEditorToolkit.h"
#include "SMLDeformerInputCurvesWidget.h"
#include "SMLDeformerInputBonesWidget.h"
#include "SMLDeformerBonePickerDialog.h"
#include "SMLDeformerCurvePickerDialog.h"
#include "Engine/SkeletalMesh.h"
#include "IDetailsView.h"
#include "Misc/Attribute.h"
#include "Internationalization/Text.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

#define LOCTEXT_NAMESPACE "MLDeformerInputWidget"

namespace UE::MLDeformer
{
	void SMLDeformerInputWidget::AddSection(TSharedPtr<SWidget> Widget, const FSectionInfo& SectionInfo)
	{
		SectionVerticalBox->AddSlot()
		.AutoHeight()
		[
			SNew(SExpandableArea)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
			.BodyBorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
			.HeaderPadding(FMargin(4.0f, 2.0f))
			.Padding(0.0f)
			.MaxHeight(300.0f)
			.AllowAnimatedTransition(false)
			.HeaderContent()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(SectionInfo.SectionTitle)
					.TextStyle(FAppStyle::Get(), "ButtonText")
					.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
				]
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.AutoWidth()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ToolTipText(SectionInfo.PlusButtonTooltip)
					.OnClicked(SectionInfo.PlusButtonPressed)
					.ContentPadding(FMargin(1, 0))					
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]
			.BodyContent()			
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
				.Padding(FMargin(8.0f, 2.0f))
				[
					Widget.ToSharedRef()
				]
			]
		];
	}

	FReply SMLDeformerInputWidget::ShowBonesPlusIconContextMenu()
	{
		const FMLDeformerInputBonesWidgetCommands& Actions = FMLDeformerInputBonesWidgetCommands::Get();
		FMenuBuilder Menu(true, BonesCommandList);
		Menu.BeginSection("BoneManagementActions", LOCTEXT("BoneManagementActionsHeading", "Bone Management"));
		{
			Menu.AddMenuEntry(Actions.AddInputBones);
			Menu.AddMenuEntry(Actions.AddAnimatedBones);

			if (!EditorModel->GetModel()->GetBoneIncludeList().IsEmpty())
			{
				Menu.AddMenuEntry(Actions.ClearInputBones);
			}
		}
		Menu.EndSection();

		AddInputBonesPlusIconMenuItems(Menu);

		FSlateApplication::Get().PushMenu(
			AsShared(),
			FWidgetPath(),
			Menu.MakeWidget(),
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect(FPopupTransitionEffect::TopMenu));

		return FReply::Handled();
	}

	void SMLDeformerInputWidget::CreateBonesWidget()
	{
		BonesCommandList = MakeShared<FUICommandList>();

		SAssignNew(InputBonesWidget, SMLDeformerInputBonesWidget)
			.EditorModel(EditorModel)
			.InputWidget(SharedThis(this));

		FSectionInfo SectionInfo;
		SectionInfo.SectionTitle = TAttribute<FText>::CreateSP(InputBonesWidget.Get(), &SMLDeformerInputBonesWidget::GetSectionTitle);
		SectionInfo.PlusButtonPressed = FOnClicked::CreateSP(this, &SMLDeformerInputWidget::ShowBonesPlusIconContextMenu);
		SectionInfo.PlusButtonTooltip = LOCTEXT("BonesPlusButtonTooltip", "Manage bone list.");		
		AddSection(InputBonesWidget, SectionInfo);

		InputBonesWidget->BindCommands(BonesCommandList);
	}

	void SMLDeformerInputWidget::CreateCurvesWidget()
	{
		CurvesCommandList = MakeShared<FUICommandList>();

		SAssignNew(InputCurvesWidget, SMLDeformerInputCurvesWidget)
			.EditorModel(EditorModel)
			.InputWidget(SharedThis(this));

		FSectionInfo SectionInfo;
		SectionInfo.SectionTitle = TAttribute<FText>::CreateSP(InputCurvesWidget.Get(), &SMLDeformerInputCurvesWidget::GetSectionTitle);
		SectionInfo.PlusButtonPressed = FOnClicked::CreateSP(this, &SMLDeformerInputWidget::ShowCurvesPlusIconContextMenu);
		SectionInfo.PlusButtonTooltip = LOCTEXT("CurvesPlusButtonTooltip", "Manage curve list.");
		AddSection(InputCurvesWidget, SectionInfo);

		InputCurvesWidget->BindCommands(CurvesCommandList);
	}

	FReply SMLDeformerInputWidget::ShowCurvesPlusIconContextMenu()
	{
		const FMLDeformerInputCurvesWidgetCommands& Actions = FMLDeformerInputCurvesWidgetCommands::Get();
		FMenuBuilder Menu(true, CurvesCommandList);
		Menu.BeginSection("CurveManagementActions", LOCTEXT("CurveManagementActionsHeading", "Curve Management"));
		{
			Menu.AddMenuEntry(Actions.AddInputCurves);
			Menu.AddMenuEntry(Actions.AddAnimatedCurves);

			if (!EditorModel->GetModel()->GetCurveIncludeList().IsEmpty())
			{
				Menu.AddMenuEntry(Actions.ClearInputCurves);
			}
		}
		Menu.EndSection();

		AddInputCurvesPlusIconMenuItems(Menu);

		FSlateApplication::Get().PushMenu(
			AsShared(),
			FWidgetPath(),
			Menu.MakeWidget(),
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect(FPopupTransitionEffect::TopMenu));

		return FReply::Handled();
	}


	void SMLDeformerInputWidget::AddSectionSeparator()
	{
		SectionVerticalBox->AddSlot()
		.AutoHeight()
		[
			SNew(SSeparator)
			.Thickness(2.0f)
		];
	}

	void SMLDeformerInputWidget::Construct(const FArguments& InArgs)
	{
		EditorModel = InArgs._EditorModel;
	
		ChildSlot
		[
			SAssignNew(SectionVerticalBox, SVerticalBox)
		];

		CreateBonesWidget();
		AddSectionSeparator();
		CreateCurvesWidget();
	}

	void SMLDeformerInputWidget::Refresh()
	{
		if (InputBonesWidget.IsValid())
		{
			InputBonesWidget->Refresh();
		}

		if (InputCurvesWidget.IsValid())
		{
			InputCurvesWidget->Refresh();
		}
	}

	void SMLDeformerInputWidget::ClearSelectionForAllWidgetsExceptThis(TSharedPtr<SWidget> ExceptThisWidget)
	{
		if (InputBonesWidget.IsValid() && ExceptThisWidget != InputBonesWidget->GetTreeWidget())
		{
			InputBonesWidget->GetTreeWidget()->ClearSelection();
		}

		if (InputCurvesWidget.IsValid() && ExceptThisWidget != InputCurvesWidget->GetListWidget())
		{
			InputCurvesWidget->GetListWidget()->ClearSelection();
		}
	}

	void SMLDeformerInputWidget::RegisterCommands()
	{
		FMLDeformerInputBonesWidgetCommands::Register();
		FMLDeformerInputCurvesWidgetCommands::Register();
	}

	void SMLDeformerInputWidget::UnregisterCommands()
	{
		FMLDeformerInputBonesWidgetCommands::Unregister();
		FMLDeformerInputCurvesWidgetCommands::Unregister();
	}

	TSharedPtr<FJsonObject> SMLDeformerInputWidget::GetJsonObject(const FString& JsonString)
	{
		TSharedPtr<FJsonObject> JsonObject;
		const TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonString);
		if (FJsonSerializer::Deserialize(Reader, JsonObject))
		{
			return JsonObject;
		}

		return TSharedPtr<FJsonObject>();
	}
	
	bool SMLDeformerInputWidget::ExtractStringFromJson(const TSharedPtr<FJsonObject>& JsonObject, const FString& Key, FString& OutString)
	{
		if (JsonObject.IsValid())
		{
			if (JsonObject->HasTypedField(Key, EJson::String))
			{
				OutString = JsonObject->GetStringField(Key);
				return true;
			}
		}
		return false;
	}
	
	bool SMLDeformerInputWidget::HasValidClipBoardData(const FString& DataTypeId, TSharedPtr<FJsonObject>& OutJsonObject)
	{
		// Get the text from the clipboard.
		FString Text;
		FPlatformApplicationMisc::ClipboardPaste(Text);

		const TSharedPtr<FJsonObject> JsonObject = GetJsonObject(Text);
		if (JsonObject.IsValid())
		{
			OutJsonObject = JsonObject;
			
			// Check if we find the "DataTypeId" field, and get it's value.
			// Then compare it to the data type id we are interested in.
			FString DataTypeIdValue;
			if (ExtractStringFromJson(JsonObject, TEXT("DataId"), DataTypeIdValue))
			{
				return DataTypeIdValue.Compare(DataTypeId, ESearchCase::IgnoreCase) == 0;
			}
		}

		// Invalid JSON or no DataTypeId key found, or the value didn't match the one we search for.
		OutJsonObject = JsonObject;
		return false;
	}

	TSharedPtr<FUICommandList> SMLDeformerInputWidget::GetBonesCommandList() const
	{ 
		return BonesCommandList;
	}

	TSharedPtr<FUICommandList> SMLDeformerInputWidget::GetCurvesCommandList() const
	{
		return CurvesCommandList;
	}

	TSharedPtr<SWidget> SMLDeformerInputWidget::GetExtraBonePickerWidget()
	{ 
		return TSharedPtr<SWidget>();
	}

}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
