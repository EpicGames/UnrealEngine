// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMLDeformerMaskConfigDialog.h"
#include "SMLDeformerNewVertexAttributeDialog.h"
#include "MLDeformerEditorModel.h"
#include "SkeletalMeshAttributes.h"
#include "MeshDescription.h"
#include "Engine/SkeletalMesh.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MLDeformerMaskConfigDialog"

namespace UE::MLDeformer
{
	void SMLDeformerMaskConfigDialog::Construct(const FArguments& InArgs, FMLDeformerEditorModel* InEditorModel)
	{
		check(InEditorModel);

		MaskInfo = InArgs._InitialMaskInfo;
		EditorModel = InEditorModel;
		OnSetNewVertexAttributeValues = InArgs._OnSetNewVertexAttributeValues;

		MaskingModeNames.Add(MakeShared<FString>(LOCTEXT("MaskModeNameGenerated", "Auto Generated").ToString()));
		MaskingModeNames.Add(MakeShared<FString>(LOCTEXT("MaskModeNameVertexAttribute", "Mesh Vertex Attribute").ToString()));

		// The widget that shows up when using vertex attributes.
		TSharedPtr<SWidget> MeshAttributeModeWidget = CreateMeshAttributeModeWidget();
		MeshAttributeModeWidget->SetVisibility(TAttribute<EVisibility>::CreateLambda([this]()
		{					
			return (MaskInfo.MaskMode == EMLDeformerMaskingMode::VertexAttribute) ? EVisibility::Visible : EVisibility::Hidden;
		}));

		UpdateAttributeNames();

		SCustomDialog::Construct
		(
			SCustomDialog::FArguments()
			.AutoCloseOnButtonPress(true)
			.Title(LOCTEXT("DialogTitle", "Mask Configuration"))
			.UseScrollBox(false)
			.Buttons(
			{
				SCustomDialog::FButton(LOCTEXT("OKText", "OK"))
				.SetPrimary(true)
				.SetFocus(),
				SCustomDialog::FButton(LOCTEXT("CancelText", "Cancel"))
			})
			.Content()
			[
				SNew(SBox)
				.Padding(FMargin(10.0f, 10.0f))
				.MinDesiredWidth(400.0f)
				.HAlign(EHorizontalAlignment::HAlign_Fill)
				.VAlign(EVerticalAlignment::VAlign_Fill)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(4.0f)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(4.0f)
						.VAlign(EVerticalAlignment::VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("MaskingModeLabel", "Masking Mode:"))
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(4.0f)
						.VAlign(EVerticalAlignment::VAlign_Center)
						[
							SNew(SComboBox<TSharedPtr<FString>>)
							.ToolTipText(LOCTEXT("MaskingModeToolTip", "Specify whether you would like to use auto-generated masks or use a vertex attribute on the mesh, which can be painted."))
							.OptionsSource(&MaskingModeNames)
							.OnGenerateWidget_Lambda([this](const TSharedPtr<FString>& Name)
							{
								return SNew(STextBlock)
								.Text(FText::FromString(*Name));
							})
							.OnSelectionChanged_Lambda([this](TSharedPtr<FString> Selected, ESelectInfo::Type)
							{
								const int32 Index = MaskingModeNames.Find(Selected);
								check(Index != INDEX_NONE);
								MaskInfo.MaskMode = static_cast<EMLDeformerMaskingMode>(Index);
							})
							[
								SNew(STextBlock)
								.Text_Lambda([this]() { return FText::FromString(*MaskModeEnumToString(MaskInfo.MaskMode)); }) 
							]
						]
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(4.0f)
					[
						MeshAttributeModeWidget.ToSharedRef()
					]
				]
			]
		);
	}

	void SMLDeformerMaskConfigDialog::UpdateAttributeNames()
	{
		check(EditorModel);
		AttributeNames = EditorModel->GetModel()->GetVertexAttributeNames();
	}

	TSharedPtr<FString> SMLDeformerMaskConfigDialog::MaskModeEnumToString(EMLDeformerMaskingMode MaskMode) const
	{
		const int32 Index = static_cast<int32>(MaskMode);
		return MaskingModeNames[Index];
	}

	TSharedPtr<SWidget> SMLDeformerMaskConfigDialog::CreateMeshAttributeModeWidget()
	{
		return 
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 2.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(4.0f, 0.0f)
				.AutoWidth()
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AttributeName", "Attribute Name:"))
				]
				+SHorizontalBox::Slot()
				.Padding(4.0f, 0.0f)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					SAssignNew(VertexAttributeComboWidget, SComboBox<FName>)
					.OptionsSource(&AttributeNames)
					.OnGenerateWidget_Lambda([this](const FName& Name)
					{
						return SNew(STextBlock)
						.Text(FText::FromName(Name));
					})
					.OnSelectionChanged_Lambda([this](FName Selected, ESelectInfo::Type)
					{
						MaskInfo.VertexAttributeName = Selected;
					})
					[
						SNew(STextBlock)
						.Text_Lambda([this]() { return FText::FromName(MaskInfo.VertexAttributeName); }) 
					]
				]
				+SHorizontalBox::Slot()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.Padding(0.0f, 2.0f)
				.AutoWidth()
				[
					SNew(SButton)
					.Text(LOCTEXT("CreateButton", "Create New"))
					.ToolTipText(LOCTEXT("CreateButtonTooltip", "Create a new vertex attribute on the Skeletal Mesh. This will modify the Skeletal Mesh, so please make sure to save it."))
					.HAlign(EHorizontalAlignment::HAlign_Center)
					.VAlign(EVerticalAlignment::VAlign_Center)
					.OnClicked_Raw(this, &SMLDeformerMaskConfigDialog::OnCreateVertexAttributeClicked)
				]
			];
	}

	FReply SMLDeformerMaskConfigDialog::OnCreateVertexAttributeClicked()
	{
		USkeletalMesh* SkeletalMesh = EditorModel->GetModel()->GetSkeletalMesh();
		if (!SkeletalMesh)
		{
			return FReply::Handled();
		}

		// Show the Create New VertexAttribute dialog, which allows the user to enter a name to create a new one.
		TSharedPtr<SMLDeformerNewVertexAttributeDialog> Dialog = SNew(SMLDeformerNewVertexAttributeDialog, SkeletalMesh)
			.bAutoCreateAttribute(true)
			.DefaultAttributeValue(0.0f);

		const SMLDeformerNewVertexAttributeDialog::EReturnCode ReturnCode = Dialog->ShowModal();

		// If we created a new attribute, update the list of attributes and auto select the new one.
		// Also initialize the vertex values to the mask as if it was generated.
		if (ReturnCode == SMLDeformerNewVertexAttributeDialog::EReturnCode::CreatePressed)
		{
			// Update the combobox items.
			UpdateAttributeNames();

			// Select the newly created item directly.
			const FName NewAttributeName = *Dialog->GetAttributeName();
			if (VertexAttributeComboWidget.IsValid())
			{
				VertexAttributeComboWidget->SetSelectedItem(NewAttributeName);
			}

			// Generate the mask and set the values to the vertex attributes.
			const int32 LODIndex = 0;
			FMeshDescription* MeshDescription = SkeletalMesh->GetMeshDescription(LODIndex);
			if (MeshDescription)
			{
				check(MeshDescription->VertexAttributes().HasAttribute(NewAttributeName));
				TVertexAttributesRef<float> AttributeRef = MeshDescription->VertexAttributes().GetAttributesRef<float>(NewAttributeName);
				OnSetNewVertexAttributeValues.ExecuteIfBound(AttributeRef);
			}
		}
	
		return FReply::Handled();
	}

}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
