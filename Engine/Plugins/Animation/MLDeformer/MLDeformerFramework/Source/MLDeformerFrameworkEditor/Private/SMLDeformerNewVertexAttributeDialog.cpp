// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMLDeformerNewVertexAttributeDialog.h"
#include "SkeletalMeshAttributes.h"
#include "MeshDescription.h"
#include "Engine/SkeletalMesh.h"
#include "Editor/EditorEngine.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "MLDeformerNewVertexAttributeDialog"

namespace UE::MLDeformer
{
	void SMLDeformerNewVertexAttributeDialog::Construct(const FArguments& InArgs, USkeletalMesh* InSkeletalMesh)
	{
		SkeletalMesh = InSkeletalMesh;

		bAutoCreate = InArgs._bAutoCreateAttribute;
		DefaultAttributeValue = InArgs._DefaultAttributeValue;

		TSharedPtr<SEditableTextBox> AttributeNameEditWidget;
		SWindow::Construct
		(
			SWindow::FArguments()
			.Title(LOCTEXT("EnterAttributesNameDialogTitle", "Create New Vertex Attribute"))
			.SizingRule(ESizingRule::Autosized)
			.SupportsMinimize(false)
			.SupportsMaximize(false)
			.IsTopmostWindow(true)
			[
				SNew(SBox)
				.Padding(FMargin(10.0f, 10.0f))
				.HAlign(EHorizontalAlignment::HAlign_Fill)
				.VAlign(EVerticalAlignment::VAlign_Fill)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.Padding(4.0f)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0.0f, 0.0f, 4.0f, 0.0f)
						.VAlign(EVerticalAlignment::VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("CreateNewAttributeNameLabel", "Attribute Name:"))
						]
						+SHorizontalBox::Slot()
						.VAlign(EVerticalAlignment::VAlign_Center)
						[
							SNew(SBox)
							.MinDesiredWidth(200.0f)
							[
								SAssignNew(AttributeNameEditWidget, SEditableTextBox)
								.OnKeyDownHandler(this, &SMLDeformerNewVertexAttributeDialog::OnKeyDown)
								.OnTextChanged_Lambda([this](const FText& InText)
								{
									AttributeName = InText.ToString();
								})
							]
						]
					]
					+SVerticalBox::Slot()
					.Padding(4.0f)
					.AutoHeight()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("AttributeExistsError", "There already is an attribute with that name!"))
						.ColorAndOpacity(FLinearColor::Red)
						.Visibility_Lambda([this]() { return HasVertexAttribute(SkeletalMesh, *AttributeName) ? EVisibility::Visible : EVisibility::Hidden; })
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(4.0f)
					.HAlign(EHorizontalAlignment::HAlign_Right)
					.VAlign(EVerticalAlignment::VAlign_Center)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(FAppStyle::Get().GetMargin("StandardDialog.SlotPadding"))
						[
							SAssignNew(CreateButton, SButton)
							.Text(LOCTEXT("CreateButtonText", "Create"))
							.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))
							.IsEnabled_Lambda([this]()
							{
								return !AttributeName.IsEmpty() && !HasVertexAttribute(SkeletalMesh, *AttributeName);
							})
							.OnClicked(this, &SMLDeformerNewVertexAttributeDialog::OnCreateClicked)
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(FAppStyle::Get().GetMargin("StandardDialog.SlotPadding"))
						[
							SNew(SButton)
							.Text(LOCTEXT("CancelButtonText", "Cancel"))
							.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button"))
							.OnClicked_Lambda([this]()
							{
								ReturnCode = EReturnCode::Canceled;
								RequestDestroyWindow();
								return FReply::Handled();
							})
						]
					]
				]
			]
		);

		SetWidgetToFocusOnActivate(AttributeNameEditWidget);
	}

	FReply SMLDeformerNewVertexAttributeDialog::OnCreateClicked()
	{
		if (bAutoCreate)
		{
			check(SkeletalMesh);
			const bool bCreateResult = CreateVertexAttribute(*SkeletalMesh, AttributeName, DefaultAttributeValue);
			check(bCreateResult);
		}
		ReturnCode = EReturnCode::CreatePressed;
		RequestDestroyWindow();
		return FReply::Handled();
	}

	FReply SMLDeformerNewVertexAttributeDialog::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
	{		
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			ReturnCode = EReturnCode::Canceled;
			RequestDestroyWindow();
			return FReply::Handled();
		}

		if (InKeyEvent.GetKey() == EKeys::Enter && CreateButton.IsValid() && CreateButton->IsEnabled())
		{
			return OnCreateClicked();
		}

		return FReply::Unhandled();
	}

	bool SMLDeformerNewVertexAttributeDialog::HasVertexAttribute(const USkeletalMesh* SkeletalMesh, FName InAttributeName)
	{
		if (SkeletalMesh)
		{
			const int32 LODIndex = 0;
			const FMeshDescription* MeshDescription = SkeletalMesh->GetMeshDescription(LODIndex);
			if (MeshDescription)
			{
				return MeshDescription->VertexAttributes().HasAttribute(InAttributeName);
			}
		}

		return false;
	}

	SMLDeformerNewVertexAttributeDialog::EReturnCode SMLDeformerNewVertexAttributeDialog::ShowModal()
	{
		UEditorEngine* Editor = Cast<UEditorEngine>(GEngine);
		check(Editor);
		Editor->EditorAddModalWindow(SharedThis(this));
		return ReturnCode;
	}

	bool SMLDeformerNewVertexAttributeDialog::CreateVertexAttribute(USkeletalMesh& SkeletalMesh, const FString& AttributeName, float DefaultValue)
	{
		const int32 LODIndex = 0;
		FMeshDescription* MeshDescription = SkeletalMesh.GetMeshDescription(LODIndex);
		if (!MeshDescription)
		{
			return false;
		}

		if (MeshDescription->VertexAttributes().HasAttribute(*AttributeName))
		{
			return false;
		}

		MeshDescription->VertexAttributes().RegisterAttribute<float>(*AttributeName, 1, DefaultValue);
		
		return SkeletalMesh.CommitMeshDescription(LODIndex);
	}

}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
