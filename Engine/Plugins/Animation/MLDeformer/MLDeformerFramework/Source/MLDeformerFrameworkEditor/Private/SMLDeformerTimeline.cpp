// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMLDeformerTimeline.h"
#include "AnimPreviewInstance.h"
#include "Animation/AnimData/AnimDataModel.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimSequenceHelpers.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "EditorWidgetsModule.h"
#include "Fonts/FontMeasure.h"
#include "FrameNumberNumericInterface.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IPersonaPreviewScene.h"
#include "ISequencerWidgetsModule.h"
#include "MLDeformerEditorModel.h"
#include "MLDeformerModel.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneFwd.h"
#include "MovieSceneTimeHelpers.h"
#include "Preferences/PersonaOptions.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "TimeSliderArgs.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SWidget.h"

#define LOCTEXT_NAMESPACE "SMLDeformerTimeline"

namespace UE::MLDeformer
{
	struct FPaintSectionAreaViewArgs
	{
		FPaintSectionAreaViewArgs()
			: bDisplayTickLines(false), bDisplayScrubPosition(false)
		{}

		/** Whether to display tick lines. */
		bool bDisplayTickLines;
		/** Whether to display the scrub position. */
		bool bDisplayScrubPosition;
		/** Optional Paint args for the playback range. */
		TOptional<FPaintPlaybackRangeArgs> PlaybackRangeArgs;
	};

	/**
	 * A time slider controller for the anim timeline.
	 */
	class FMLTimeSliderController 
		: public ITimeSliderController
	{
	public:
		FMLTimeSliderController(const FTimeSliderArgs& InArgs, TWeakPtr<FMLDeformerEditorModel> InWeakModel, TWeakPtr<SMLDeformerTimeline> InWeakTimeline);

		/**
		 * Determines the optimal spacing between tick marks in the slider for a given pixel density.
		 * Increments until a minimum amount of slate units specified by MinTick is reached.
		 * @param InPixelsPerInput The density of pixels between each input.
		 * @param MinTick The minimum slate units per tick allowed.
		 * @param MinTickSpacing The minimum tick spacing in time units allowed.
		 * @return The optimal spacing in time units.
		 */
		float DetermineOptimalSpacing(float InPixelsPerInput, uint32 MinTick, float MinTickSpacing) const;
		void SetModel(TWeakPtr<FMLDeformerEditorModel> InModel);
		virtual int32 OnPaintTimeSlider(bool bMirrorLabels, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
		virtual int32 OnPaintViewArea(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, bool bEnabled, const FPaintViewAreaArgs& Args) const override;
		virtual FReply OnMouseButtonDown(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		virtual FReply OnMouseButtonUp(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		virtual FReply OnMouseMove(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		virtual FReply OnMouseWheel(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		virtual FCursorReply OnCursorQuery(const SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
		virtual void SetViewRange(double NewRangeMin, double NewRangeMax, EViewRangeInterpolation Interpolation) override;
		virtual void SetClampRange(double NewRangeMin, double NewRangeMax) override;
		virtual void SetPlayRange(FFrameNumber RangeStart, int32 RangeDuration) override;
		virtual FFrameRate GetDisplayRate() const override { return TimeSliderArgs.DisplayRate.Get(); }
		virtual FFrameRate GetTickResolution() const override { return TimeSliderArgs.TickResolution.Get(); }
		virtual FAnimatedRange GetViewRange() const override { return TimeSliderArgs.ViewRange.Get(); }
		virtual FAnimatedRange GetClampRange() const override { return TimeSliderArgs.ClampRange.Get(); }
		virtual TRange<FFrameNumber> GetPlayRange() const override { return TimeSliderArgs.PlaybackRange.Get(TRange<FFrameNumber>()); }
		virtual FFrameTime GetScrubPosition() const { return TimeSliderArgs.ScrubPosition.Get(); }
		virtual void SetScrubPosition(FFrameTime InTime, bool bEvaluate) { TimeSliderArgs.OnScrubPositionChanged.ExecuteIfBound(InTime, false, bEvaluate); }

		/**
		 * Clamp the given range to the clamp range.
		 * @param NewRangeMin The new lower bound of the range.
		 * @param NewRangeMax The new upper bound of the range.
		 */
		void ClampViewRange(double& NewRangeMin, double& NewRangeMax);

		/**
		 * Zoom the range by a given delta.
		 * @param InDelta The total amount to zoom by (+ve = zoom out, -ve = zoom in).
		 * @param ZoomBias Bias to apply to lower/upper extents of the range (0 = lower, 0.5 = equal, 1 = upper).
		 */
		bool ZoomByDelta(float InDelta, float ZoomBias = 0.5f);

		/**
		 * Pan the range by a given delta.
		 * @param InDelta The total amount to pan by (+ve = pan forwards in time, -ve = pan backwards in time).
		 */
		void PanByDelta(float InDelta);

		/** Determine frame time from a mouse position. */
		FFrameTime GetFrameTimeFromMouse(const FGeometry& Geometry, FVector2D ScreenSpacePosition) const;

		FTimeSliderArgs& GetTimeSliderArgs() { return TimeSliderArgs; }

	private:
		// Forward declared as class members to prevent name collision with similar types defined in other units.
		struct FDrawTickArgs;
		struct FScrubRangeToScreen;

		/**
		 * Call this method when the user's interaction has changed the scrub position.
		 * @param NewValue Value resulting from the user's interaction.
		 * @param bIsScrubbing True if done via scrubbing, false if just releasing scrubbing.
		 */
		void CommitScrubPosition(FFrameTime NewValue, bool bIsScrubbing);

		/**
		 * Draw time tick marks.
		 * @param OutDrawElements List to add draw elements to.
		 * @param ViewRange The currently visible time range in seconds.
		 * @param RangeToScreen Time range to screen space converter.
		 * @param InArgs Parameters for drawing the tick lines.
		 */
		void DrawTicks(FSlateWindowElementList& OutDrawElements, const TRange<double>& ViewRange, const FScrubRangeToScreen& RangeToScreen, FDrawTickArgs& InArgs) const;

		/**
		 * Draw the selection range.
		 * @return The new layer ID.
		 */
		int32 DrawSelectionRange(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FScrubRangeToScreen& RangeToScreen, const FPaintPlaybackRangeArgs& Args) const;

		/**
		 * Draw the playback range.
		 * @return the new layer ID.
		 */
		int32 DrawPlaybackRange(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FScrubRangeToScreen& RangeToScreen, const FPaintPlaybackRangeArgs& Args) const;

	private:
		/**
		 * Hit test the lower bound of a range.
		 */
		bool HitTestRangeStart(const FScrubRangeToScreen& RangeToScreen, const TRange<double>& Range, float HitPixel) const;

		/**
		 * Hit test the upper bound of a range.
		 */
		bool HitTestRangeEnd(const FScrubRangeToScreen& RangeToScreen, const TRange<double>& Range, float HitPixel) const;

		void SetPlaybackRangeStart(FFrameNumber NewStart);
		void SetPlaybackRangeEnd(FFrameNumber NewEnd);

		void SetSelectionRangeStart(FFrameNumber NewStart);
		void SetSelectionRangeEnd(FFrameNumber NewEnd);

		FFrameTime ComputeFrameTimeFromMouse(const FGeometry& Geometry, FVector2D ScreenSpacePosition, FScrubRangeToScreen RangeToScreen, bool CheckSnapping = true) const;

	private:
		struct FScrubPixelRange
		{
			TRange<float> Range;
			TRange<float> HandleRange;
			bool bClamped;
		};

		FScrubPixelRange GetHitTestScrubberPixelRange(FFrameTime ScrubTime, const FScrubRangeToScreen& RangeToScreen) const;
		FScrubPixelRange GetScrubberPixelRange(FFrameTime ScrubTime, const FScrubRangeToScreen& RangeToScreen) const;
		FScrubPixelRange GetScrubberPixelRange(FFrameTime ScrubTime, FFrameRate Resolution, FFrameRate PlayRate, const FScrubRangeToScreen& RangeToScreen, float DilationPixels = 0.f) const;

	private:
		/** Pointer back to the model object. */
		TWeakPtr<FMLDeformerEditorModel> WeakModel;

		/** Pointer back to the timeline. */
		TWeakPtr<SMLDeformerTimeline> WeakTimeline;

		FTimeSliderArgs TimeSliderArgs;

		/** Brush for drawing the fill area on the scrubber. */
		const FSlateBrush* ScrubFillBrush;

		/** Brush for drawing an upwards facing scrub handles. */
		const FSlateBrush* ScrubHandleUpBrush;

		/** Brush for drawing a downwards facing scrub handle. */
		const FSlateBrush* ScrubHandleDownBrush;

		/** Brush for drawing an editable time. */
		const FSlateBrush* EditableTimeBrush;

		/** Total mouse delta during dragging. **/
		float DistanceDragged;

		/** If we are dragging a scrubber or dragging to set the time range. */
		enum DragType
		{
			DRAG_SCRUBBING_TIME,
			DRAG_PLAYBACK_START,
			DRAG_PLAYBACK_END,
			DRAG_SELECTION_START,
			DRAG_SELECTION_END,
			DRAG_TIME,
			DRAG_NONE
		};

		DragType MouseDragType;

		/** If we are currently panning the panel. */
		bool bPanning;

		/** Index of the current dragged time. */
		int32 DraggedTimeIndex;

		/** Mouse down position range. */
		FVector2D MouseDownPosition[2];

		/** Geometry on mouse down. */
		FGeometry MouseDownGeometry;

		/** Range stack. */
		TArray<TRange<double>> ViewRangeStack;
	};

	/** Utility struct for converting between scrub range space and local/absolute screen space. */
	struct FMLTimeSliderController::FScrubRangeToScreen
	{
		double ViewStart;
		float PixelsPerInput;

		FScrubRangeToScreen(const TRange<double>& InViewInput, const FVector2D& InWidgetSize)
		{
			const float ViewInputRange = InViewInput.Size<double>();
			ViewStart = InViewInput.GetLowerBoundValue();
			PixelsPerInput = ViewInputRange > 0 ? (InWidgetSize.X / ViewInputRange) : 0;
		}

		/** Local Widget Space -> Curve Input domain. */
		double LocalXToInput(float ScreenX) const
		{
			return PixelsPerInput > 0 ? (ScreenX / PixelsPerInput) + ViewStart : ViewStart;
		}

		/** Curve Input domain -> local Widget Space. */
		float InputToLocalX(double Input) const
		{
			return (Input - ViewStart) * PixelsPerInput;
		}
	};


	/**
	 * Gets the the next spacing value in the series to determine a good spacing value.
	 * E.g, .001, .005, .010, .050, .100, .500, 1.000, etc.
	 */
	static float GetNextSpacing(uint32 CurrentStep)
	{
		if (CurrentStep & 0x01)
		{
			// Odd numbers
			return FMath::Pow(10.0f, 0.5f * ((float)(CurrentStep - 1)) + 1.0f);
		}
		else
		{
			// Even numbers
			return 0.5f * FMath::Pow(10.0f, 0.5f * ((float)(CurrentStep)) + 1.0f);
		}
	}

	FMLTimeSliderController::FMLTimeSliderController(const FTimeSliderArgs& InArgs, TWeakPtr<FMLDeformerEditorModel> InWeakModel, TWeakPtr<SMLDeformerTimeline> InWeakTimeline)
		: WeakModel(InWeakModel)
		, WeakTimeline(InWeakTimeline)
		, TimeSliderArgs(InArgs)
		, DistanceDragged(0.0f)
		, MouseDragType(DRAG_NONE)
		, bPanning(false)
		, DraggedTimeIndex(INDEX_NONE)
	{
		ScrubFillBrush = FAppStyle::GetBrush(TEXT("Sequencer.Timeline.ScrubFill"));
		ScrubHandleUpBrush = FAppStyle::GetBrush(TEXT("Sequencer.Timeline.VanillaScrubHandleUp"));
		ScrubHandleDownBrush = FAppStyle::GetBrush(TEXT("Sequencer.Timeline.VanillaScrubHandleDown"));
		EditableTimeBrush = FAppStyle::GetBrush(TEXT("AnimTimeline.SectionMarker"));
	}

	void FMLTimeSliderController::SetModel(TWeakPtr<FMLDeformerEditorModel> InModel)
	{
		WeakModel = InModel;
		if (WeakTimeline.IsValid())
		{
			WeakTimeline.Pin()->UpdateTimeSliderArgs(TimeSliderArgs);
		}
	}

	FFrameTime FMLTimeSliderController::ComputeFrameTimeFromMouse(const FGeometry& Geometry, FVector2D ScreenSpacePosition, FScrubRangeToScreen RangeToScreen, bool CheckSnapping) const
	{
		const FVector2D CursorPos = Geometry.AbsoluteToLocal(ScreenSpacePosition);
		const double MouseValue = RangeToScreen.LocalXToInput(CursorPos.X);
		return MouseValue * GetTickResolution();
	}

	FMLTimeSliderController::FScrubPixelRange FMLTimeSliderController::GetHitTestScrubberPixelRange(FFrameTime ScrubTime, const FScrubRangeToScreen& RangeToScreen) const
	{
		static const float DragToleranceSlateUnits = 2.0f, MouseTolerance = 2.0f;
		return GetScrubberPixelRange(ScrubTime, GetTickResolution(), GetDisplayRate(), RangeToScreen, DragToleranceSlateUnits + MouseTolerance);
	}

	FMLTimeSliderController::FScrubPixelRange FMLTimeSliderController::GetScrubberPixelRange(FFrameTime ScrubTime, const FScrubRangeToScreen& RangeToScreen) const
	{
		return GetScrubberPixelRange(ScrubTime, GetTickResolution(), GetDisplayRate(), RangeToScreen);
	}

	FMLTimeSliderController::FScrubPixelRange FMLTimeSliderController::GetScrubberPixelRange(FFrameTime ScrubTime, FFrameRate Resolution, FFrameRate PlayRate, const FScrubRangeToScreen& RangeToScreen, float DilationPixels) const
	{
		const FFrameNumber Frame = ScrubTime.FloorToFrame();
		float StartPixel = RangeToScreen.InputToLocalX(Frame / Resolution);
		float EndPixel = RangeToScreen.InputToLocalX((Frame + 1) / Resolution);

		const float RoundedStartPixel = FMath::RoundToInt(StartPixel);
		EndPixel -= (StartPixel - RoundedStartPixel);

		StartPixel = RoundedStartPixel;
		EndPixel = FMath::Max(EndPixel, StartPixel + 1);

		const float MinScrubSize = 14.0f;
		FScrubPixelRange Range;
		Range.bClamped = EndPixel - StartPixel < MinScrubSize;
		Range.Range = TRange<float>(StartPixel, EndPixel);
		if (Range.bClamped)
		{
			Range.HandleRange = TRange<float>(
				(StartPixel + EndPixel - MinScrubSize) * 0.5f,
				(StartPixel + EndPixel + MinScrubSize) * 0.5f);
		}
		else
		{
			Range.HandleRange = Range.Range;
		}

		return Range;
	}

	struct FMLTimeSliderController::FDrawTickArgs
	{
		/** Geometry of the area. */
		FGeometry AllottedGeometry;
		/** Culling rect of the area. */
		FSlateRect CullingRect;
		/** Color of each tick. */
		FLinearColor TickColor;
		/** Offset in Y where to start the tick. */
		float TickOffset;
		/** Height in of major ticks. */
		float MajorTickHeight;
		/** Start layer for elements. */
		int32 StartLayer;
		/** Draw effects to apply. */
		ESlateDrawEffect DrawEffects;
		/** Whether or not to only draw major ticks. */
		bool bOnlyDrawMajorTicks;
		/** Whether or not to mirror labels. */
		bool bMirrorLabels;
	};

	void FMLTimeSliderController::DrawTicks(FSlateWindowElementList& OutDrawElements, const TRange<double>& ViewRange, const FScrubRangeToScreen& RangeToScreen, FDrawTickArgs& InArgs) const
	{
		TSharedPtr<SMLDeformerTimeline> Timeline = WeakTimeline.Pin();
		if (!Timeline.IsValid())
		{
			return;
		}

		if (!FMath::IsFinite(ViewRange.GetLowerBoundValue()) || !FMath::IsFinite(ViewRange.GetUpperBoundValue()))
		{
			return;
		}

		const FFrameRate     FrameResolution = GetTickResolution();
		const FPaintGeometry PaintGeometry = InArgs.AllottedGeometry.ToPaintGeometry();
		const FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 8);

		double MajorGridStep = 0.0;
		int32 MinorDivisions = 0;
		if (!Timeline->GetGridMetrics(InArgs.AllottedGeometry.Size.X, MajorGridStep, MinorDivisions))
		{
			return;
		}

		if (InArgs.bOnlyDrawMajorTicks)
		{
			MinorDivisions = 0;
		}

		TArray<FVector2D> LinePoints;
		LinePoints.SetNumUninitialized(2);

		const bool bAntiAliasLines = false;
		const double FirstMajorLine = FMath::FloorToDouble(ViewRange.GetLowerBoundValue() / MajorGridStep) * MajorGridStep;
		const double LastMajorLine = FMath::CeilToDouble(ViewRange.GetUpperBoundValue() / MajorGridStep) * MajorGridStep;

		FString FrameString;
		for (double CurrentMajorLine = FirstMajorLine; CurrentMajorLine < LastMajorLine; CurrentMajorLine += MajorGridStep)
		{
			const float MajorLinePx = RangeToScreen.InputToLocalX(CurrentMajorLine);

			LinePoints[0] = FVector2D(MajorLinePx, InArgs.TickOffset);
			LinePoints[1] = FVector2D(MajorLinePx, InArgs.TickOffset + InArgs.MajorTickHeight);

			// Draw each tick mark.
			FSlateDrawElement::MakeLines(
				OutDrawElements,
				InArgs.StartLayer,
				PaintGeometry,
				LinePoints,
				InArgs.DrawEffects,
				InArgs.TickColor,
				bAntiAliasLines
			);

			if (!InArgs.bOnlyDrawMajorTicks)
			{
				FrameString = TimeSliderArgs.NumericTypeInterface->ToString((CurrentMajorLine * FrameResolution).RoundToFrame().Value);

				// Space the text between the tick mark but slightly above.
				const FVector2D TextOffset(MajorLinePx + 5.0f, InArgs.bMirrorLabels ? 3.0f : FMath::Abs(InArgs.AllottedGeometry.Size.Y - (InArgs.MajorTickHeight + 3.0f)));
				FSlateDrawElement::MakeText(
					OutDrawElements,
					InArgs.StartLayer + 1,
					InArgs.AllottedGeometry.ToPaintGeometry(InArgs.AllottedGeometry.Size, FSlateLayoutTransform(TextOffset)),
					FrameString,
					SmallLayoutFont,
					InArgs.DrawEffects,
					InArgs.TickColor * 0.65f
				);
			}

			for (int32 Step = 1; Step < MinorDivisions; ++Step)
			{
				// Compute the size of each tick mark.  If we are half way between to visible values display a slightly larger tick mark.
				const float MinorTickHeight = ((MinorDivisions % 2 == 0) && (Step % (MinorDivisions / 2)) == 0) ? 6.0f : 2.0f;
				const float MinorLinePx = RangeToScreen.InputToLocalX(CurrentMajorLine + Step * MajorGridStep / MinorDivisions);

				LinePoints[0] = FVector2D(MinorLinePx, InArgs.bMirrorLabels ? 0.0f : FMath::Abs(InArgs.AllottedGeometry.Size.Y - MinorTickHeight));
				LinePoints[1] = FVector2D(MinorLinePx, LinePoints[0].Y + MinorTickHeight);

				// Draw each sub mark.
				FSlateDrawElement::MakeLines(
					OutDrawElements,
					InArgs.StartLayer,
					PaintGeometry,
					LinePoints,
					InArgs.DrawEffects,
					InArgs.TickColor,
					bAntiAliasLines);
			}
		}
	}

	int32 FMLTimeSliderController::OnPaintTimeSlider(bool bMirrorLabels, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
	{
		const bool bEnabled = bParentEnabled;
		const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
		const TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
		const TRange<FFrameNumber> LocalPlaybackRange = TimeSliderArgs.PlaybackRange.Get();
		const float LocalViewRangeMin = LocalViewRange.GetLowerBoundValue();
		const float LocalViewRangeMax = LocalViewRange.GetUpperBoundValue();
		const float LocalSequenceLength = LocalViewRangeMax - LocalViewRangeMin;

		if (LocalSequenceLength > 0)
		{
			FScrubRangeToScreen RangeToScreen(LocalViewRange, AllottedGeometry.Size);

			// Draw the ticks.
			const float MajorTickHeight = 9.0f;
			FDrawTickArgs Args;
			{
				Args.AllottedGeometry = AllottedGeometry;
				Args.bMirrorLabels = bMirrorLabels;
				Args.bOnlyDrawMajorTicks = false;
				Args.TickColor = FLinearColor::White;
				Args.CullingRect = MyCullingRect;
				Args.DrawEffects = DrawEffects;
				Args.StartLayer = LayerId;
				Args.TickOffset = bMirrorLabels ? 0.0f : FMath::Abs(AllottedGeometry.Size.Y - MajorTickHeight);
				Args.MajorTickHeight = MajorTickHeight;
			}
			DrawTicks(OutDrawElements, LocalViewRange, RangeToScreen, Args);

			// Draw playback & selection range.
			FPaintPlaybackRangeArgs PlaybackRangeArgs(
				bMirrorLabels ? FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Bottom_L") : FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Top_L"),
				bMirrorLabels ? FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Bottom_R") : FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_Top_R"),
				6.0f);

			LayerId = DrawPlaybackRange(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, RangeToScreen, PlaybackRangeArgs);

			PlaybackRangeArgs.SolidFillOpacity = 0.05f;
			LayerId = DrawSelectionRange(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, RangeToScreen, PlaybackRangeArgs);

			// Draw the scrub handle.
			const float HandleStart = RangeToScreen.InputToLocalX(TimeSliderArgs.ScrubPosition.Get().AsDecimal() / GetTickResolution().AsDecimal()) - 7.0f;
			const float HandleEnd = HandleStart + 13.0f;

			const int32 ArrowLayer = LayerId + 2;
			FPaintGeometry MyGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(HandleEnd - HandleStart, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(HandleStart, 0)));
			FLinearColor ScrubColor = InWidgetStyle.GetColorAndOpacityTint();
			{
				ScrubColor.A = ScrubColor.A * 0.75f;
				ScrubColor.B *= 0.1f;
				ScrubColor.G *= 0.2f;
			}

			const FSlateBrush* Brush = (bMirrorLabels ? ScrubHandleUpBrush : ScrubHandleDownBrush);

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				ArrowLayer,
				MyGeometry,
				Brush,
				DrawEffects,
				ScrubColor);

			{
				// Draw the current time next to the scrub handle.
				FString FrameString = TimeSliderArgs.NumericTypeInterface->ToString(TimeSliderArgs.ScrubPosition.Get().GetFrame().Value);

				if (GetDefault<UPersonaOptions>()->bTimelineDisplayPercentage)
				{
					const double Percentage = FMath::Clamp(TimeSliderArgs.ScrubPosition.Get().AsDecimal() / FFrameTime(LocalPlaybackRange.Size<FFrameNumber>()).AsDecimal(), 0.0, 1.0);
					FNumberFormattingOptions Options;
					Options.MaximumFractionalDigits = 2;
					FrameString += TEXT(" (") + FText::AsPercent(Percentage, &Options).ToString() + TEXT(")");
				}

				const FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 10);
				const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
				const FVector2D TextSize = FontMeasureService->Measure(FrameString, SmallLayoutFont);

				// Flip the text position if getting near the end of the view range.
				static const float TextOffsetPx = 2.0f;
				const bool bDrawLeft = (AllottedGeometry.Size.X - HandleEnd) < (TextSize.X + 14.0f) - TextOffsetPx;
				const float TextPosition = bDrawLeft ? HandleStart - TextSize.X - TextOffsetPx : HandleEnd + TextOffsetPx;
				const FVector2D TextOffset(TextPosition, Args.bMirrorLabels ? TextSize.Y - 6.f : Args.AllottedGeometry.Size.Y - (Args.MajorTickHeight + TextSize.Y));

				FSlateDrawElement::MakeText(
					OutDrawElements,
					Args.StartLayer + 1,
					Args.AllottedGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(TextOffset)),
					FrameString,
					SmallLayoutFont,
					Args.DrawEffects,
					Args.TickColor);
			}

			return ArrowLayer;
		}

		return LayerId;
	}

	int32 FMLTimeSliderController::DrawSelectionRange(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FScrubRangeToScreen& RangeToScreen, const FPaintPlaybackRangeArgs& Args) const
	{
		TRange<double> SelectionRange = TimeSliderArgs.SelectionRange.Get() / GetTickResolution();

		if (!SelectionRange.IsEmpty() && SelectionRange.HasLowerBound() && SelectionRange.HasUpperBound())
		{
			const float SelectionRangeL = RangeToScreen.InputToLocalX(SelectionRange.GetLowerBoundValue()) - 1;
			const float SelectionRangeR = RangeToScreen.InputToLocalX(SelectionRange.GetUpperBoundValue()) + 1;
			const auto DrawColor = FAppStyle::GetSlateColor("SelectionColor").GetColor(FWidgetStyle());

			if (Args.SolidFillOpacity > 0.f)
			{
				FSlateDrawElement::MakeBox(
					OutDrawElements,
					LayerId + 1,
					AllottedGeometry.ToPaintGeometry(FVector2f(SelectionRangeR - SelectionRangeL, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(SelectionRangeL, 0.f))),
					FAppStyle::GetBrush("WhiteBrush"),
					ESlateDrawEffect::None,
					DrawColor.CopyWithNewOpacity(Args.SolidFillOpacity));
			}

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId + 1,
				AllottedGeometry.ToPaintGeometry(FVector2f(Args.BrushWidth, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(SelectionRangeL, 0.f))),
				Args.StartBrush,
				ESlateDrawEffect::None,
				DrawColor);

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId + 1,
				AllottedGeometry.ToPaintGeometry(FVector2f(Args.BrushWidth, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(SelectionRangeR - Args.BrushWidth, 0.f))),
				Args.EndBrush,
				ESlateDrawEffect::None,
				DrawColor);
		}

		return LayerId + 1;
	}

	int32 FMLTimeSliderController::DrawPlaybackRange(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FScrubRangeToScreen& RangeToScreen, const FPaintPlaybackRangeArgs& Args) const
	{
		if (!TimeSliderArgs.PlaybackRange.IsSet())
		{
			return LayerId;
		}

		const TRange<FFrameNumber> PlaybackRange = TimeSliderArgs.PlaybackRange.Get();
		const FFrameRate TickResolution = GetTickResolution();
		const float PlaybackRangeL = RangeToScreen.InputToLocalX(PlaybackRange.GetLowerBoundValue() / TickResolution);
		const float PlaybackRangeR = RangeToScreen.InputToLocalX(PlaybackRange.GetUpperBoundValue() / TickResolution) - 1;

		const uint8 OpacityBlend = TimeSliderArgs.SubSequenceRange.Get().IsSet() ? 128 : 255;
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + 1,
			AllottedGeometry.ToPaintGeometry(FVector2f(Args.BrushWidth, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(PlaybackRangeL, 0.0f))),
			Args.StartBrush,
			ESlateDrawEffect::None,
			FColor(32, 128, 32, OpacityBlend)	// 120, 75, 50 (HSV)
		);

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + 1,
			AllottedGeometry.ToPaintGeometry(FVector2f(Args.BrushWidth, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(PlaybackRangeR - Args.BrushWidth, 0.0f))),
			Args.EndBrush,
			ESlateDrawEffect::None,
			FColor(128, 32, 32, OpacityBlend)	// 0, 75, 50 (HSV)
		);

		// Black tint for excluded regions
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + 1,
			AllottedGeometry.ToPaintGeometry(FVector2f(PlaybackRangeL, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(0.f, 0.f))),
			FAppStyle::GetBrush("WhiteBrush"),
			ESlateDrawEffect::None,
			FLinearColor::Black.CopyWithNewOpacity(0.3f * OpacityBlend / 255.0f)
		);

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + 1,
			AllottedGeometry.ToPaintGeometry(FVector2f(AllottedGeometry.Size.X - PlaybackRangeR, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(PlaybackRangeR, 0.0f))),
			FAppStyle::GetBrush("WhiteBrush"),
			ESlateDrawEffect::None,
			FLinearColor::Black.CopyWithNewOpacity(0.3f * OpacityBlend / 255.0f)
		);

		return LayerId + 1;
	}

	FReply FMLTimeSliderController::OnMouseButtonDown(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		DistanceDragged = 0;
		MouseDownPosition[0] = MouseDownPosition[1] = MouseEvent.GetScreenSpacePosition();
		MouseDownGeometry = MyGeometry;
		return FReply::Unhandled();
	}

	FReply FMLTimeSliderController::OnMouseButtonUp(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		const bool bHandleLeftMouseButton = (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && WidgetOwner.HasMouseCapture();
		const bool bHandleRightMouseButton = (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton) && WidgetOwner.HasMouseCapture() && TimeSliderArgs.AllowZoom;

		const FScrubRangeToScreen RangeToScreen = FScrubRangeToScreen(TimeSliderArgs.ViewRange.Get(), MyGeometry.Size);
		const FFrameTime MouseTime = ComputeFrameTimeFromMouse(MyGeometry, MouseEvent.GetScreenSpacePosition(), RangeToScreen);

		if (bHandleRightMouseButton)
		{
			if (!bPanning && DistanceDragged == 0.0f)
			{
				return WeakTimeline.Pin()->OnMouseButtonUp(MyGeometry, MouseEvent).ReleaseMouseCapture();
			}

			bPanning = false;
			DistanceDragged = 0.0f;

			return FReply::Handled().ReleaseMouseCapture();
		}
		else if (bHandleLeftMouseButton)
		{
			if (MouseDragType == DRAG_PLAYBACK_START)
			{
				TimeSliderArgs.OnPlaybackRangeEndDrag.ExecuteIfBound();
			}
			else if (MouseDragType == DRAG_PLAYBACK_END)
			{
				TimeSliderArgs.OnPlaybackRangeEndDrag.ExecuteIfBound();
			}
			else if (MouseDragType == DRAG_SELECTION_START)
			{
				TimeSliderArgs.OnSelectionRangeEndDrag.ExecuteIfBound();
			}
			else if (MouseDragType == DRAG_SELECTION_END)
			{
				TimeSliderArgs.OnSelectionRangeEndDrag.ExecuteIfBound();
			}
			else
			{
				TimeSliderArgs.OnEndScrubberMovement.ExecuteIfBound();

				CommitScrubPosition(MouseTime, /*bIsScrubbing=*/false);
			}

			MouseDragType = DRAG_NONE;
			DistanceDragged = 0.0f;
			return FReply::Handled().ReleaseMouseCapture();
		}

		return FReply::Unhandled();
	}

	FReply FMLTimeSliderController::OnMouseMove(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		const bool bHandleLeftMouseButton = MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton);
		const bool bHandleRightMouseButton = MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton) && TimeSliderArgs.AllowZoom;

		if (bHandleRightMouseButton)
		{
			if (!bPanning)
			{
				DistanceDragged += FMath::Abs(MouseEvent.GetCursorDelta().X);
				if (DistanceDragged > FSlateApplication::Get().GetDragTriggerDistance())
				{
					bPanning = true;
				}
			}
			else
			{
				const TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
				const double LocalViewRangeMin = LocalViewRange.GetLowerBoundValue();
				const double LocalViewRangeMax = LocalViewRange.GetUpperBoundValue();

				const FScrubRangeToScreen ScaleInfo(LocalViewRange, MyGeometry.Size);
				const FVector2D ScreenDelta = MouseEvent.GetCursorDelta();
				const double InputDeltaX = ScreenDelta.X / ScaleInfo.PixelsPerInput;
				double NewViewOutputMin = LocalViewRangeMin - InputDeltaX;
				double NewViewOutputMax = LocalViewRangeMax - InputDeltaX;

				ClampViewRange(NewViewOutputMin, NewViewOutputMax);
				SetViewRange(NewViewOutputMin, NewViewOutputMax, EViewRangeInterpolation::Immediate);
			}
		}
		else if (bHandleLeftMouseButton)
		{
			const TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
			const FScrubRangeToScreen RangeToScreen(LocalViewRange, MyGeometry.Size);
			DistanceDragged += FMath::Abs(MouseEvent.GetCursorDelta().X);

			if (MouseDragType == DRAG_NONE)
			{
				if (DistanceDragged > FSlateApplication::Get().GetDragTriggerDistance())
				{
					const FFrameTime MouseDownFree = ComputeFrameTimeFromMouse(MyGeometry, MouseDownPosition[0], RangeToScreen, false);

					const FFrameRate FrameResolution = GetTickResolution();
					const bool       bLockedPlayRange = TimeSliderArgs.IsPlaybackRangeLocked.Get();
					const float      MouseDownPixel = RangeToScreen.InputToLocalX(MouseDownFree / FrameResolution);
					const bool       bHitScrubber = GetHitTestScrubberPixelRange(TimeSliderArgs.ScrubPosition.Get(), RangeToScreen).HandleRange.Contains(MouseDownPixel);

					TRange<double>   SelectionRange = TimeSliderArgs.SelectionRange.Get() / FrameResolution;
					TRange<double>   PlaybackRange = TimeSliderArgs.PlaybackRange.Get() / FrameResolution;

					// Disable selection range test if it's empty so that the playback range scrubbing gets priority.
					if (!SelectionRange.IsEmpty() && !bHitScrubber && HitTestRangeEnd(RangeToScreen, SelectionRange, MouseDownPixel))
					{
						// selection range end scrubber.
						MouseDragType = DRAG_SELECTION_END;
						TimeSliderArgs.OnSelectionRangeBeginDrag.ExecuteIfBound();
					}
					else if (!SelectionRange.IsEmpty() && !bHitScrubber && HitTestRangeStart(RangeToScreen, SelectionRange, MouseDownPixel))
					{
						// selection range start scrubber.
						MouseDragType = DRAG_SELECTION_START;
						TimeSliderArgs.OnSelectionRangeBeginDrag.ExecuteIfBound();
					}
					else if (!bLockedPlayRange && !bHitScrubber && HitTestRangeEnd(RangeToScreen, PlaybackRange, MouseDownPixel))
					{
						// playback range end scrubber.
						MouseDragType = DRAG_PLAYBACK_END;
						TimeSliderArgs.OnPlaybackRangeBeginDrag.ExecuteIfBound();
					}
					else if (!bLockedPlayRange && !bHitScrubber && HitTestRangeStart(RangeToScreen, PlaybackRange, MouseDownPixel))
					{
						// playback range start scrubber.
						MouseDragType = DRAG_PLAYBACK_START;
						TimeSliderArgs.OnPlaybackRangeBeginDrag.ExecuteIfBound();
					}
					else
					{
						MouseDragType = DRAG_SCRUBBING_TIME;
						TimeSliderArgs.OnBeginScrubberMovement.ExecuteIfBound();
					}
				}
			}
			else
			{
				const FFrameTime MouseTime = ComputeFrameTimeFromMouse(MyGeometry, MouseEvent.GetScreenSpacePosition(), RangeToScreen);

				// Set the start range time?
				if (MouseDragType == DRAG_PLAYBACK_START)
				{
					SetPlaybackRangeStart(MouseTime.FrameNumber);
				}
				// Set the end range time?
				else if (MouseDragType == DRAG_PLAYBACK_END)
				{
					SetPlaybackRangeEnd(MouseTime.FrameNumber - 1);
				}
				else if (MouseDragType == DRAG_SELECTION_START)
				{
					SetSelectionRangeStart(MouseTime.FrameNumber);
				}
				// Set the end range time?
				else if (MouseDragType == DRAG_SELECTION_END)
				{
					SetSelectionRangeEnd(MouseTime.FrameNumber);
				}
				else if (MouseDragType == DRAG_SCRUBBING_TIME)
				{
					// Delegate responsibility for clamping to the current viewrange to the client.
					CommitScrubPosition(MouseTime, /*bIsScrubbing=*/true);
				}
			}
		}

		if (DistanceDragged != 0.f && (bHandleLeftMouseButton || bHandleRightMouseButton))
		{
			return FReply::Handled().CaptureMouse(WidgetOwner.AsShared());
		}

		return FReply::Handled();
	}

	void FMLTimeSliderController::CommitScrubPosition(FFrameTime NewValue, bool bIsScrubbing)
	{
		// Manage the scrub position ourselves if its not bound to a delegate.
		if (!TimeSliderArgs.ScrubPosition.IsBound())
		{
			TimeSliderArgs.ScrubPosition.Set(NewValue);
		}

		// TODO: Change if anim timeline needs to handle sequencer style middle mouse manipulation which changes time but doesn't evaluate.
		TimeSliderArgs.OnScrubPositionChanged.ExecuteIfBound(NewValue, bIsScrubbing, /*bEvaluate*/ true);
	}

	FReply FMLTimeSliderController::OnMouseWheel(SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (TimeSliderArgs.AllowZoom && MouseEvent.IsControlDown())
		{
			const float MouseFractionX = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()).X / MyGeometry.GetLocalSize().X;
			const float ZoomDelta = -0.2f * MouseEvent.GetWheelDelta();
			if (ZoomByDelta(ZoomDelta, MouseFractionX))
			{
				return FReply::Handled();
			}
		}
		else if (MouseEvent.IsShiftDown())
		{
			PanByDelta(-MouseEvent.GetWheelDelta());
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	FCursorReply FMLTimeSliderController::OnCursorQuery(const SWidget& WidgetOwner, const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
	{
		const FScrubRangeToScreen RangeToScreen(TimeSliderArgs.ViewRange.Get(), MyGeometry.Size);
		const FFrameRate FrameResolution = GetTickResolution();
		const TRange<double> SelectionRange = TimeSliderArgs.SelectionRange.Get() / FrameResolution;
		const TRange<double> PlaybackRange = TimeSliderArgs.PlaybackRange.Get() / FrameResolution;
		const float HitTestPixel = MyGeometry.AbsoluteToLocal(CursorEvent.GetScreenSpacePosition()).X;
		const bool bLockedPlayRange = TimeSliderArgs.IsPlaybackRangeLocked.Get();
		const bool bHitScrubber = GetHitTestScrubberPixelRange(TimeSliderArgs.ScrubPosition.Get(), RangeToScreen).HandleRange.Contains(HitTestPixel);

		if (MouseDragType == DRAG_SCRUBBING_TIME)
		{
			return FCursorReply::Unhandled();
		}

		// Use L/R resize cursor if we're dragging or hovering a playback range bound.
		if ((MouseDragType == DRAG_PLAYBACK_END) ||
			(MouseDragType == DRAG_PLAYBACK_START) ||
			(MouseDragType == DRAG_SELECTION_START) ||
			(MouseDragType == DRAG_SELECTION_END) ||
			(MouseDragType == DRAG_TIME) ||
			(!bLockedPlayRange && !bHitScrubber && HitTestRangeStart(RangeToScreen, PlaybackRange, HitTestPixel)) ||
			(!bLockedPlayRange && !bHitScrubber && HitTestRangeEnd(RangeToScreen, PlaybackRange, HitTestPixel)) ||
			(!SelectionRange.IsEmpty() && !bHitScrubber && HitTestRangeStart(RangeToScreen, SelectionRange, HitTestPixel)) ||
			(!SelectionRange.IsEmpty() && !bHitScrubber && HitTestRangeEnd(RangeToScreen, SelectionRange, HitTestPixel)))
		{
			return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
		}

		return FCursorReply::Unhandled();
	}

	int32 FMLTimeSliderController::OnPaintViewArea(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, bool bEnabled, const FPaintViewAreaArgs& Args) const
	{
		const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

		TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get();
		FScrubRangeToScreen RangeToScreen(LocalViewRange, AllottedGeometry.Size);

		if (Args.PlaybackRangeArgs.IsSet())
		{
			FPaintPlaybackRangeArgs PaintArgs = Args.PlaybackRangeArgs.GetValue();
			LayerId = DrawPlaybackRange(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, RangeToScreen, PaintArgs);
			PaintArgs.SolidFillOpacity = 0.2f;
			LayerId = DrawSelectionRange(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, RangeToScreen, PaintArgs);
		}

		if (Args.bDisplayTickLines)
		{
			static FLinearColor TickColor(0.1f, 0.1f, 0.1f, 0.3f);

			// Draw major tick lines in the section area.
			FDrawTickArgs DrawTickArgs;
			{
				DrawTickArgs.AllottedGeometry = AllottedGeometry;
				DrawTickArgs.bMirrorLabels = false;
				DrawTickArgs.bOnlyDrawMajorTicks = true;
				DrawTickArgs.TickColor = TickColor;
				DrawTickArgs.CullingRect = MyCullingRect;
				DrawTickArgs.DrawEffects = DrawEffects;
				// Draw major ticks under sections.
				DrawTickArgs.StartLayer = LayerId - 1;
				// Draw the tick the entire height of the section area.
				DrawTickArgs.TickOffset = 0.0f;
				DrawTickArgs.MajorTickHeight = AllottedGeometry.Size.Y;
			}

			DrawTicks(OutDrawElements, LocalViewRange, RangeToScreen, DrawTickArgs);
		}

		if (Args.bDisplayScrubPosition)
		{
			// Draw a line for the scrub position.
			const float LinePos = RangeToScreen.InputToLocalX(TimeSliderArgs.ScrubPosition.Get().AsDecimal() / GetTickResolution().AsDecimal());

			TArray<FVector2D> LinePoints;
			{
				LinePoints.AddUninitialized(2);
				LinePoints[0] = FVector2D(0.0f, 0.0f);
				LinePoints[1] = FVector2D(0.0f, FMath::FloorToFloat(AllottedGeometry.Size.Y));
			}

			FSlateDrawElement::MakeLines(
				OutDrawElements,
				LayerId + 1,
				AllottedGeometry.ToPaintGeometry(FVector2f(1.0f, 1.0f), FSlateLayoutTransform(FVector2f(LinePos, 0.0f))),
				LinePoints,
				DrawEffects,
				FLinearColor(1.0f, 1.0f, 1.0f, 0.5f),
				false
			);
		}

		return LayerId;
	}

	FFrameTime FMLTimeSliderController::GetFrameTimeFromMouse(const FGeometry& Geometry, FVector2D ScreenSpacePosition) const
	{
		const FScrubRangeToScreen ScrubRangeToScreen(TimeSliderArgs.ViewRange.Get(), Geometry.Size);
		return ComputeFrameTimeFromMouse(Geometry, ScreenSpacePosition, ScrubRangeToScreen);
	}

	void FMLTimeSliderController::ClampViewRange(double& NewRangeMin, double& NewRangeMax)
	{
		bool bNeedsClampSet = false;
		double NewClampRangeMin = TimeSliderArgs.ClampRange.Get().GetLowerBoundValue();
		if (NewRangeMin < TimeSliderArgs.ClampRange.Get().GetLowerBoundValue())
		{
			NewClampRangeMin = NewRangeMin;
			bNeedsClampSet = true;
		}

		double NewClampRangeMax = TimeSliderArgs.ClampRange.Get().GetUpperBoundValue();
		if (NewRangeMax > TimeSliderArgs.ClampRange.Get().GetUpperBoundValue())
		{
			NewClampRangeMax = NewRangeMax;
			bNeedsClampSet = true;
		}

		if (bNeedsClampSet)
		{
			SetClampRange(NewClampRangeMin, NewClampRangeMax);
		}
	}

	void FMLTimeSliderController::SetViewRange(double NewRangeMin, double NewRangeMax, EViewRangeInterpolation Interpolation)
	{
		// Clamp to a minimum size to avoid zero-sized or negative visible ranges.
		double MinVisibleTimeRange = FFrameNumber(1) / GetTickResolution();
		TRange<double> ExistingViewRange = TimeSliderArgs.ViewRange.Get();

		if (NewRangeMax == ExistingViewRange.GetUpperBoundValue())
		{
			if (NewRangeMin > NewRangeMax - MinVisibleTimeRange)
			{
				NewRangeMin = NewRangeMax - MinVisibleTimeRange;
			}
		}
		else if (NewRangeMax < NewRangeMin + MinVisibleTimeRange)
		{
			NewRangeMax = NewRangeMin + MinVisibleTimeRange;
		}

		// Clamp to the clamp range.
		const TRange<double> NewRange = TRange<double>(NewRangeMin, NewRangeMax);
		TimeSliderArgs.OnViewRangeChanged.ExecuteIfBound(NewRange, Interpolation);

		if (!TimeSliderArgs.ViewRange.IsBound())
		{
			// The  output is not bound to a delegate so we'll manage the value ourselves (no animation).
			TimeSliderArgs.ViewRange.Set(NewRange);
		}
	}

	void FMLTimeSliderController::SetClampRange(double NewRangeMin, double NewRangeMax)
	{
		const TRange<double> NewRange(NewRangeMin, NewRangeMax);

		TimeSliderArgs.OnClampRangeChanged.ExecuteIfBound(NewRange);

		if (!TimeSliderArgs.ClampRange.IsBound())
		{
			// The output is not bound to a delegate so we'll manage the value ourselves (no animation).
			TimeSliderArgs.ClampRange.Set(NewRange);
		}
	}

	void FMLTimeSliderController::SetPlayRange(FFrameNumber RangeStart, int32 RangeDuration)
	{
		check(RangeDuration >= 0);
		const TRange<FFrameNumber> NewRange(RangeStart, RangeStart + RangeDuration);
		TimeSliderArgs.OnPlaybackRangeChanged.ExecuteIfBound(NewRange);
		if (!TimeSliderArgs.PlaybackRange.IsBound())
		{
			// The output is not bound to a delegate so we'll manage the value ourselves (no animation).
			TimeSliderArgs.PlaybackRange.Set(NewRange);
		}
	}

	bool FMLTimeSliderController::ZoomByDelta(float InDelta, float MousePositionFraction)
	{
		const TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get().GetAnimationTarget();
		const double LocalViewRangeMax = LocalViewRange.GetUpperBoundValue();
		const double LocalViewRangeMin = LocalViewRange.GetLowerBoundValue();
		const double OutputViewSize = LocalViewRangeMax - LocalViewRangeMin;
		const double OutputChange = OutputViewSize * InDelta;

		double NewViewOutputMin = LocalViewRangeMin - (OutputChange * MousePositionFraction);
		double NewViewOutputMax = LocalViewRangeMax + (OutputChange * (1.0f - MousePositionFraction));
		if (NewViewOutputMin < NewViewOutputMax)
		{
			ClampViewRange(NewViewOutputMin, NewViewOutputMax);
			SetViewRange(NewViewOutputMin, NewViewOutputMax, EViewRangeInterpolation::Animated);
			return true;
		}

		return false;
	}

	void FMLTimeSliderController::PanByDelta(float InDelta)
	{
		// The fraction of the current view range to scroll per unit delta.
		const float ScrollPanFraction = 0.1f;
		const TRange<double> LocalViewRange = TimeSliderArgs.ViewRange.Get().GetAnimationTarget();
		const double CurrentMin = LocalViewRange.GetLowerBoundValue();
		const double CurrentMax = LocalViewRange.GetUpperBoundValue();

		// Adjust the delta to be a percentage of the current range.
		InDelta *= ScrollPanFraction * (CurrentMax - CurrentMin);

		double NewViewOutputMin = CurrentMin + InDelta;
		double NewViewOutputMax = CurrentMax + InDelta;

		ClampViewRange(NewViewOutputMin, NewViewOutputMax);
		SetViewRange(NewViewOutputMin, NewViewOutputMax, EViewRangeInterpolation::Animated);
	}

	bool FMLTimeSliderController::HitTestRangeStart(const FScrubRangeToScreen& RangeToScreen, const TRange<double>& Range, float HitPixel) const
	{
		if (Range.HasLowerBound())
		{
			static const float BrushSizeInStateUnits = 6.0f;
			static const float DragToleranceSlateUnits = 2.0f;
			static const float MouseTolerance = 2.0f;
			const float RangeStartPixel = RangeToScreen.InputToLocalX(Range.GetLowerBoundValue());

			// Hit test against the brush region to the right of the playback start position, +/- DragToleranceSlateUnits.
			return (HitPixel >= RangeStartPixel - MouseTolerance - DragToleranceSlateUnits) &&
				(HitPixel <= RangeStartPixel + MouseTolerance + BrushSizeInStateUnits + DragToleranceSlateUnits);
		}

		return false;
	}

	bool FMLTimeSliderController::HitTestRangeEnd(const FScrubRangeToScreen& RangeToScreen, const TRange<double>& Range, float HitPixel) const
	{
		if (Range.HasUpperBound())
		{
			static const float BrushSizeInStateUnits = 6.0f;
			static const float DragToleranceSlateUnits = 2.0f;
			static const float MouseTolerance = 2.0f;
			const float RangeEndPixel = RangeToScreen.InputToLocalX(Range.GetUpperBoundValue());

			// Hit test against the brush region to the left of the playback end position, +/- DragToleranceSlateUnits.
			return (HitPixel >= RangeEndPixel - MouseTolerance - BrushSizeInStateUnits - DragToleranceSlateUnits) &&
				(HitPixel <= RangeEndPixel + MouseTolerance + DragToleranceSlateUnits);
		}

		return false;
	}

	void FMLTimeSliderController::SetPlaybackRangeStart(FFrameNumber NewStart)
	{
		const TRange<FFrameNumber> PlaybackRange = TimeSliderArgs.PlaybackRange.Get();
		if (NewStart <= UE::MovieScene::DiscreteExclusiveUpper(PlaybackRange))
		{
			TimeSliderArgs.OnPlaybackRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(NewStart, PlaybackRange.GetUpperBound()));
		}
	}

	void FMLTimeSliderController::SetPlaybackRangeEnd(FFrameNumber NewEnd)
	{
		const TRange<FFrameNumber> PlaybackRange = TimeSliderArgs.PlaybackRange.Get();
		if (NewEnd >= UE::MovieScene::DiscreteInclusiveLower(PlaybackRange))
		{
			TimeSliderArgs.OnPlaybackRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(PlaybackRange.GetLowerBound(), NewEnd));
		}
	}

	void FMLTimeSliderController::SetSelectionRangeStart(FFrameNumber NewStart)
	{
		const TRange<FFrameNumber> SelectionRange = TimeSliderArgs.SelectionRange.Get();
		if (SelectionRange.IsEmpty())
		{
			TimeSliderArgs.OnSelectionRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(NewStart, NewStart + 1));
		}
		else if (NewStart <= UE::MovieScene::DiscreteExclusiveUpper(SelectionRange))
		{
			TimeSliderArgs.OnSelectionRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(NewStart, SelectionRange.GetUpperBound()));
		}
	}

	void FMLTimeSliderController::SetSelectionRangeEnd(FFrameNumber NewEnd)
	{
		const TRange<FFrameNumber> SelectionRange = TimeSliderArgs.SelectionRange.Get();
		if (SelectionRange.IsEmpty())
		{
			TimeSliderArgs.OnSelectionRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(NewEnd - 1, NewEnd));
		}
		else if (NewEnd >= UE::MovieScene::DiscreteInclusiveLower(SelectionRange))
		{
			TimeSliderArgs.OnSelectionRangeChanged.ExecuteIfBound(TRange<FFrameNumber>(SelectionRange.GetLowerBound(), NewEnd));
		}
	}

	class SAnimTimelineTransportControls : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SAnimTimelineTransportControls) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TWeakPtr<FMLDeformerEditorModel>& InModel);
		void SetModel(TWeakPtr<FMLDeformerEditorModel> InModel)
		{
			Model = InModel;
		}

	private:
		FReply OnClick_Forward_Step();
		FReply OnClick_Forward_End();
		FReply OnClick_Backward_Step();
		FReply OnClick_Backward_End();
		FReply OnClick_Forward();
		FReply OnClick_ToggleLoop();

		bool IsLoopStatusOn() const;
		EPlaybackMode::Type GetPlaybackMode() const;
		UAnimSingleNodeInstance* GetPreviewInstance() const;

	private:
		/** Anim timeline model. */
		TWeakPtr<FMLDeformerEditorModel> Model;
	};

	void SAnimTimelineTransportControls::Construct(const FArguments& InArgs, TWeakPtr<FMLDeformerEditorModel>& InModel)
	{
		// Geometry Cache only supports forward playback, so backwards play is removed.
		Model = InModel;
		FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");

		FTransportControlArgs TransportControlArgs;
		TransportControlArgs.OnForwardPlay = FOnClicked::CreateSP(this, &SAnimTimelineTransportControls::OnClick_Forward);
		TransportControlArgs.OnForwardStep = FOnClicked::CreateSP(this, &SAnimTimelineTransportControls::OnClick_Forward_Step);
		TransportControlArgs.OnBackwardStep = FOnClicked::CreateSP(this, &SAnimTimelineTransportControls::OnClick_Backward_Step);
		TransportControlArgs.OnForwardEnd = FOnClicked::CreateSP(this, &SAnimTimelineTransportControls::OnClick_Forward_End);
		TransportControlArgs.OnBackwardEnd = FOnClicked::CreateSP(this, &SAnimTimelineTransportControls::OnClick_Backward_End);
		TransportControlArgs.OnGetPlaybackMode = FOnGetPlaybackMode::CreateSP(this, &SAnimTimelineTransportControls::GetPlaybackMode);

		ChildSlot
		[
			EditorWidgetsModule.CreateTransportControl(TransportControlArgs)
		];
	}

	FReply SAnimTimelineTransportControls::OnClick_Forward_Step()
	{
		if (Model.IsValid())
		{
			FFrameNumber ScrubPosition = Model.Pin()->GetTickResScrubPosition();
			ScrubPosition.Value += Model.Pin()->GetTicksPerFrame();
			Model.Pin()->SetScrubPosition(ScrubPosition, false);
			return FReply::Handled();
		}
		return FReply::Unhandled();
	}

	FReply SAnimTimelineTransportControls::OnClick_Forward_End()
	{
		if (Model.IsValid())
		{
			const TRange<FFrameNumber> PlaybackRange = Model.Pin()->GetPlaybackRange();
			FFrameNumber UpperBound = PlaybackRange.GetUpperBoundValue();
			Model.Pin()->SetScrubPosition(UpperBound, false);
			return FReply::Handled();
		}
		return FReply::Unhandled();
	}

	FReply SAnimTimelineTransportControls::OnClick_Backward_Step()
	{
		if (Model.IsValid())
		{
			FFrameNumber ScrubPosition = Model.Pin()->GetTickResScrubPosition();
			ScrubPosition.Value -= Model.Pin()->GetTicksPerFrame();
			Model.Pin()->SetScrubPosition(ScrubPosition, false);
			return FReply::Handled();
		}
		return FReply::Unhandled();
	}

	FReply SAnimTimelineTransportControls::OnClick_Backward_End()
	{
		if (Model.IsValid())
		{
			const TRange<FFrameNumber> PlaybackRange = Model.Pin()->GetPlaybackRange();
			FFrameNumber LowerBound = PlaybackRange.GetLowerBoundValue();
			Model.Pin()->SetScrubPosition(LowerBound, false);
			return FReply::Handled();
		}
		return FReply::Unhandled();
	}

	FReply SAnimTimelineTransportControls::OnClick_Forward()
	{
		if (Model.IsValid())
		{
			Model.Pin()->OnPlayPressed();
			return FReply::Handled();
		}
		return FReply::Unhandled();
	}


	EPlaybackMode::Type SAnimTimelineTransportControls::GetPlaybackMode() const
	{
		if (Model.IsValid())
		{
			if (Model.Pin()->IsPlayingAnim())
			{
				return EPlaybackMode::PlayingForward;
			}
		}
		return EPlaybackMode::Stopped;
	}

	/**
	 * An overlay that displays global information in the track area.
	 */
	class SAnimTimelineOverlay : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SAnimTimelineOverlay)
			: _DisplayTickLines(true)
			, _DisplayScrubPosition(false)
		{}

		SLATE_ATTRIBUTE(bool, DisplayTickLines)
			SLATE_ATTRIBUTE(bool, DisplayScrubPosition)
			SLATE_ATTRIBUTE(FPaintPlaybackRangeArgs, PaintPlaybackRangeArgs)
			SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TSharedRef<FMLTimeSliderController> InTimeSliderController)
		{
			bDisplayScrubPosition = InArgs._DisplayScrubPosition;
			bDisplayTickLines = InArgs._DisplayTickLines;
			PaintPlaybackRangeArgs = InArgs._PaintPlaybackRangeArgs;
			TimeSliderController = InTimeSliderController;
		}

	private:
		/** SWidget Interface. */
		virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	private:
		/** Controller for manipulating time. */
		TSharedPtr<FMLTimeSliderController> TimeSliderController;
		/** Whether or not to display the scrub position. */
		TAttribute<bool> bDisplayScrubPosition;
		/** Whether or not to display tick lines. */
		TAttribute<bool> bDisplayTickLines;
		/** User-supplied options for drawing playback range. */
		TAttribute<FPaintPlaybackRangeArgs> PaintPlaybackRangeArgs;
	};

	int32 SAnimTimelineOverlay::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
	{
		FPaintViewAreaArgs PaintArgs;
		PaintArgs.bDisplayTickLines = bDisplayTickLines.Get();
		PaintArgs.bDisplayScrubPosition = bDisplayScrubPosition.Get();

		if (PaintPlaybackRangeArgs.IsSet())
		{
			PaintArgs.PlaybackRangeArgs = PaintPlaybackRangeArgs.Get();
		}

		TimeSliderController->OnPaintViewArea(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, ShouldBeEnabled(bParentEnabled), PaintArgs);
		return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	}


	void SMLDeformerTimeline::UpdateTimeSliderArgs(FTimeSliderArgs& TimeSliderArgs)
	{
		TimeSliderArgs.ScrubPosition = MakeAttributeLambda([this](){ return Model.IsValid() ? Model.Pin()->GetTickResScrubPosition() : FFrameTime(0); });
		TimeSliderArgs.PlaybackRange = MakeAttributeLambda([this](){ return Model.IsValid() ? Model.Pin()->GetPlaybackRange() : TRange<FFrameNumber>(0, 0); });
		TimeSliderArgs.ClampRange = MakeAttributeLambda([this]()
		{
			if (Model.IsValid())
			{
				const TRange<double> Range = Model.Pin()->GetWorkingRange();
				return FAnimatedRange(Range.GetLowerBoundValue(), Range.GetUpperBoundValue());
			}
			else
			{
				return FAnimatedRange(0.0, 0.0);
			}
		});

		if (Model.IsValid())
		{
			TimeSliderArgs.OnClampRangeChanged = FOnTimeRangeChanged::CreateSP(Model.Pin().Get(), &FMLDeformerEditorModel::HandleWorkingRangeChanged);
		}
	}


	void SMLDeformerTimeline::Construct(const FArguments& InArgs, FMLDeformerEditorToolkit* InEditor)
	{
		const TWeakPtr<FMLDeformerEditorModel> WeakModel = InEditor->GetActiveModelPointer();

		Model = WeakModel;
		OnReceivedFocus = InArgs._OnReceivedFocus;

		const int32 TickResolutionValue = Model.IsValid() ? Model.Pin()->GetTickResolution() : 1000 * 30;	// 1000*30 is ticks per frame multiplied by 30 fps.
		const int32 SequenceFrameRate = FMath::RoundToInt(Model.IsValid() ? Model.Pin()->GetFrameRate() : 30);

		ViewRange = MakeAttributeLambda([this]()
		{
			if (Model.IsValid())
			{
				const TRange<double> Range = Model.Pin()->GetViewRange();
				return FAnimatedRange(Range.GetLowerBoundValue(), Range.GetUpperBoundValue());
			}
			else
			{
				return FAnimatedRange(0.0, 0.0);
			}
		});

		TAttribute<EFrameNumberDisplayFormats> DisplayFormat = MakeAttributeLambda([this]()
		{
			if (Model.IsValid())
			{
				return Model.Pin()->IsDisplayingFrames() ? EFrameNumberDisplayFormats::Frames : EFrameNumberDisplayFormats::Seconds;
			}
			else
			{
				return EFrameNumberDisplayFormats::Frames;
			}
		});

		TAttribute<FFrameRate> TickResolution = MakeAttributeLambda([TickResolutionValue]()
		{
			return FFrameRate(TickResolutionValue, 1);
		});

		TAttribute<FFrameRate> DisplayRate = MakeAttributeLambda([SequenceFrameRate]()
		{
			return FFrameRate(SequenceFrameRate, 1);
		});

		// Create our numeric type interface so we can pass it to the time slider below.
		NumericTypeInterface = MakeShareable(new FFrameNumberInterface(DisplayFormat, 0, TickResolution, DisplayRate));

		FTimeSliderArgs TimeSliderArgs;
		{
			TimeSliderArgs.ScrubPosition = MakeAttributeLambda([WeakModel](){ return WeakModel.IsValid() ? WeakModel.Pin()->GetTickResScrubPosition() : FFrameTime(0); });
			TimeSliderArgs.ViewRange = ViewRange;
			TimeSliderArgs.PlaybackRange = MakeAttributeLambda([WeakModel](){ return WeakModel.IsValid() ? WeakModel.Pin()->GetPlaybackRange() : TRange<FFrameNumber>(0, 0); });
			TimeSliderArgs.ClampRange = MakeAttributeLambda([this]()
			{
				if (Model.IsValid())
				{
					const TRange<double> Range = Model.Pin()->GetWorkingRange();
					return FAnimatedRange(Range.GetLowerBoundValue(), Range.GetUpperBoundValue());
				}
				else
				{
					return FAnimatedRange(0.0, 0.0);
				}
			});
			TimeSliderArgs.DisplayRate = DisplayRate;
			TimeSliderArgs.TickResolution = TickResolution;
			TimeSliderArgs.OnViewRangeChanged = FOnViewRangeChanged::CreateSP(this, &SMLDeformerTimeline::HandleViewRangeChanged);

			if (Model.IsValid())
			{
				TimeSliderArgs.OnClampRangeChanged = FOnTimeRangeChanged::CreateSP(Model.Pin().Get(), &FMLDeformerEditorModel::HandleWorkingRangeChanged);
			}

			TimeSliderArgs.IsPlaybackRangeLocked = true;
			TimeSliderArgs.PlaybackStatus = EMovieScenePlayerStatus::Stopped;
			TimeSliderArgs.NumericTypeInterface = NumericTypeInterface;
			TimeSliderArgs.OnScrubPositionChanged = FOnScrubPositionChanged::CreateSP(this, &SMLDeformerTimeline::HandleScrubPositionChanged);
		}

		TimeSliderController = MakeShareable(new FMLTimeSliderController(TimeSliderArgs, Model, SharedThis(this)));
	
		TSharedRef<FMLTimeSliderController> TimeSliderControllerRef = TimeSliderController.ToSharedRef();

		// Create the top slider.
		const bool bMirrorLabels = false;
		ISequencerWidgetsModule& SequencerWidgets = FModuleManager::Get().LoadModuleChecked<ISequencerWidgetsModule>("SequencerWidgets");
		TopTimeSlider = SequencerWidgets.CreateTimeSlider(TimeSliderControllerRef, bMirrorLabels);

		// Create bottom time range slider.
		TSharedRef<ITimeSlider> BottomTimeRange = SequencerWidgets.CreateTimeRange(
			FTimeRangeArgs(
				EShowRange::ViewRange | EShowRange::WorkingRange | EShowRange::PlaybackRange,
				EShowRange::ViewRange | EShowRange::WorkingRange,
				TimeSliderControllerRef,
				EVisibility::Visible,
				NumericTypeInterface.ToSharedRef()
			),
			SequencerWidgets.CreateTimeRangeSlider(TimeSliderControllerRef)
		);

		TSharedRef<SScrollBar> ScrollBar = SNew(SScrollBar)
			.Thickness(FVector2D(5.0f, 5.0f));

		ColumnFillCoefficients[0] = 0.2f;
		ColumnFillCoefficients[1] = 0.8f;

		TransportControls = SNew(SAnimTimelineTransportControls, Model);

		TAttribute<float> FillCoefficient_0, FillCoefficient_1;
		{
			FillCoefficient_0.Bind(TAttribute<float>::FGetter::CreateSP(this, &SMLDeformerTimeline::GetColumnFillCoefficient, 0));
			FillCoefficient_1.Bind(TAttribute<float>::FGetter::CreateSP(this, &SMLDeformerTimeline::GetColumnFillCoefficient, 1));
		}

		const float CommonPadding = 3.0f;
		const FMargin ResizeBarPadding(4.0f, 0, 0, 0);

		ChildSlot
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				[
					SNew(SOverlay)
					+SOverlay::Slot()
					[
						SNew(SGridPanel)
						.FillRow(1, 1.0f)
						.FillColumn(0, FillCoefficient_0)
						.FillColumn(1, FillCoefficient_1)

						// Outliner search box.
						+SGridPanel::Slot(0, 0, SGridPanel::Layer(10))
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.FillWidth(0.9)
							.Padding(2.0f)
							[
								SNew(SComboBox<TSharedPtr<FMLDeformerTrainingInputAnimName>>)
								.OptionsSource(&TrainingAnimNames)
								.OnGenerateWidget(this, &SMLDeformerTimeline::OnGenerateAnimListComboWidget)
								.OnSelectionChanged(this, &SMLDeformerTimeline::OnSelectTrainingAnim)
								.Visibility_Lambda
								(
									[this]()
									{
										if (!Model.IsValid())
										{
											return EVisibility::Hidden;
										}

										TSharedPtr<FMLDeformerEditorModel> EditorModel = Model.Pin();
										check(EditorModel.IsValid());										
										const EMLDeformerVizMode VizMode = EditorModel->GetModel()->GetVizSettings()->GetVisualizationMode();
										return (VizMode == EMLDeformerVizMode::TrainingData) ? EVisibility::Visible : EVisibility::Hidden;
									}
								)
								[
									SNew(STextBlock)
									.Text_Lambda
									(
										[this]()
										{
											if (SelectedTrainingAnimNameIndex != INDEX_NONE && TrainingAnimNames.IsValidIndex(SelectedTrainingAnimNameIndex))
											{
												return FText::FromString(TrainingAnimNames[SelectedTrainingAnimNameIndex]->Name);
											}

											return FText::FromString("Select Animation");
										}
									)
								]
							]
							+SHorizontalBox::Slot()
							.FillWidth(0.1f)
							.VAlign(VAlign_Center)
							+SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Center)
							.AutoWidth()
							.Padding(2.0f, 0.0f, 2.0f, 0.0f)
							[
								SNew(SBox)
								.MinDesiredWidth(30.0f)
								.VAlign(VAlign_Center)
								.HAlign(HAlign_Center)
								[
									// Current play time.
									SNew(SSpinBox<double>)
									.Style(&FAppStyle::GetWidgetStyle<FSpinBoxStyle>("Sequencer.PlayTimeSpinBox"))
									.Value_Lambda([this]() -> double
									{
										if (Model.IsValid())
										{
											return Model.Pin()->GetTickResScrubPosition().Value;
										}
										return 0.0;
									})
									.OnValueChanged(this, &SMLDeformerTimeline::SetPlayTime)
									.OnValueCommitted_Lambda([this](double InFrame, ETextCommit::Type)
									{
										SetPlayTime(InFrame);
									})
									.MinValue(TOptional<double>())
									.MaxValue(TOptional<double>())
									.TypeInterface(NumericTypeInterface)
									.Delta(this, &SMLDeformerTimeline::GetSpinboxDelta)
									.LinearDeltaSensitivity(25)
								]
							]
						]
						// Transport controls.
						+SGridPanel::Slot(0, 3, SGridPanel::Layer(10))
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						[
							TransportControls.ToSharedRef()
						]

						// Second column.
						+SGridPanel::Slot(1, 0)
						.Padding(ResizeBarPadding)
						.RowSpan(2)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							[
								SNew(SSpacer)
							]
						]

						+SGridPanel::Slot(1, 0, SGridPanel::Layer(10))
						.Padding(ResizeBarPadding)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							.BorderBackgroundColor(FLinearColor(0.5f, 0.5f, 0.5f, 1.0f))
							.Padding(0)
							.Clipping(EWidgetClipping::ClipToBounds)
							[
								TopTimeSlider.ToSharedRef()
							]
						]

						// Overlay that draws the tick lines.
						+SGridPanel::Slot(1, 1, SGridPanel::Layer(10))
						.Padding(ResizeBarPadding)
						[
							SNew(SAnimTimelineOverlay, TimeSliderControllerRef)
							.Visibility( EVisibility::HitTestInvisible )
							.DisplayScrubPosition(false)
							.DisplayTickLines(true)
							.Clipping(EWidgetClipping::ClipToBounds)
							.PaintPlaybackRangeArgs(FPaintPlaybackRangeArgs(FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_L"), FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_R"), 6.0f))
						]

						// Overlay that draws the scrub position.
						+SGridPanel::Slot(1, 1, SGridPanel::Layer(20))
						.Padding(ResizeBarPadding)
						[
							SNew(SAnimTimelineOverlay, TimeSliderControllerRef)
							.Visibility(EVisibility::HitTestInvisible)
							.DisplayScrubPosition(true)
							.DisplayTickLines(false)
							.Clipping(EWidgetClipping::ClipToBounds)
						]

						// Play range slider.
						+SGridPanel::Slot(1, 3, SGridPanel::Layer(10))
						.Padding(ResizeBarPadding)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							.BorderBackgroundColor(FLinearColor(0.5f, 0.5f, 0.5f, 1.0f))
							.Clipping(EWidgetClipping::ClipToBounds)
							.Padding(0)
							[
								BottomTimeRange
							]
						]
					]
				]
			]
		];
	}

	FReply SMLDeformerTimeline::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if(MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			const FWidgetPath WidgetPath = (MouseEvent.GetEventPath() != nullptr) ? *MouseEvent.GetEventPath() : FWidgetPath();

			const bool bCloseAfterSelection = true;
			FMenuBuilder MenuBuilder(bCloseAfterSelection, NULL);
			const TWeakPtr<FMLDeformerEditorModel> WeakModel = Model;

			MenuBuilder.BeginSection("TimelineOptions", LOCTEXT("TimelineOptions", "Timeline Options") );
			{
				MenuBuilder.AddSubMenu(
					LOCTEXT("TimeFormat", "Time Format"),
					LOCTEXT("TimeFormatTooltip", "Choose the format of times we display in the timeline"),
					FNewMenuDelegate::CreateLambda([WeakModel](FMenuBuilder& InMenuBuilder)
					{
						InMenuBuilder.BeginSection("TimeFormat", LOCTEXT("TimeFormat", "Time Format") );
						{
							InMenuBuilder.AddMenuEntry(
									LOCTEXT("DisplayFrames", "Display Frames"),
									LOCTEXT("DisplayFrames_Tooltip", "Display Timeline in Frames."),
									FSlateIcon(),
									FUIAction(FExecuteAction::CreateLambda([WeakModel]() {
											if (WeakModel.IsValid())
											{
												WeakModel.Pin()->SetDisplayFrames(true);
											}
										}
									),
									FCanExecuteAction(),
									FIsActionChecked::CreateLambda([WeakModel]() {
											if (WeakModel.IsValid()) 
											{ 
												return WeakModel.Pin()->IsDisplayingFrames();
											}
											return false;
										}
									)
								),
								NAME_None,
								EUserInterfaceActionType::RadioButton
							);
							InMenuBuilder.AddMenuEntry(
									LOCTEXT("DisplaySeconds", "Display Seconds"),
									LOCTEXT("DisplaySeconds_Tooltip", "Display Timeline in Seconds."),
									FSlateIcon(),
									FUIAction(FExecuteAction::CreateLambda([WeakModel]() {
											if (WeakModel.IsValid())
											{
												WeakModel.Pin()->SetDisplayFrames(false);
											}
										}
									),
									FCanExecuteAction(),
									FIsActionChecked::CreateLambda([WeakModel]() {
											if (WeakModel.IsValid())
											{
												return !WeakModel.Pin()->IsDisplayingFrames();
											}
											return false;
										}
									)
								),
								NAME_None,
								EUserInterfaceActionType::RadioButton
							);
						}
						InMenuBuilder.EndSection();
					})
				);
			}

			MenuBuilder.EndSection();

			FSlateApplication::Get().PushMenu(SharedThis(this), WidgetPath, MenuBuilder.MakeWidget(), FSlateApplication::Get().GetCursorPos(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	TSharedRef<INumericTypeInterface<double>> SMLDeformerTimeline::GetNumericTypeInterface() const
	{
		return NumericTypeInterface.ToSharedRef();
	}

	// FFrameRate::ComputeGridSpacing doesnt deal well with prime numbers, so we have a custom impl here
	static bool ComputeGridSpacing(const FFrameRate& InFrameRate, float PixelsPerSecond, double& OutMajorInterval, int32& OutMinorDivisions, float MinTickPx, float DesiredMajorTickPx)
	{
		// First try built-in spacing.
		const bool bResult = InFrameRate.ComputeGridSpacing(PixelsPerSecond, OutMajorInterval, OutMinorDivisions, MinTickPx, DesiredMajorTickPx);
		if (!bResult || OutMajorInterval == 1.0)
		{
			if (PixelsPerSecond <= 0.0f)
			{
				return false;
			}

			const int32 RoundedFPS = FMath::RoundToInt(InFrameRate.AsDecimal());

			if (RoundedFPS > 0)
			{
				// Showing frames.
				TArray<int32, TInlineAllocator<10>> CommonBases;

				// Divide the rounded frame rate by 2s, 3s or 5s recursively.
				{
					const int32 Denominators[] = { 2, 3, 5 };

					int32 LowestBase = RoundedFPS;
					for (;;)
					{
						CommonBases.Add(LowestBase);
	
						if (LowestBase % 2 == 0)      { LowestBase = LowestBase / 2; }
						else if (LowestBase % 3 == 0) { LowestBase = LowestBase / 3; }
						else if (LowestBase % 5 == 0) { LowestBase = LowestBase / 5; }
						else
						{ 
							int32 LowestResult = LowestBase;
							for (int32 Denominator : Denominators)
							{
								const int32 Result = LowestBase / Denominator;
								if(Result > 0 && Result < LowestResult)
								{
									LowestResult = Result;
								}
							}

							if (LowestResult < LowestBase)
							{
								LowestBase = LowestResult;
							}
							else
							{
								break;
							}
						}
					}
				}

				Algo::Reverse(CommonBases);

				const int32 Scale = FMath::CeilToInt(DesiredMajorTickPx / PixelsPerSecond * InFrameRate.AsDecimal());
				const int32 BaseIndex = FMath::Min(Algo::LowerBound(CommonBases, Scale), CommonBases.Num() - 1);
				const int32 Base = CommonBases[BaseIndex];

				const int32 MajorIntervalFrames = FMath::CeilToInt(Scale / float(Base)) * Base;
				OutMajorInterval = MajorIntervalFrames * InFrameRate.AsInterval();

				// Find the lowest number of divisions we can show that's larger than the minimum tick size.
				OutMinorDivisions = 0;
				for (int32 DivIndex = 0; DivIndex < BaseIndex; ++DivIndex)
				{
					if (Base % CommonBases[DivIndex] == 0)
					{
						const int32 MinorDivisions = MajorIntervalFrames/CommonBases[DivIndex];
						if (OutMajorInterval / MinorDivisions * PixelsPerSecond >= MinTickPx)
						{
							OutMinorDivisions = MinorDivisions;
							break;
						}
					}
				}
			}
		}

		return OutMajorInterval != 0;
	}

	void SMLDeformerTimeline::SetDisplayFormat(EFrameNumberDisplayFormats InFormat)
	{
		if (Model.IsValid())
		{
			if (InFormat == EFrameNumberDisplayFormats::Frames)
			{
				Model.Pin()->SetDisplayFrames(true);
			}
			else
			{
				Model.Pin()->SetDisplayFrames(false);
			}
		}
	}

	bool SMLDeformerTimeline::IsDisplayFormatChecked(EFrameNumberDisplayFormats InFormat) const
	{
		if (Model.IsValid())
		{
			if (InFormat == EFrameNumberDisplayFormats::Frames)
			{
				return Model.Pin()->IsDisplayingFrames();
			}
			else if (InFormat == EFrameNumberDisplayFormats::Seconds)
			{
				return !Model.Pin()->IsDisplayingFrames();
			}
		}
		return false;
	}

	bool SMLDeformerTimeline::GetGridMetrics(float PhysicalWidth, double& OutMajorInterval, int32& OutMinorDivisions) const
	{
		const FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 8);
		const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		double FrameRate = 30.0; 
		if (Model.IsValid())
		{
			FrameRate = Model.Pin()->GetFrameRate();
		}
		const FFrameRate DisplayRate(FMath::Max(FMath::RoundToInt(FrameRate), 1), 1);
		const double BiggestTime = ViewRange.Get().GetUpperBoundValue();
		const FString TickString = NumericTypeInterface->ToString((BiggestTime * DisplayRate).FrameNumber.Value);
		const FVector2D MaxTextSize = FontMeasureService->Measure(TickString, SmallLayoutFont);
		static const float MajorTickMultiplier = 2.0f;
		const float MinTickPx = MaxTextSize.X + 5.0f;
		const float DesiredMajorTickPx = MaxTextSize.X * MajorTickMultiplier;

		if (PhysicalWidth > 0 && DisplayRate.AsDecimal() > 0)
		{
			return ComputeGridSpacing(
				DisplayRate,
				PhysicalWidth / ViewRange.Get().Size<double>(),
				OutMajorInterval,
				OutMinorDivisions,
				MinTickPx,
				DesiredMajorTickPx);
		}

		return false;
	}

	TSharedPtr<ITimeSliderController> SMLDeformerTimeline::GetTimeSliderController() const 
	{ 
		return TimeSliderController; 
	}

	void SMLDeformerTimeline::SetModel(TWeakPtr<FMLDeformerEditorModel> InModel)
	{
		Model = InModel; 
		TimeSliderController->SetModel(Model);
		TransportControls->SetModel(Model);
	}

	void SMLDeformerTimeline::SetTrainingAnimNames(const TArray<TSharedPtr<FMLDeformerTrainingInputAnimName>>& Names)
	{
		const int32 OldSelectedItemIndex = SelectedTrainingAnimNameIndex;

		// Detect when the selected item index needs to be updated as the position in the name list changed.
		TSharedPtr<FMLDeformerTrainingInputAnimName> PrevSelected;
		if (SelectedTrainingAnimNameIndex != INDEX_NONE)
		{
			PrevSelected = TrainingAnimNames[SelectedTrainingAnimNameIndex];
			if (Names.IsValidIndex(SelectedTrainingAnimNameIndex) && PrevSelected != Names[SelectedTrainingAnimNameIndex])
			{
				for (int32 Index = 0; Index < Names.Num(); ++Index)
				{
					if (Names[Index].IsValid() && Names[Index]->Name == PrevSelected->Name)
					{
						SelectedTrainingAnimNameIndex = Index;
						break;
					}
				}
			}
		}

		TrainingAnimNames = Names;

		if (!Names.IsEmpty() && SelectedTrainingAnimNameIndex == INDEX_NONE)
		{
			SelectedTrainingAnimNameIndex = 0;
		}

		if (Names.IsEmpty())
		{
			SelectedTrainingAnimNameIndex = INDEX_NONE;
		}

		if (OldSelectedItemIndex != SelectedTrainingAnimNameIndex && SelectedTrainingAnimNameIndex != INDEX_NONE)
		{
			OnSelectTrainingAnim(TrainingAnimNames[SelectedTrainingAnimNameIndex], ESelectInfo::Type::Direct);
		}
	}

	void SMLDeformerTimeline::OnColumnFillCoefficientChanged(float FillCoefficient, int32 ColumnIndex)
	{
		ColumnFillCoefficients[ColumnIndex] = FillCoefficient;
	}

	void SMLDeformerTimeline::HandleScrubPositionChanged(FFrameTime NewScrubPosition, bool bIsScrubbing, bool bEvaluate)
	{
		if (Model.IsValid())
		{
			Model.Pin()->SetScrubPosition(NewScrubPosition, bIsScrubbing);
		}
	}

	void SMLDeformerTimeline::HandleViewRangeChanged(TRange<double> InRange, EViewRangeInterpolation InInterpolation)
	{
		if (Model.IsValid())
		{
			Model.Pin()->SetViewRange(InRange);
		}
	}

	double SMLDeformerTimeline::GetSpinboxDelta() const
	{
		if (Model.IsValid())
		{
			return FFrameRate(Model.Pin()->GetTickResolution(), 1).AsDecimal() * FFrameRate(FMath::RoundToInt(Model.Pin()->GetFrameRate()), 1).AsInterval();
		}
		return 0.0;
	}

	void SMLDeformerTimeline::SetPlayTime(double InFrameTime)
	{
		const FFrameTime FrameTime((int)InFrameTime, 0.0f);
		if (Model.IsValid())
		{
			Model.Pin()->SetScrubPosition(FrameTime, false);
		}
	}

	TSharedRef<SWidget> SMLDeformerTimeline::OnGenerateAnimListComboWidget(TSharedPtr<FMLDeformerTrainingInputAnimName> Item) const
	{
		return SNew(STextBlock)
			.Text(FText::FromString(Item->Name));
	}

	void SMLDeformerTimeline::OnSelectTrainingAnim(TSharedPtr<FMLDeformerTrainingInputAnimName> Item, ESelectInfo::Type SelectInfo)
	{
		if (Item.IsValid())
		{
			SelectedTrainingAnimNameIndex = TrainingAnimNames.Find(Item);

			if (Model.IsValid())
			{
				TSharedPtr<FMLDeformerEditorModel> EditorModel = Model.Pin();
				EditorModel->SetActiveTrainingInputAnimIndex(Item->TrainingInputAnimIndex);
				if (EditorModel->GetEditor())
				{
					EditorModel->GetEditor()->UpdateTimeSliderRange();					
				}
				EditorModel->RequestTriggerInputAssetsChangedNextTick();
			}
		}
	}

	void SMLDeformerTimeline::OnDeletedTrainingInputAnim(int32 Index)
	{
		if (TrainingAnimNames.IsEmpty())
		{
			SelectedTrainingAnimNameIndex = INDEX_NONE;
			return;
		}

		const int32 OldSelectedItemIndex = SelectedTrainingAnimNameIndex;
		if (TrainingAnimNames[SelectedTrainingAnimNameIndex]->TrainingInputAnimIndex >= Index)
		{
			// Find a new valid selected item index.
			SelectedTrainingAnimNameIndex--;
			if (SelectedTrainingAnimNameIndex < 0 && !TrainingAnimNames.IsEmpty())
			{
				SelectedTrainingAnimNameIndex = 0;
			}

			if (SelectedTrainingAnimNameIndex < 0)
			{
				SelectedTrainingAnimNameIndex = INDEX_NONE;
			}

			if (SelectedTrainingAnimNameIndex > TrainingAnimNames.Num() - 1)
			{
				SelectedTrainingAnimNameIndex = TrainingAnimNames.Num() - 1;
			}
		}

		if (OldSelectedItemIndex != SelectedTrainingAnimNameIndex)
		{
			OnSelectTrainingAnim(TrainingAnimNames[SelectedTrainingAnimNameIndex], ESelectInfo::Type::Direct);
		}
	}
}	// namespace UE::MLDeformer

#undef LOCTEXT_NAMESPACE
