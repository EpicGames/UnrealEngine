// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimatedRange.h"
#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ITimeSlider.h"
#include "ITransportControl.h"
#include "Framework/Commands/Commands.h"
#include "MLDeformerEditorToolkit.h"
#include "MLDeformerTrainingInputAnim.h"

struct FPaintPlaybackRangeArgs;
class SSearchBox;
class UAnimSingleNodeInstance;
struct FTimeSliderArgs;
enum class EViewRangeInterpolation;
enum class EFrameNumberDisplayFormats : uint8;

namespace UE::MLDeformer
{
	class FMLDeformerEditorToolkit;
	class FMLDeformerEditorModel;
	class FMLTimeSliderController;
	class SAnimTimelineTransportControls; 

	/**
	 * Implements a timeline widget for the ML Deformer editor.
	 */
	class SMLDeformerTimeline : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SMLDeformerTimeline) {}
			/** The current total range of frame indices. */
			SLATE_ATTRIBUTE(FInt32Interval, ViewIndices)
			/** Called when any widget contained within the anim timeline has received focus. */
			SLATE_EVENT(FSimpleDelegate, OnReceivedFocus)
		SLATE_END_ARGS()

	public:
		/**
		 * Construct this widget.
		 * @param InArgs The declaration data for this widget.
		 * @param InModel The model for the anim timeline.
		 */
		void Construct(const FArguments& InArgs, FMLDeformerEditorToolkit* InEditor);

		// SWidget interface.
		virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		// End SWidget interface.

		/** Compute a major grid interval and number of minor divisions to display. */
		bool GetGridMetrics(float PhysicalWidth, double& OutMajorInterval, int32& OutMinorDivisions) const;

		/** Get the time slider controller. */
		TSharedPtr<ITimeSliderController> GetTimeSliderController() const;

		void SetModel(TWeakPtr<FMLDeformerEditorModel> InModel);

		/** Set the list of names to show in the timeline animation list combobox. */
		void SetTrainingAnimNames(const TArray<TSharedPtr<FMLDeformerTrainingInputAnimName>>& Names);

		void OnDeletedTrainingInputAnim(int32 Index);

		void UpdateTimeSliderArgs(FTimeSliderArgs& TimeSliderArgs);

	private:
		/**
		 * @return The fill percentage of the animation outliner.
		 */
		float GetColumnFillCoefficient(int32 ColumnIndex) const { return ColumnFillCoefficients[ColumnIndex]; }

		TSharedRef<SWidget> OnGenerateAnimListComboWidget(TSharedPtr<FMLDeformerTrainingInputAnimName> Item) const;
		void OnSelectTrainingAnim(TSharedPtr<FMLDeformerTrainingInputAnimName> Item, ESelectInfo::Type SelectInfo);

		/** Get numeric Type interface for converting between frame numbers and display formats. */
		TSharedRef<INumericTypeInterface<double>> GetNumericTypeInterface() const;

		/** Called when a column fill percentage is changed by a splitter slot. */
		void OnColumnFillCoefficientChanged(float FillCoefficient, int32 ColumnIndex);

		/** Handles an additive layer key being set. */
		void HandleScrubPositionChanged(FFrameTime NewScrubPosition, bool bIsScrubbing, bool bEvaluate);
		void HandleViewRangeChanged(TRange<double> InRange, EViewRangeInterpolation InInterpolation);

		double GetSpinboxDelta() const;
		void SetPlayTime(double InFrameTime);
		void SetDisplayFormat(EFrameNumberDisplayFormats InFormat);
		bool IsDisplayFormatChecked(EFrameNumberDisplayFormats InFormat) const;

	private:
		/** Anim timeline model. */
		TWeakPtr<FMLDeformerEditorModel> Model;

		/** The currently active editor model. */
		UE::MLDeformer::FMLDeformerEditorToolkit* Editor = nullptr;

		/** The time slider controller. */
		TSharedPtr<FMLTimeSliderController> TimeSliderController;

		/** The anim timeline transport controls, which contains the forward, backward and step actions. */
		TSharedPtr<SAnimTimelineTransportControls> TransportControls;

		/** The top time slider widget. */
		TSharedPtr<ITimeSlider> TopTimeSlider;

		/** The fill coefficients of each column in the grid. */
		float ColumnFillCoefficients[2] = { 0.2f, 0.8f };

		/** Called when the user has begun dragging the selection selection range. */
		FSimpleDelegate OnSelectionRangeBeginDrag;

		/** Called when the user has finished dragging the selection selection range. */
		FSimpleDelegate OnSelectionRangeEndDrag;

		/** Called when any widget contained within the anim timeline has received focus. */
		FSimpleDelegate OnReceivedFocus;

		/** Numeric Type interface for converting between frame numbers and display formats. */
		TSharedPtr<INumericTypeInterface<double>> NumericTypeInterface;

		/** The view range. */
		TAttribute<FAnimatedRange> ViewRange;

		/** Filter text used to search the tree. */
		FText FilterText;

		/** The training animation names as they will appear inside the timeline combobox on the left (in training mode only). */
		TArray<TSharedPtr<FMLDeformerTrainingInputAnimName>> TrainingAnimNames;

		/** The currently selected animation index. Index into the TrainingAnimNames array.*/
		int32 SelectedTrainingAnimNameIndex = INDEX_NONE;
	};

}// namespace UE::MLDeformer
