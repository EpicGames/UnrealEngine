// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/TrainingDataProcessor/AnimCustomization.h"
#include "MLDeformerTrainingDataProcessorSettings.h"
#include "Animation/AnimSequence.h"
#include "PropertyCustomizationHelpers.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SComboBox.h"
#include "SWarningOrErrorBox.h"
#include "DetailWidgetRow.h"
#include "IPropertyUtilities.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetDataTagMap.h"
#include "Animation/Skeleton.h"

#define LOCTEXT_NAMESPACE "MLDeformerTrainingDataProcessorAnimListCustomize"

namespace UE::MLDeformer::TrainingDataProcessor
{
	TSharedRef<IPropertyTypeCustomization> FAnimCustomization::MakeInstance()
	{
		return MakeShareable(new FAnimCustomization());
	}

	void FAnimCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow,
	                                         IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		const int32 AnimIndex = StructPropertyHandle->GetIndexInArray();
		check(AnimIndex != INDEX_NONE);

		PropertyUtilities = StructCustomizationUtils.GetPropertyUtilities();

		const UMLDeformerTrainingDataProcessorSettings& Settings = *FindSettings(StructPropertyHandle);
		const FMLDeformerTrainingDataProcessorAnim& Anim = Settings.AnimList[AnimIndex];

		int32 NumFramesInAnim = 0;
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		const FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(Anim.AnimSequence.ToSoftObjectPath());
		if (AssetData.IsValid())
		{
			const FAssetTagValueRef Tag = AssetData.TagsAndValues.FindTag(TEXT("Number Of Frames"));
			if (Tag.IsSet())
			{
				NumFramesInAnim = FCString::Atoi(*Tag.GetValue());
			}
		}

		HeaderRow
			.NameContent()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.HAlign(EHorizontalAlignment::HAlign_Left)
			[
				SNew(STextBlock)
				.Text(FText::Format(LOCTEXT("HeaderTextFmt", "Animation #{0}"), AnimIndex))
				.Font(StructCustomizationUtils.GetRegularFont())
				.ColorAndOpacity(Anim.bEnabled ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground())
			]
			.ValueContent()
			[
				SNew(SHorizontalBox) // Required to work around a text alignment issue. Otherwise the text block will not center align.
				+ SHorizontalBox::Slot()
				.VAlign(EVerticalAlignment::VAlign_Center)
				.HAlign(EHorizontalAlignment::HAlign_Left)
				[
					SNew(STextBlock)
					.Text(FText::Format(LOCTEXT("HeaderTextValue", "{0} Frames"), NumFramesInAnim))
					.Font(StructCustomizationUtils.GetRegularFont())
					.ColorAndOpacity(Anim.bEnabled ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground())
				]
			];
	}

	void FAnimCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder,
	                                           IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		// Get all the child properties.
		uint32 NumChildren;
		StructPropertyHandle->GetNumChildren(NumChildren);
		TMap<FName, TSharedPtr<IPropertyHandle>> PropertyHandles;
		PropertyHandles.Reserve(NumChildren);
		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
			const FName PropertyName = ChildHandle->GetProperty()->GetFName();
			PropertyHandles.Add(PropertyName, ChildHandle);
		}

		const int32 AnimIndex = StructPropertyHandle->GetIndexInArray();
		check(AnimIndex != INDEX_NONE);

		TSharedPtr<IPropertyHandle> AnimSequenceHandle = PropertyHandles.FindChecked(TEXT("AnimSequence"));
		TSharedPtr<IPropertyHandle> EnabledHandle = PropertyHandles.FindChecked(TEXT("bEnabled"));
		check(AnimSequenceHandle.IsValid());
		check(EnabledHandle.IsValid());

		AnimSequenceHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FAnimCustomization::RefreshProperties));
		EnabledHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FAnimCustomization::RefreshProperties));

		const UMLDeformerTrainingDataProcessorSettings& Settings = *FindSettings(StructPropertyHandle);
		const FMLDeformerTrainingDataProcessorAnim& Anim = Settings.AnimList[AnimIndex];

		// Add the properties
		IDetailPropertyRow& AnimRow = ChildBuilder.AddProperty(AnimSequenceHandle.ToSharedRef());
		AnimRow.EditCondition(Anim.bEnabled, nullptr);
		AnimRow.CustomWidget()
	       .NameContent()
			[
				AnimRow.GetPropertyHandle()->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(SObjectPropertyEntryBox)
				.PropertyHandle(AnimRow.GetPropertyHandle())
				.AllowedClass(UAnimSequence::StaticClass())
				.ObjectPath(Anim.AnimSequence.ToSoftObjectPath().ToString())
				.ThumbnailPool(StructCustomizationUtils.GetThumbnailPool())
				.OnShouldFilterAsset_Lambda([this, &Settings](const FAssetData& AssetData)
				{
					const USkeleton* Skeleton = Settings.FindSkeleton();
					return (!Skeleton || !Skeleton->IsCompatibleForEditor(AssetData));
				})
			];

		ChildBuilder.AddProperty(EnabledHandle.ToSharedRef());

		// Verify the geometry cache against the anim sequence.
		ChildBuilder.AddCustomRow(FText::FromString("SkeletonMismatchWarning"))
		            .Visibility(TAttribute<EVisibility>::CreateSP(this, &FAnimCustomization::GetAnimErrorVisibility, StructPropertyHandle, AnimIndex))
		            .WholeRowContent()
		[
			SNew(SBox)
			.Padding(FMargin(0.0f, 4.0f))
			[
				SNew(SWarningOrErrorBox)
				.MessageStyle(EMessageStyle::Error)
				.Message(LOCTEXT("SkeletonMismatchWarningText",
				                 "The skeleton does not match the one used by the Skeletal Mesh of the ML Deformer model."))
			]
		];
	}

	EVisibility FAnimCustomization::GetAnimErrorVisibility(const TSharedRef<IPropertyHandle> StructPropertyHandle, int32 AnimIndex) const
	{
		const UMLDeformerTrainingDataProcessorSettings& Settings = *FindSettings(StructPropertyHandle);
		const FMLDeformerTrainingDataProcessorAnim& Anim = Settings.AnimList[AnimIndex];

		const USkeleton* Skeleton = Settings.FindSkeleton();
		if (Skeleton && Anim.AnimSequence.LoadSynchronous() && Anim.AnimSequence->GetSkeleton() != Skeleton)
		{
			return EVisibility::Visible;
		}
		return EVisibility::Collapsed;
	}

	void FAnimCustomization::RefreshProperties() const
	{
		if (PropertyUtilities.IsValid())
		{
			PropertyUtilities->ForceRefresh();
		}
	}

	UMLDeformerTrainingDataProcessorSettings* FAnimCustomization::FindSettings(const TSharedRef<IPropertyHandle>& StructPropertyHandle)
	{
		TArray<UObject*> Objects;
		StructPropertyHandle->GetOuterObjects(Objects);
		UObject* OwnerObject = !Objects.IsEmpty() ? Objects[0] : nullptr;
		UMLDeformerTrainingDataProcessorSettings* Settings = Cast<UMLDeformerTrainingDataProcessorSettings>(OwnerObject);
		return Settings;
	}
} // namespace UE::MLDeformer::TrainingDataProcessor

#undef LOCTEXT_NAMESPACE
