// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/TrainingDataProcessor/BoneGroupsListCustomization.h"
#include "Tools/TrainingDataProcessor/SBoneGroupsListWidget.h"
#include "Tools/TrainingDataProcessor/TrainingDataProcessorTool.h"
#include "MLDeformerTrainingDataProcessorSettings.h"
#include "MLDeformerModel.h"
#include "PropertyCustomizationHelpers.h"
#include "DetailWidgetRow.h"
#include "IPropertyUtilities.h"
#include "Animation/Skeleton.h"

#define LOCTEXT_NAMESPACE "MLDeformerTrainingDataProcessorBoneListCustomize"

namespace UE::MLDeformer::TrainingDataProcessor
{
	TSharedRef<IPropertyTypeCustomization> FBoneGroupsListCustomization::MakeInstance()
	{
		return MakeShareable(new FBoneGroupsListCustomization());
	}

	void FBoneGroupsListCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow,
	                                                   IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		StructProperty = StructPropertyHandle;
		PropertyUtilities = StructCustomizationUtils.GetPropertyUtilities();

		// Get the object that this property is inside.
		// We will use that to perform transactions for undo/redo.
		TArray<UObject*> Objects;
		StructProperty->GetOuterObjects(Objects);
		UObject* UndoObject = Objects.IsEmpty() ? nullptr : Objects[0];

		HeaderRow
			.NameContent()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.HAlign(EHorizontalAlignment::HAlign_Left)
			[
				StructPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.HAlign(EHorizontalAlignment::HAlign_Fill)
			[
				SNew(SBoneGroupsListWidget, PropertyUtilities->GetNotifyHook())
				.Skeleton(FindSkeletonForProperty(StructProperty))
				.UndoObject(UndoObject)
				.GetBoneGroups(this, &FBoneGroupsListCustomization::GetBoneGroups)
			];
	}

	TArray<FMLDeformerTrainingDataProcessorBoneGroup>* FBoneGroupsListCustomization::GetBoneGroups() const
	{
		TArray<void*> RawData;
		StructProperty->AccessRawData(RawData);
		if (!RawData.IsEmpty())
		{
			FMLDeformerTrainingDataProcessorBoneGroupsList* BoneGroupList = static_cast<FMLDeformerTrainingDataProcessorBoneGroupsList*>(RawData[0]);
			check(BoneGroupList);
			return &BoneGroupList->Groups;
		}
		return nullptr;
	}
} // namespace UE::MLDeformer::TrainingDataProcessor

#undef LOCTEXT_NAMESPACE
