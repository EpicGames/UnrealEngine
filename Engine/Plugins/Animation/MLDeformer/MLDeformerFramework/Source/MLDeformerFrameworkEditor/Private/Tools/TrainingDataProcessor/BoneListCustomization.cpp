// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/TrainingDataProcessor/BoneListCustomization.h"
#include "Tools/TrainingDataProcessor/SBoneListWidget.h"
#include "Tools/TrainingDataProcessor/TrainingDataProcessorTool.h"
#include "MLDeformerTrainingDataProcessorSettings.h"
#include "MLDeformerModel.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/SComboBox.h"
#include "DetailWidgetRow.h"
#include "IPropertyUtilities.h"
#include "Animation/Skeleton.h"

#define LOCTEXT_NAMESPACE "MLDeformerTrainingDataProcessorBoneListCustomize"

namespace UE::MLDeformer::TrainingDataProcessor
{
	TSharedRef<IPropertyTypeCustomization> FBoneListCustomization::MakeInstance()
	{
		return MakeShareable(new FBoneListCustomization());
	}

	void FBoneListCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow,
	                                             IPropertyTypeCustomizationUtils& StructCustomizationUtils)
	{
		StructProperty = StructPropertyHandle;
		PropertyUtilities = StructCustomizationUtils.GetPropertyUtilities();
	
		// Get the object that this property is inside.
		// We will use that to perform transactions for undo/redo.
		TArray<UObject*> Objects;
		StructProperty->GetOuterObjects(Objects);
		UObject* UndoObject = Objects.IsEmpty() ? nullptr : Objects[0];

		HeaderRow
			.NameContent()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.HAlign(EHorizontalAlignment::HAlign_Left)
			[
				StructPropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.HAlign(EHorizontalAlignment::HAlign_Fill)
			[
				SNew(SBoneListWidget, PropertyUtilities->GetNotifyHook())
				.Skeleton(FindSkeletonForProperty(StructProperty))
				.UndoObject(UndoObject)
				.GetBoneNames(this, &FBoneListCustomization::GetBoneNames)
			];
	}

	TArray<FName>* FBoneListCustomization::GetBoneNames() const
	{
		TArray<void*> RawData;
		StructProperty->AccessRawData(RawData);
		if (!RawData.IsEmpty())
		{
			FMLDeformerTrainingDataProcessorBoneList* BoneNames = static_cast<FMLDeformerTrainingDataProcessorBoneList*>(RawData[0]);
			check(BoneNames);
			return &BoneNames->BoneNames;
		}
		return nullptr;
	}
} // namespace UE::MLDeformer::TrainingDataProcessor

#undef LOCTEXT_NAMESPACE
