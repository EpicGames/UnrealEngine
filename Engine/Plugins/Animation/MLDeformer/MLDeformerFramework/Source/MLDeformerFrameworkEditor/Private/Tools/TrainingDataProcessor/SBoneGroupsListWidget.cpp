// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/TrainingDataProcessor/SBoneGroupsListWidget.h"
#include "MLDeformerTrainingDataProcessorSettings.h"
#include "MLDeformerModule.h"
#include "SMLDeformerBonePickerDialog.h"
#include "MLDeformerEditorStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Engine/SkeletalMesh.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Animation/Skeleton.h"
#include "Framework/Commands/UICommandList.h"
#include "Editor.h"
#include "Misc/NotifyHook.h"

#define LOCTEXT_NAMESPACE "BoneGroupsListWidget"

namespace UE::MLDeformer::TrainingDataProcessor
{
	FBoneGroupsListWidgetCommands::FBoneGroupsListWidgetCommands()
		: TCommands<FBoneGroupsListWidgetCommands>
		("Bone Groups",
		 LOCTEXT("BoneGroupsCommandDesc", "Bone Groups"),
		 NAME_None,
		 FMLDeformerEditorStyle::Get().GetStyleSetName())
	{
	}

	void FBoneGroupsListWidgetCommands::RegisterCommands()
	{
		UI_COMMAND(CreateGroup, "Create New Group", "Create a new bone group.", EUserInterfaceActionType::Button, FInputChord(EKeys::Insert));
		UI_COMMAND(DeleteSelectedItems, "Delete Selected Items", "Deletes the selected bones and/or groups.", EUserInterfaceActionType::Button,
		           FInputChord(EKeys::Delete));
		UI_COMMAND(ClearGroups, "Clear All Groups", "Clears the entire list of bone groups.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(AddBoneToGroup, "Add Bones To Group", "Add new bones to the group.", EUserInterfaceActionType::Button, FInputChord());
	}

	void SBoneGroupsTreeWidget::Construct(const FArguments& InArgs)
	{
		BoneGroupsWidget = InArgs._BoneGroupsWidget;

		STreeView<TSharedPtr<FBoneGroupTreeElement>>::FArguments SuperArgs;
		SuperArgs.TreeItemsSource(&RootElements);
		SuperArgs.SelectionMode(ESelectionMode::Multi);
		SuperArgs.OnGenerateRow(this, &SBoneGroupsTreeWidget::MakeTableRowWidget);
		SuperArgs.OnGetChildren_Static(&SBoneGroupsTreeWidget::HandleGetChildrenForTree);
		SuperArgs.OnContextMenuOpening(this, &SBoneGroupsTreeWidget::CreateContextMenuWidget);
		SuperArgs.HighlightParentNodesForSelection(false);
		SuperArgs.AllowInvisibleItemSelection(true);

		STreeView<TSharedPtr<FBoneGroupTreeElement>>::Construct(SuperArgs);

		RefreshTree();
	}

	TSharedPtr<SWidget> SBoneGroupsTreeWidget::CreateContextMenuWidget() const
	{
		const FBoneGroupsListWidgetCommands& Actions = FBoneGroupsListWidgetCommands::Get();

		const TSharedPtr<FUICommandList> CommandList = BoneGroupsWidget.IsValid()
			                                               ? BoneGroupsWidget.Pin()->GetCommandList()
			                                               : TSharedPtr<FUICommandList>();
		FMenuBuilder Menu(true, CommandList);

		const TArray<TSharedPtr<FBoneGroupTreeElement>> CurSelectedItems = GetSelectedItems();
		Menu.BeginSection("BoneGroupActions", LOCTEXT("BoneGroupActionsHeading", "Bone Group Actions"));
		{
			if (CurSelectedItems.Num() == 1 && CurSelectedItems[0]->IsGroup())
			{
				Menu.AddMenuEntry(Actions.AddBoneToGroup);
			}

			if (!CurSelectedItems.IsEmpty())
			{
				Menu.AddMenuEntry(Actions.DeleteSelectedItems);
			}
		}
		Menu.EndSection();

		return Menu.MakeWidget();
	}

	int32 SBoneGroupsTreeWidget::GetNumSelectedGroups() const
	{
		int32 NumSelectedGroups = 0;
		for (const TSharedPtr<FBoneGroupTreeElement>& Item : SelectedItems)
		{
			if (Item->IsGroup())
			{
				NumSelectedGroups++;
			}
		}
		return NumSelectedGroups;
	}

	void SBoneGroupsTreeWidget::AddElement(const TSharedPtr<FBoneGroupTreeElement>& Element, const TSharedPtr<FBoneGroupTreeElement>& ParentElement)
	{
		if (!ParentElement)
		{
			RootElements.Add(Element);
		}
		else
		{
			ParentElement->Children.Add(Element);
			Element->ParentGroup = ParentElement->AsWeak();
		}
	}

	TSharedRef<ITableRow> FBoneGroupTreeElement::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable,
	                                                               const TSharedRef<FBoneGroupTreeElement>& InTreeElement,
	                                                               const TSharedPtr<SBoneGroupsTreeWidget>& InTreeWidget)
	{
		return SNew(SBoneGroupTreeRowWidget, InOwnerTable, InTreeElement, InTreeWidget);
	}

	TSharedRef<ITableRow> SBoneGroupsTreeWidget::MakeTableRowWidget(TSharedPtr<FBoneGroupTreeElement> InItem,
	                                                                const TSharedRef<STableViewBase>& OwnerTable)
	{
		return InItem->MakeTreeRowWidget(OwnerTable, InItem.ToSharedRef(), SharedThis(this));
	}

	void SBoneGroupsTreeWidget::HandleGetChildrenForTree(TSharedPtr<FBoneGroupTreeElement> InItem,
	                                                     TArray<TSharedPtr<FBoneGroupTreeElement>>& OutChildren)
	{
		OutChildren = InItem.Get()->Children;
	}

	void SBoneGroupsTreeWidget::UpdateTreeElements()
	{
		RootElements.Reset();
		if (!BoneGroupsWidget.IsValid())
		{
			return;
		}

		const FLinearColor ErrorColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.InputsWidget.ErrorColor");

		const TSharedPtr<UE::MLDeformer::TrainingDataProcessor::SBoneGroupsListWidget> GroupWidget = BoneGroupsWidget.Pin();
		TArray<FMLDeformerTrainingDataProcessorBoneGroup>* BoneGroups = GroupWidget->GetBoneGroupsValues();
		if (!BoneGroups)
		{
			return;
		}

		TWeakObjectPtr<USkeleton> Skeleton = GroupWidget->GetSkeleton();
		if (!Skeleton.IsValid())
		{
			return;
		}

		const TStrongObjectPtr<USkeleton> StrongSkeleton = Skeleton.Pin();
		const FReferenceSkeleton& RefSkeleton = StrongSkeleton->GetReferenceSkeleton();

		const FString& FilterText = GroupWidget->GetFilterText();

		const int32 NumGroups = BoneGroups->Num();
		for (int32 BoneGroupIndex = 0; BoneGroupIndex < NumGroups; ++BoneGroupIndex)
		{
			const FMLDeformerTrainingDataProcessorBoneGroup& BoneGroup = (*BoneGroups)[BoneGroupIndex];

			bool bGroupHasVisibleBones = false;
			if (!FilterText.IsEmpty())
			{
				for (const FName BoneName : BoneGroup.BoneNames)
				{
					if (BoneName.ToString().Contains(FilterText))
					{
						bGroupHasVisibleBones = true;
						break;
					}
				}
			}
			else
			{
				bGroupHasVisibleBones = true;
			}

			if (!bGroupHasVisibleBones)
			{
				continue;
			}

			bool bGroupHasError = false;
			for (const FName BoneName : BoneGroup.BoneNames)
			{
				if (BoneName.IsNone() || RefSkeleton.FindBoneIndex(BoneName) == INDEX_NONE)
				{
					bGroupHasError = true;
					break;
				}
			}

			// Add the group header.
			TSharedPtr<FBoneGroupTreeElement> GroupElement = MakeShared<FBoneGroupTreeElement>();
			GroupElement->Name = BoneGroup.GroupName;
			GroupElement->TextColor = bGroupHasError ? FSlateColor(ErrorColor) : FSlateColor::UseForeground();
			GroupElement->GroupIndex = BoneGroupIndex;
			AddElement(GroupElement, nullptr);
			SetItemExpansion(GroupElement, true);

			// Add the items in the group.
			for (int32 BoneIndex = 0; BoneIndex < BoneGroup.BoneNames.Num(); ++BoneIndex)
			{
				const FName BoneName = BoneGroup.BoneNames[BoneIndex];
				if (!FilterText.IsEmpty() && !BoneName.ToString().Contains(FilterText))
				{
					continue;
				}

				const bool bBoneHasError = (BoneName.IsNone() || RefSkeleton.FindBoneIndex(BoneName) == INDEX_NONE);

				TSharedPtr<FBoneGroupTreeElement> ItemElement = MakeShared<FBoneGroupTreeElement>();
				ItemElement->Name = BoneName.ToString();
				ItemElement->TextColor = bBoneHasError ? FSlateColor(ErrorColor) : FSlateColor::UseForeground();
				ItemElement->GroupBoneIndex = BoneIndex;
				AddElement(ItemElement, GroupElement);

				bGroupHasError |= bBoneHasError;
			}

			if (bGroupHasError)
			{
				GroupElement->TextColor = FSlateColor(ErrorColor);
			}
		}
	}

	FReply SBoneGroupsTreeWidget::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
	{
		TSharedPtr<FUICommandList> CommandList = BoneGroupsWidget.IsValid() ? BoneGroupsWidget.Pin()->GetCommandList() : TSharedPtr<FUICommandList>();
		if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}

		return STreeView<TSharedPtr<FBoneGroupTreeElement>>::OnKeyDown(InGeometry, InKeyEvent);
	}

	void SBoneGroupsTreeWidget::RefreshTree()
	{
		UpdateTreeElements();
		RequestTreeRefresh();
	}

	TSharedPtr<SWidget> SBoneGroupsTreeWidget::CreateContextWidget()
	{
		return TSharedPtr<SWidget>();
	}

	void SBoneGroupTreeRowWidget::Construct(const FArguments& InArgs,
	                                        const TSharedRef<STableViewBase>& OwnerTable,
	                                        const TSharedRef<FBoneGroupTreeElement>& InTreeElement,
	                                        const TSharedPtr<SBoneGroupsTreeWidget>& InTreeView)
	{
		WeakTreeElement = InTreeElement;
		STableRow<TSharedPtr<FBoneGroupTreeElement>>::Construct
		(
			STableRow<TSharedPtr<FBoneGroupTreeElement>>::FArguments()
			.ShowWires(true)
			.Content()
			[
				SNew(STextBlock)
				.Text(this, &SBoneGroupTreeRowWidget::GetName)
				.Font_Lambda([this, &InTreeElement]()
				{
					if (WeakTreeElement.IsValid() && WeakTreeElement.Pin()->IsGroup())
					{
						return FAppStyle::GetFontStyle("BoldFont");
					}
					return FAppStyle::GetFontStyle("NormalFont");
				})
				.ColorAndOpacity_Lambda([this]()
				{
					return WeakTreeElement.IsValid() ? WeakTreeElement.Pin()->TextColor : FSlateColor::UseForeground();
				})
			],
			OwnerTable
		);
	}

	FText SBoneGroupTreeRowWidget::GetName() const
	{
		if (WeakTreeElement.IsValid())
		{
			return FText::FromString(WeakTreeElement.Pin()->Name);
		}
		return FText();
	}

	void SBoneGroupsTreeWidget::Refresh()
	{
		RefreshTree();
	}

	const TArray<TSharedPtr<FBoneGroupTreeElement>>& SBoneGroupsTreeWidget::GetRootElements() const
	{
		return RootElements;
	}

	//------------------------------------------------------------
	// SBoneGroupsListWidget
	//------------------------------------------------------------
	SBoneGroupsListWidget::~SBoneGroupsListWidget()
	{
		GEditor->UnregisterForUndo(this);
	}

	void SBoneGroupsListWidget::Construct(const FArguments& InArgs, FNotifyHook* InNotifyHook)
	{
		Skeleton = InArgs._Skeleton;
		UndoObject = InArgs._UndoObject;
		GetBoneGroups = InArgs._GetBoneGroups;
		NotifyHook = InNotifyHook;

		FString ErrorMessage;
		if (!Skeleton.IsValid())
		{
			ErrorMessage += LOCTEXT("SkeletonErrorMessage", "Please pass a Skeleton to your SBoneGroupsListWidget.\n").ToString();
		}

		if (!GetBoneGroups.IsBound())
		{
			ErrorMessage += LOCTEXT("GetBoneGroupsMessage", "GetBoneGroups has not been bound in your SBoneGroupsListWidget.\n").ToString();
		}

		if (!ErrorMessage.IsEmpty())
		{
			ChildSlot
			[
				SNew(STextBlock)
				.Text(FText::FromString(ErrorMessage))
				.AutoWrapText(true)
				.ColorAndOpacity(FLinearColor(1.0f, 1.0f, 0.0f))
			];
		}
		else
		{
			ChildSlot
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(FMargin(0.0f, 4.0f, 0.0f, 2.0f))
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.FillWidth(1.0f)
					.VAlign(VAlign_Center)
					[
						SNew(SSearchBox)
						.HintText(LOCTEXT("BoneGroupsSearchBoxHint", "Search"))
						.OnTextChanged(this, &SBoneGroupsListWidget::OnFilterTextChanged)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(1.0f, 1.0f)
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.ToolTipText(LOCTEXT("AddButtonToolTip", "Create and add a new bone group."))
						.OnClicked(this, &SBoneGroupsListWidget::OnAddButtonClicked)
						.ContentPadding(FMargin(0))
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(1.0f, 1.0f)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.ToolTipText(LOCTEXT("ClearButtonToolTip", "Clear all bone groups."))
						.OnClicked(this, &SBoneGroupsListWidget::OnClearButtonClicked)
						.ContentPadding(FMargin(0))
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]

				]
				+ SVerticalBox::Slot()
				.MinHeight(100.0f)
				.MaxHeight(300.0f)
				.Padding(FMargin(0.0f, 0.0f, 4.0f, 4.0f))
				[
					SAssignNew(TreeWidget, SBoneGroupsTreeWidget)
					.BoneGroupsWidget(SharedThis(this))
				]
			];

			CommandList = MakeShared<FUICommandList>();
			BindCommands(CommandList);
			RefreshTree();
		}

		GEditor->RegisterForUndo(this);
	}

	void SBoneGroupsListWidget::BindCommands(const TSharedPtr<FUICommandList>& InCommandList)
	{
		const FBoneGroupsListWidgetCommands& GroupCommands = FBoneGroupsListWidgetCommands::Get();
		InCommandList->MapAction(GroupCommands.CreateGroup, FExecuteAction::CreateSP(this, &SBoneGroupsListWidget::OnCreateBoneGroup));
		InCommandList->MapAction(GroupCommands.DeleteSelectedItems, FExecuteAction::CreateSP(this, &SBoneGroupsListWidget::OnDeleteSelectedItems));
		InCommandList->MapAction(GroupCommands.ClearGroups, FExecuteAction::CreateSP(this, &SBoneGroupsListWidget::OnClearBoneGroups));
		InCommandList->MapAction(GroupCommands.AddBoneToGroup, FExecuteAction::CreateSP(this, &SBoneGroupsListWidget::OnAddBoneToGroup));
	}

	void SBoneGroupsListWidget::OnFilterTextChanged(const FText& InFilterText)
	{
		FilterText = InFilterText.ToString();
		RefreshTree();
	}

	void SBoneGroupsListWidget::RefreshTree() const
	{
		if (TreeWidget.IsValid())
		{
			TreeWidget->RefreshTree();
		}
	}

	void SBoneGroupsListWidget::OnCreateBoneGroup() const
	{
		if (!Skeleton.IsValid())
		{
			return;
		}

		TArray<FMLDeformerTrainingDataProcessorBoneGroup>* BoneGroups = GetBoneGroupsValues();
		if (!BoneGroups)
		{
			return;
		}

		const TStrongObjectPtr<USkeleton> StrongSkeleton = Skeleton.Pin();
		const FReferenceSkeleton& RefSkeleton = StrongSkeleton->GetReferenceSkeleton();

		// Remove bones that are already in a bone group.
		// This prevents the user from adding bones that exist in multiple groups.
		TArray<FName> AllowedBones = RefSkeleton.GetRawRefBoneNames();
		for (const FMLDeformerTrainingDataProcessorBoneGroup& BoneGroup : *BoneGroups)
		{
			for (const FName BoneName : BoneGroup.BoneNames)
			{
				AllowedBones.Remove(BoneName);
			}
		}

		const TSharedPtr<SMLDeformerBonePickerDialog> Dialog =
			SNew(SMLDeformerBonePickerDialog)
			.RefSkeleton(&RefSkeleton)
			.AllowMultiSelect(true)
			.IncludeList(AllowedBones);

		Dialog->ShowModal();

		const TArray<FName>& BoneNames = Dialog->GetPickedBoneNames();
		if (BoneNames.IsEmpty())
		{
			return;
		}

		TStrongObjectPtr<UObject> StrongUndoObject = UndoObject.Pin();
		check(StrongUndoObject);

		FScopedTransaction Transaction(TEXT("SBoneGroupsListWidget"), LOCTEXT("CreateBoneGroupText", "Create Bone Group"), StrongUndoObject.Get());
		StrongUndoObject->Modify();

		BoneGroups->AddDefaulted();
		FMLDeformerTrainingDataProcessorBoneGroup& BoneGroup = BoneGroups->Last();
		BoneGroup.GroupName = TEXT("Bone Group");
		for (const FName BoneName : BoneNames)
		{
			BoneGroup.BoneNames.AddUnique(BoneName);
		}

		RefreshTree();

		// Trigger an event about adding the group.
		if (NotifyHook)
		{
			FProperty* GroupsProperty = FindFieldChecked<FProperty>(FMLDeformerTrainingDataProcessorBoneGroupsList::StaticStruct(),
			                                                        GET_MEMBER_NAME_CHECKED(FMLDeformerTrainingDataProcessorBoneGroupsList, Groups));
			FPropertyChangedEvent GroupsEvent(GroupsProperty, EPropertyChangeType::ArrayAdd);
			NotifyHook->NotifyPostChange(GroupsEvent, GroupsProperty);
		}
	}

	void SBoneGroupsListWidget::OnDeleteSelectedItems() const
	{
		TArray<FMLDeformerTrainingDataProcessorBoneGroup>* BoneGroups = GetBoneGroupsValues();
		if (!BoneGroups)
		{
			return;
		}

		TStrongObjectPtr<UObject> StrongUndoObject = UndoObject.Pin();
		check(StrongUndoObject);

		const TArray<TSharedPtr<FBoneGroupTreeElement>> CurSelectedItems = TreeWidget->GetSelectedItems();

		TArray<int32> GroupsToRemove;
		TArray<TSharedPtr<FBoneGroupTreeElement>> SelectedGroups;
		TArray<TSharedPtr<FBoneGroupTreeElement>> SelectedBones;

		// Check if the selection contains bones and/or groups.
		for (const TSharedPtr<FBoneGroupTreeElement>& SelectedItem : CurSelectedItems)
		{
			if (SelectedItem->IsGroup())
			{
				SelectedGroups.Add(SelectedItem);
				GroupsToRemove.Add(SelectedItem->GroupIndex);
			}
			else
			{
				SelectedBones.Add(SelectedItem);
			}
		}

		FScopedTransaction Transaction(TEXT("SBoneGroupsListWidget"), LOCTEXT("RemoveBoneGroupItemsText", "Remove Bones from Group"),
		                               StrongUndoObject.Get());
		StrongUndoObject->Modify();

		// Remove all selected bones.
		if (!SelectedBones.IsEmpty())
		{
			for (const TSharedPtr<FBoneGroupTreeElement>& BoneItem : SelectedBones)
			{
				if (BoneItem.IsValid() && BoneItem->ParentGroup.IsValid())
				{
					const TSharedPtr<FBoneGroupTreeElement>& ParentGroup = BoneItem->ParentGroup.Pin();
					FMLDeformerTrainingDataProcessorBoneGroup& ParentBoneGroup = (*BoneGroups)[ParentGroup->GroupIndex];
					ParentBoneGroup.BoneNames.Remove(FName(BoneItem->Name));
				}
			}

			// Trigger an event about changing the bone names for this group.
			if (NotifyHook)
			{
				FProperty* BoneNamesProperty = FindFieldChecked<FProperty>(FMLDeformerTrainingDataProcessorBoneGroup::StaticStruct(),
				                                                           GET_MEMBER_NAME_CHECKED(
					                                                           FMLDeformerTrainingDataProcessorBoneGroup, BoneNames));
				const FPropertyChangedEvent BoneNamesEvent(BoneNamesProperty, EPropertyChangeType::ArrayRemove);
				NotifyHook->NotifyPostChange(BoneNamesEvent, BoneNamesProperty);
			}
		}

		// Sort group indices big to small (back to front).
		if (!GroupsToRemove.IsEmpty())
		{
			GroupsToRemove.Sort([](const int32& A, const int32& B) { return A > B; });

			// Remove the items, back to front.
			for (const int32 Index : GroupsToRemove)
			{
				BoneGroups->RemoveAt(Index);
			}

			// Trigger an event about groups being deleted.
			if (NotifyHook)
			{
				FProperty* GroupsProperty = FindFieldChecked<FProperty>(FMLDeformerTrainingDataProcessorBoneGroupsList::StaticStruct(),
				                                                        GET_MEMBER_NAME_CHECKED(
					                                                        FMLDeformerTrainingDataProcessorBoneGroupsList, Groups));
				const FPropertyChangedEvent GroupsEvent(GroupsProperty, EPropertyChangeType::ArrayRemove);
				NotifyHook->NotifyPostChange(GroupsEvent, GroupsProperty);
			}
		}

		RefreshTree();
	}

	void SBoneGroupsListWidget::OnClearBoneGroups() const
	{
		const TStrongObjectPtr<UObject> StrongUndoObject = UndoObject.Pin();
		check(StrongUndoObject);

		FScopedTransaction Transaction(LOCTEXT("ClearBoneGroupItemsText", "Clear Bone Groups"));
		StrongUndoObject->Modify();

		TArray<FMLDeformerTrainingDataProcessorBoneGroup>* BoneGroups = GetBoneGroupsValues();
		if (BoneGroups)
		{
			BoneGroups->Empty();
			RefreshTree();

			// Trigger an event about clearing the group.
			if (NotifyHook)
			{
				FProperty* GroupsProperty = FindFieldChecked<FProperty>(
					FMLDeformerTrainingDataProcessorBoneGroupsList::StaticStruct(),
					GET_MEMBER_NAME_CHECKED(FMLDeformerTrainingDataProcessorBoneGroupsList, Groups));
				const FPropertyChangedEvent GroupsEvent(GroupsProperty, EPropertyChangeType::ArrayClear);
				NotifyHook->NotifyPostChange(GroupsEvent, GroupsProperty);
			}
		}
	}

	void SBoneGroupsListWidget::OnAddBoneToGroup() const
	{
		if (!Skeleton.IsValid() || !TreeWidget.IsValid())
		{
			return;
		}

		const TStrongObjectPtr<USkeleton> StrongSkeleton = Skeleton.Pin();
		const FReferenceSkeleton& RefSkeleton = StrongSkeleton->GetReferenceSkeleton();

		const TStrongObjectPtr<UObject> StrongUndoObject = UndoObject.Pin();
		check(StrongUndoObject);

		// Find the group we want to add something to.
		check(TreeWidget->GetSelectedItems().Num() == 1);
		const int32 GroupIndex = TreeWidget->GetSelectedItems()[0]->GroupIndex;
		check(GroupIndex != INDEX_NONE);

		TArray<FMLDeformerTrainingDataProcessorBoneGroup>* BoneGroups = GetBoneGroupsValues();
		FMLDeformerTrainingDataProcessorBoneGroup& BoneGroup = (*BoneGroups)[GroupIndex];

		// Build the highlighted bone names list.
		TArray<FName> HighlightedBones;
		HighlightedBones.Reserve(BoneGroup.BoneNames.Num());
		for (const FName BoneName : BoneGroup.BoneNames)
		{
			HighlightedBones.Add(BoneName);
		}

		const FLinearColor HighlightColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.InputsWidget.HighlightColor");
		const TSharedPtr<SMLDeformerBonePickerDialog> Dialog =
			SNew(SMLDeformerBonePickerDialog)
			.RefSkeleton(&RefSkeleton)
			.AllowMultiSelect(true)
			.HighlightBoneNames(HighlightedBones)
			.HighlightBoneNamesColor(HighlightColor);

		Dialog->ShowModal();

		const TArray<FName>& BoneNames = Dialog->GetPickedBoneNames();
		if (!BoneNames.IsEmpty())
		{
			FScopedTransaction Transaction(LOCTEXT("AddBoneGroupItemsText", "Add Bones To Group"));
			StrongUndoObject->Modify();

			for (const FName BoneName : BoneNames)
			{
				BoneGroup.BoneNames.AddUnique(BoneName);
			}

			RefreshTree();

			// Trigger an event about changing the bone names for this group.
			if (NotifyHook)
			{
				FProperty* BoneNamesProperty = FindFieldChecked<FProperty>(FMLDeformerTrainingDataProcessorBoneGroup::StaticStruct(),
				                                                           GET_MEMBER_NAME_CHECKED(
					                                                           FMLDeformerTrainingDataProcessorBoneGroup, BoneNames));
				const FPropertyChangedEvent BoneNamesEvent(BoneNamesProperty, EPropertyChangeType::ValueSet);
				NotifyHook->NotifyPostChange(BoneNamesEvent, BoneNamesProperty);
			}
		}
	}

	FReply SBoneGroupsListWidget::OnAddButtonClicked() const
	{
		OnCreateBoneGroup();
		return FReply::Handled();
	}

	FReply SBoneGroupsListWidget::OnClearButtonClicked() const
	{
		OnClearBoneGroups();
		return FReply::Handled();
	}

	TSharedPtr<SBoneGroupsTreeWidget> SBoneGroupsListWidget::GetTreeWidget() const
	{
		return TreeWidget;
	}

	TSharedPtr<FUICommandList> SBoneGroupsListWidget::GetCommandList() const
	{
		return CommandList;
	}

	TArray<FMLDeformerTrainingDataProcessorBoneGroup>* SBoneGroupsListWidget::GetBoneGroupsValues() const
	{
		check(GetBoneGroups.IsBound());
		return GetBoneGroups.Execute();
	}

	TWeakObjectPtr<USkeleton> SBoneGroupsListWidget::GetSkeleton() const
	{
		return Skeleton;
	}

	void SBoneGroupsListWidget::PostUndo(bool bSuccess)
	{
		RefreshTree();
	}

	void SBoneGroupsListWidget::PostRedo(bool bSuccess)
	{
		RefreshTree();
	}

	const FString& SBoneGroupsListWidget::GetFilterText() const
	{
		return FilterText;
	}
} // namespace UE::MLDeformer::TrainingDataProcessor

#undef LOCTEXT_NAMESPACE
