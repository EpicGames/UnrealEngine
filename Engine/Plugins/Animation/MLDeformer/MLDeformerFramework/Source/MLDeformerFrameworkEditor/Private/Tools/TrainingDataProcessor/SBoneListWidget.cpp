// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/TrainingDataProcessor/SBoneListWidget.h"
#include "MLDeformerTrainingDataProcessorSettings.h"
#include "MLDeformerModule.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Animation/Skeleton.h"
#include "SMLDeformerBonePickerDialog.h"
#include "Framework/Commands/UICommandList.h"
#include "ScopedTransaction.h"
#include "Editor.h"
#include "Misc/NotifyHook.h"

#define LOCTEXT_NAMESPACE "MLDeformerTrainingDataProcessorBoneListCustomize"

namespace UE::MLDeformer::TrainingDataProcessor
{
	TSharedRef<ITableRow> FBoneTreeWidgetElement::MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable,
	                                                                TSharedRef<FBoneTreeWidgetElement> InTreeElement,
	                                                                TSharedPtr<SBoneTreeWidget> InTreeWidget)
	{
		return SNew(SBoneTreeRowWidget, InOwnerTable, InTreeElement, InTreeWidget);
	}

	void SBoneTreeWidget::Construct(const FArguments& InArgs)
	{
		BoneListWidget = InArgs._BoneListWidget;

		STreeView<TSharedPtr<FBoneTreeWidgetElement>>::FArguments SuperArgs;
		SuperArgs.TreeItemsSource(&RootElements);
		SuperArgs.SelectionMode(ESelectionMode::Multi);
		SuperArgs.OnGenerateRow(this, &SBoneTreeWidget::MakeTableRowWidget);
		SuperArgs.OnGetChildren_Static(&SBoneTreeWidget::HandleGetChildrenForTree);
		SuperArgs.OnContextMenuOpening(this, &SBoneTreeWidget::OnContextMenuOpening);
		SuperArgs.AllowInvisibleItemSelection(true);

		STreeView<TSharedPtr<FBoneTreeWidgetElement>>::Construct(SuperArgs);
	}

	TSharedPtr<SWidget> SBoneTreeWidget::OnContextMenuOpening() const
	{
		const FBoneListWidgetCommands& Actions = FBoneListWidgetCommands::Get();
		TSharedPtr<FUICommandList> CommandList = BoneListWidget.IsValid() ? BoneListWidget.Pin()->GetCommandList() : TSharedPtr<FUICommandList>();
		FMenuBuilder Menu(true, CommandList);
		Menu.BeginSection("BoneActions", LOCTEXT("BoneActionsHeading", "Bone Actions"));
		{
			Menu.AddMenuEntry(Actions.AddBones);

			if (!GetSelectedItems().IsEmpty())
			{
				Menu.AddMenuEntry(Actions.RemoveBones);
			}

			if (!RootElements.IsEmpty())
			{
				Menu.AddMenuEntry(Actions.ClearBones);
			}
		}
		Menu.EndSection();

		return Menu.MakeWidget();
	}

	void SBoneTreeWidget::HandleGetChildrenForTree(TSharedPtr<FBoneTreeWidgetElement> InItem, TArray<TSharedPtr<FBoneTreeWidgetElement>>& OutChildren)
	{
		OutChildren = InItem.Get()->Children;
	}

	void SBoneTreeWidget::RecursiveSortElements(const TSharedPtr<FBoneTreeWidgetElement>& Element)
	{
		if (Element.IsValid())
		{
			Element->Children.Sort(
				[](const TSharedPtr<FBoneTreeWidgetElement>& ItemA, const TSharedPtr<FBoneTreeWidgetElement>& ItemB)
				{
					return (ItemA->Name.ToString() < ItemB->Name.ToString());
				});

			for (const TSharedPtr<FBoneTreeWidgetElement>& Child : Element->Children)
			{
				RecursiveSortElements(Child);
			}
		}
		else // We need to sort the root level.
		{
			RootElements.Sort(
				[](const TSharedPtr<FBoneTreeWidgetElement>& ItemA, const TSharedPtr<FBoneTreeWidgetElement>& ItemB)
				{
					return (ItemA->Name.ToString() < ItemB->Name.ToString());
				});

			for (const TSharedPtr<FBoneTreeWidgetElement>& Child : RootElements)
			{
				RecursiveSortElements(Child);
			}
		}
	}

	void SBoneTreeWidget::RefreshElements(const TArray<FName>& BoneNames, const FReferenceSkeleton* RefSkeleton, const FString& FilterText)
	{
		RootElements.Reset();

		const FSlateColor ErrorColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.InputsWidget.ErrorColor");

		// If we have no reference skeleton, just add everything as flat list as we don't have hierarchy data.
		if (!RefSkeleton)
		{
			for (const FName BoneName : BoneNames)
			{
				if (FilterText.IsEmpty() || BoneName.ToString().Contains(FilterText))
				{
					TSharedPtr<FBoneTreeWidgetElement> Element = MakeShared<FBoneTreeWidgetElement>();
					Element->Name = BoneName;
					Element->TextColor = ErrorColor;
					RootElements.Add(Element);
				}
			}
		}
		else
		{
			// Add all the bones to some element map so we know the tree element for a given bone name.
			TMap<FName, TSharedPtr<FBoneTreeWidgetElement>> NameToElementMap;
			for (const FName BoneName : BoneNames)
			{
				if (!BoneName.ToString().Contains(FilterText))
				{
					continue;
				}

				const int32 RefSkelBoneIndex = RefSkeleton->FindBoneIndex(BoneName);
				TSharedPtr<FBoneTreeWidgetElement> Element = MakeShared<FBoneTreeWidgetElement>();
				Element->Name = BoneName;
				Element->TextColor = (RefSkelBoneIndex != INDEX_NONE) ? FSlateColor::UseForeground() : ErrorColor;
				NameToElementMap.Add(BoneName, Element);
			}

			// Handle parents and specify root items.
			for (auto& Element : NameToElementMap)
			{
				TSharedPtr<FBoneTreeWidgetElement> ParentElement = FindParentElementForBone(Element.Key, *RefSkeleton, NameToElementMap);
				if (ParentElement.IsValid())
				{
					ParentElement->Children.Add(Element.Value);
				}
				else
				{
					RootElements.Add(Element.Value);
				}

				SetItemExpansion(Element.Value, true);
			}
		}

		RecursiveSortElements(TSharedPtr<FBoneTreeWidgetElement>());
	}

	TSharedPtr<FBoneTreeWidgetElement> SBoneTreeWidget::FindParentElementForBone(FName BoneName, const FReferenceSkeleton& RefSkeleton,
	                                                                             const TMap<FName, TSharedPtr<FBoneTreeWidgetElement>>&
	                                                                             NameToElementMap)
	{
		const int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);

		if (BoneIndex != INDEX_NONE)
		{
			int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
			while (ParentIndex != INDEX_NONE)
			{
				const FName ParentName = RefSkeleton.GetBoneName(ParentIndex);
				const TSharedPtr<FBoneTreeWidgetElement>* ParentElementPtr = NameToElementMap.Find(ParentName);
				if (ParentElementPtr)
				{
					return *ParentElementPtr;
				}

				ParentIndex = RefSkeleton.GetParentIndex(ParentIndex);
			}
		}

		return TSharedPtr<FBoneTreeWidgetElement>();
	}

	TSharedRef<ITableRow> SBoneTreeWidget::MakeTableRowWidget(TSharedPtr<FBoneTreeWidgetElement> InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return InItem->MakeTreeRowWidget(OwnerTable, InItem.ToSharedRef(), SharedThis(this));
	}

	FReply SBoneTreeWidget::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
	{
		const TSharedPtr<FUICommandList> CommandList = BoneListWidget.IsValid()
			                                               ? BoneListWidget.Pin()->GetCommandList()
			                                               : TSharedPtr<FUICommandList>();
		if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}

		return STreeView<TSharedPtr<FBoneTreeWidgetElement>>::OnKeyDown(InGeometry, InKeyEvent);
	}

	void SBoneTreeWidget::RecursiveAddNames(const FBoneTreeWidgetElement& Element, TArray<FName>& OutNames)
	{
		OutNames.Add(Element.Name);
		for (const TSharedPtr<FBoneTreeWidgetElement>& ChildElement : Element.Children)
		{
			RecursiveAddNames(*ChildElement, OutNames);
		}
	}

	TArray<FName> SBoneTreeWidget::ExtractAllElementNames() const
	{
		TArray<FName> Names;
		for (const TSharedPtr<FBoneTreeWidgetElement>& Element : RootElements)
		{
			RecursiveAddNames(*Element, Names);
		}
		return MoveTemp(Names);
	}

	void SBoneTreeRowWidget::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable,
	                                   const TSharedRef<FBoneTreeWidgetElement>& InTreeElement, const TSharedPtr<SBoneTreeWidget>& InTreeView)
	{
		TreeElement = InTreeElement;

		STableRow<TSharedPtr<FBoneTreeWidgetElement>>::Construct
		(
			STableRow<TSharedPtr<FBoneTreeWidgetElement>>::FArguments()
			.ShowWires(true)
			.Content()
			[
				SNew(STextBlock)
				.Text(this, &SBoneTreeRowWidget::GetName)
				.ColorAndOpacity_Lambda
				(
					[this]()
					{
						return TreeElement.IsValid() ? TreeElement.Pin()->TextColor : FSlateColor::UseForeground();
					}
				)
			],
			OwnerTable
		);
	}

	FText SBoneTreeRowWidget::GetName() const
	{
		if (TreeElement.IsValid())
		{
			return FText::FromName(TreeElement.Pin()->Name);
		}
		return FText();
	}

	SBoneListWidget::~SBoneListWidget()
	{
		GEditor->UnregisterForUndo(this);
	}

	void SBoneListWidget::Construct(const FArguments& InArgs, FNotifyHook* InNotifyHook)
	{
		NotifyHook = InNotifyHook;

		Skeleton = InArgs._Skeleton;
		UndoObject = InArgs._UndoObject;
		GetBoneNames = InArgs._GetBoneNames;
		OnBonesAdded = InArgs._OnBonesAdded;
		OnBonesRemoved = InArgs._OnBonesRemoved;
		OnBonesCleared = InArgs._OnBonesCleared;

		FString ErrorMessage;
		if (!Skeleton.IsValid())
		{
			ErrorMessage += LOCTEXT("SkeletonErrorMessage", "Please pass a Skeleton to your SBoneListWidget.\n").ToString();
		}

		if (!GetBoneNames.IsBound())
		{
			ErrorMessage += LOCTEXT("GetBoneNamesMessage", "GetBoneNames has not been bound in your SBoneListWidget.\n").ToString();
		}

		if (!ErrorMessage.IsEmpty())
		{
			ChildSlot
			[
				SNew(STextBlock)
				.Text(FText::FromString(ErrorMessage))
				.AutoWrapText(true)
				.ColorAndOpacity(FLinearColor(1.0f, 1.0f, 0.0f))
			];
		}
		else
		{
			ChildSlot
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(FMargin(0.0f, 4.0f, 0.0f, 2.0f))
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.FillWidth(1.0f)
					.VAlign(VAlign_Center)
					[
						SNew(SSearchBox)
						.HintText(LOCTEXT("BonesSearchBoxHint", "Search Bones"))
						.OnTextChanged(this, &SBoneListWidget::OnFilterTextChanged)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(1.0f, 1.0f)
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.ToolTipText(LOCTEXT("AddButtonToolTip", "Add bones to the list."))
						.OnClicked(this, &SBoneListWidget::OnAddBonesButtonClicked)
						.ContentPadding(FMargin(0))
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(1.0f, 1.0f)
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.ToolTipText(LOCTEXT("ClearButtonToolTip", "Clear the bone list."))
						.OnClicked(this, &SBoneListWidget::OnClearBonesButtonClicked)
						.ContentPadding(FMargin(0))
						[
							SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
				]
				+ SVerticalBox::Slot()
				.MinHeight(100.0f)
				.MaxHeight(300.0f)
				.Padding(FMargin(0.0f, 0.0f, 4.0f, 4.0f))
				[
					SAssignNew(TreeWidget, SBoneTreeWidget)
					.BoneListWidget(SharedThis(this))
				]
			];

			CommandList = MakeShared<FUICommandList>();
			BindCommands(CommandList);
			RefreshTree();
		}

		GEditor->RegisterForUndo(this);
	}

	FReply SBoneListWidget::OnAddBonesButtonClicked() const
	{
		OnAddBones();
		return FReply::Handled();
	}

	FReply SBoneListWidget::OnClearBonesButtonClicked() const
	{
		OnClearBones();
		return FReply::Handled();
	}

	void SBoneListWidget::OnFilterTextChanged(const FText& InFilterText)
	{
		FilterText = InFilterText.ToString();
		RefreshTree();
	}

	void SBoneListWidget::BindCommands(const TSharedPtr<FUICommandList>& InCommandList)
	{
		const FBoneListWidgetCommands& Commands = FBoneListWidgetCommands::Get();
		InCommandList->MapAction(Commands.AddBones, FExecuteAction::CreateSP(this, &SBoneListWidget::OnAddBones));
		InCommandList->MapAction(Commands.RemoveBones, FExecuteAction::CreateSP(this, &SBoneListWidget::OnRemoveBones));
		InCommandList->MapAction(Commands.ClearBones, FExecuteAction::CreateSP(this, &SBoneListWidget::OnClearBones));
	}

	void SBoneListWidget::OnAddBones() const
	{
		if (!GetBoneNames.IsBound())
		{
			UE_LOGF(LogMLDeformer, Warning, "Please set a GetBoneNames to your SBoneListWidget when creating your SBoneListWidget.");
			return;
		}

		if (!Skeleton.IsValid())
		{
			UE_LOGF(LogMLDeformer, Warning, "No skeleton is available to pick bones from");
			return;
		}

		const FSlateColor HighlightColor = FMLDeformerEditorStyle::Get().GetColor("MLDeformer.InputsWidget.HighlightColor");
		const FReferenceSkeleton* RefSkeleton = const_cast<FReferenceSkeleton*>(&Skeleton->GetReferenceSkeleton());

		TArray<FName>* BoneNames = GetBoneNames.Execute();
		if (!BoneNames)
		{
			UE_LOGF(LogMLDeformer, Warning, "There are no bones that can be added.");
			return;
		}

		TSharedPtr<SMLDeformerBonePickerDialog> Dialog =
			SNew(SMLDeformerBonePickerDialog)
			.RefSkeleton(RefSkeleton)
			.AllowMultiSelect(true)
			.HighlightBoneNamesColor(HighlightColor)
			.HighlightBoneNames(*BoneNames);

		Dialog->ShowModal();

		const TArray<FName>& PickedBoneNames = Dialog->GetPickedBoneNames();
		if (!PickedBoneNames.IsEmpty())
		{
			FScopedTransaction Transaction(LOCTEXT("AddBonesText", "Add Bones"));
			if (UndoObject.IsValid())
			{
				UndoObject->Modify();
			}

			TArray<FName> BonesAdded;
			for (const FName PickedBoneName : PickedBoneNames)
			{
				if (!BoneNames->Contains(PickedBoneName))
				{
					BoneNames->Add(PickedBoneName);
					BonesAdded.Add(PickedBoneName);
				}
			}

			// Broadcast that bones got added.
			if (!BonesAdded.IsEmpty())
			{
				OnBonesAdded.ExecuteIfBound(BonesAdded);
				NotifyPropertyChanged();
			}

			RefreshTree();
		}
	}

	void SBoneListWidget::OnClearBones() const
	{
		if (!GetBoneNames.IsBound())
		{
			UE_LOGF(LogMLDeformer, Warning, "Please set a GetBoneNames to your SBoneListWidget when creating your SBoneListWidget.");
			return;
		}

		FScopedTransaction Transaction(LOCTEXT("ClearBonesText", "Clear Bones"));
		if (UndoObject.IsValid())
		{
			UndoObject->Modify();
		}

		TArray<FName>* BoneNames = GetBoneNames.Execute();
		if (BoneNames && !BoneNames->IsEmpty())
		{
			BoneNames->Empty();
			OnBonesCleared.ExecuteIfBound();
			RefreshTree();
			TreeWidget->ClearSelection();
			NotifyPropertyChanged();
		}
	}

	void SBoneListWidget::OnRemoveBones() const
	{
		if (!GetBoneNames.IsBound())
		{
			UE_LOGF(LogMLDeformer, Warning, "Please set a GetBoneNames to your SBoneListWidget when creating your SBoneListWidget.");
			return;
		}

		const TArray<TSharedPtr<FBoneTreeWidgetElement>> SelectedItems = TreeWidget->GetSelectedItems();
		if (SelectedItems.IsEmpty())
		{
			return;
		}

		FScopedTransaction Transaction(LOCTEXT("RemoveBonesText", "Remove Bones"));
		if (UndoObject.IsValid())
		{
			UndoObject->Modify();
		}

		TArray<FName>* BoneNames = GetBoneNames.Execute();
		if (BoneNames)
		{
			TArray<FName> BoneNamesRemoved;
			for (const TSharedPtr<FBoneTreeWidgetElement>& Item : SelectedItems)
			{
				BoneNames->Remove(Item->Name);
				BoneNamesRemoved.Add(Item->Name);
			}

			if (!BoneNamesRemoved.IsEmpty())
			{
				OnBonesRemoved.ExecuteIfBound(BoneNamesRemoved);
				NotifyPropertyChanged();
			}
		}

		RefreshTree();
		TreeWidget->ClearSelection();
	}

	void SBoneListWidget::RefreshTree() const
	{
		const FReferenceSkeleton* RefSkeleton = Skeleton.Get() ? &Skeleton->GetReferenceSkeleton() : nullptr;

		if (GetBoneNames.IsBound() && GetBoneNames.Execute())
		{
			const TArray<FName>* BoneNames = GetBoneNames.Execute();
			TreeWidget->RefreshElements(*BoneNames, RefSkeleton, FilterText);
		}
		else
		{
			TreeWidget->RefreshElements(TArray<FName>(), RefSkeleton, FilterText);
		}
		TreeWidget->RequestTreeRefresh();
	}

	FBoneListWidgetCommands::FBoneListWidgetCommands()
		: TCommands<FBoneListWidgetCommands>("Bone List", LOCTEXT("BoneListDesc", "Modify Bone List"), NAME_None,
		                                     FMLDeformerEditorStyle::Get().GetStyleSetName())
	{
	}

	void FBoneListWidgetCommands::RegisterCommands()
	{
		UI_COMMAND(AddBones, "Add Bones", "Add bones to the list.", EUserInterfaceActionType::Button, FInputChord(EKeys::Insert));
		UI_COMMAND(RemoveBones, "Delete Selected", "Deletes the selected bones from the list.", EUserInterfaceActionType::Button,
		           FInputChord(EKeys::Delete));
		UI_COMMAND(ClearBones, "Clear List", "Clears the entire list.", EUserInterfaceActionType::Button, FInputChord());
	}

	void SBoneListWidget::Refresh() const
	{
		RefreshTree();
	}

	TSharedPtr<SBoneTreeWidget> SBoneListWidget::GetTreeWidget() const
	{
		return TreeWidget;
	}

	TSharedPtr<FUICommandList> SBoneListWidget::GetCommandList() const
	{
		return CommandList;
	}

	void SBoneListWidget::NotifyPropertyChanged() const
	{
		if (NotifyHook)
		{
			FProperty* BoneNamesProperty = FindFieldChecked<FProperty>(FMLDeformerTrainingDataProcessorBoneList::StaticStruct(),
			                                                           GET_MEMBER_NAME_CHECKED(FMLDeformerTrainingDataProcessorBoneList, BoneNames));
			NotifyHook->NotifyPostChange(FPropertyChangedEvent(BoneNamesProperty, EPropertyChangeType::ValueSet), BoneNamesProperty);
		}
	}

	void SBoneListWidget::PostUndo(bool bSuccess)
	{
		RefreshTree();
	}

	void SBoneListWidget::PostRedo(bool bSuccess)
	{
		RefreshTree();
	}
} // namespace UE::MLDeformer::TrainingDataProcessor

#undef LOCTEXT_NAMESPACE
