// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/TrainingDataProcessor/STrainingDataProcessorMainWidget.h"
#include "MLDeformerModel.h"
#include "MLDeformerTrainingDataProcessorSettings.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "DetailsViewArgs.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/MessageDialog.h"
#include "Tools/TrainingDataProcessor/TrainingDataProcessor.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/Skeleton.h"
#include "UObject/UObjectGlobals.h"

#define LOCTEXT_NAMESPACE "TrainingDataProcessorMainWidget"

namespace UE::MLDeformer::TrainingDataProcessor
{
	void STrainingDataProcessorMainWidget::Construct(const FArguments& InArgs)
	{
		Model = InArgs._Model;

		// Create the details view.
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FDetailsViewArgs Args;
		Args.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		Args.bAllowSearch = true;
		Args.bShowObjectLabel = false;
		Args.bShowPropertyMatrixButton = false;
		Args.bShowOptions = false;
		Args.bShowObjectLabel = false;
		Args.NotifyHook = this;

		DetailsView = PropertyModule.CreateDetailView(Args);
		UMLDeformerTrainingDataProcessorSettings* TrainingDataProcessorSettings = Model ? Model->GetTrainingDataProcessorSettings() : nullptr;
		DetailsView->SetObject(TrainingDataProcessorSettings);

		// Start listening for changes in the ML Deformer model, as we should validate against the skeleton setup there.
		// We use this to trigger re-inits of the UI and components, when one of our input assets got modified (by property changes, or reimports, etc.).
		// This listens to changes in ALL objects in the engine, but we apply a filter in the IsObjectOfInterest to only react on certain objects.
		ObjectModifiedHandle = FCoreUObjectDelegates::OnObjectModified.AddLambda(
			[this](UObject* Object)
			{
				if (IsObjectOfInterest(Object))
				{
					OnAssetModified(Object);
				}
			});

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				DetailsView.ToSharedRef()
			]
			+ SVerticalBox::Slot()
			.Padding(FMargin(10.0f, 4.0f))
			.AutoHeight()
			[
				SNew(SButton)
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.Text(LOCTEXT("GenerateButtonText", "Generate Training Data"))
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))
				.OnClicked(this, &STrainingDataProcessorMainWidget::OnGenerateButtonClicked)
				.IsEnabled_Lambda([this]() { return IsValidConfiguration(); })
			]
		];

		GEditor->RegisterForUndo(this);
	}

	STrainingDataProcessorMainWidget::~STrainingDataProcessorMainWidget()
	{
		GEditor->UnregisterForUndo(this);

		if (ObjectModifiedHandle.IsValid())
		{
			FCoreUObjectDelegates::OnObjectModified.Remove(ObjectModifiedHandle);
		}
	}

	void STrainingDataProcessorMainWidget::PostUndo(bool bSuccess)
	{
		FEditorUndoClient::PostUndo(bSuccess);
		Refresh();
	}

	void STrainingDataProcessorMainWidget::PostRedo(bool bSuccess)
	{
		FEditorUndoClient::PostRedo(bSuccess);
		Refresh();
	}

	void STrainingDataProcessorMainWidget::Refresh() const
	{
		if (DetailsView.IsValid())
		{
			DetailsView->ForceRefresh();
		}
	}

	void STrainingDataProcessorMainWidget::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
	{
		FNotifyHook::NotifyPostChange(PropertyChangedEvent, PropertyThatChanged);

		if (!PropertyThatChanged)
		{
			return;
		}

		// We need to refresh if our output animation changes.
		if (PropertyThatChanged->GetName() == GET_MEMBER_NAME_CHECKED(UMLDeformerTrainingDataProcessorSettings, OutputAnimSequence))
		{
			//DetailsView->Invalidate(EInvalidateWidgetReason::None);
			Refresh();
		}
	}

	bool STrainingDataProcessorMainWidget::IsValidConfiguration() const
	{
		const UMLDeformerTrainingDataProcessorSettings* TrainingDataProcessorSettings = Model ? Model->GetTrainingDataProcessorSettings() : nullptr;
		if (!TrainingDataProcessorSettings)
		{
			return false;
		}

		const USkeleton* Skeleton = (Model && Model->GetSkeletalMesh() && Model->GetSkeletalMesh()->GetSkeleton())
			                            ? Model->GetSkeletalMesh()->GetSkeleton()
			                            : nullptr;
		return TrainingDataProcessorSettings->IsValid(Skeleton);
	}

	bool STrainingDataProcessorMainWidget::IsObjectOfInterest(UObject* Object) const
	{
		if (!Model)
		{
			return false;
		}

		// If we modify our training data processor settings, we aren't interested.
		// The reason for this is that it will trigger a UI refresh when changing any property in these settings, which we don't want.
		if (Object == Model->GetTrainingDataProcessorSettings())
		{
			return false;
		}

		// Make sure we only trigger when our skeletal mesh or skeleton changes.
		const USkeletalMesh* SkeletalMesh = Model->GetSkeletalMesh();
		const USkeleton* Skeleton = SkeletalMesh ? SkeletalMesh->GetSkeleton() : nullptr;
		if (Object == Model || Object == SkeletalMesh || Object == Skeleton)
		{
			return true;
		}

		return false;
	}

	void STrainingDataProcessorMainWidget::OnAssetModified(UObject* Object) const
	{
		if (DetailsView.IsValid())
		{
			DetailsView->ForceRefresh();
		}
	}

	FReply STrainingDataProcessorMainWidget::OnGenerateButtonClicked() const
	{
		const UMLDeformerTrainingDataProcessorSettings* TrainingDataProcessorSettings = Model ? Model->GetTrainingDataProcessorSettings() : nullptr;
		if (!TrainingDataProcessorSettings)
		{
			return FReply::Handled();
		}

		// Run the actual algorithm to process the training data.
		// This will update the output animation sequence.
		FTrainingDataProcessor ProcessorAlgo;
		const USkeleton* Skeleton = Model && Model->GetSkeletalMesh() ? Model->GetSkeletalMesh()->GetSkeleton() : nullptr;
		if (!ProcessorAlgo.Execute(*TrainingDataProcessorSettings, Skeleton))
		{
			FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok, LOCTEXT("FailMessage", "Operation failed or canceled by user."),
			                     LOCTEXT("FailedTitle", "Training Data Processor"));
		}
		else
		{
			FMessageDialog::Open(EAppMsgCategory::Success, EAppMsgType::Ok, LOCTEXT("SuccessMessage", "Animation Sequence generated successfully."),
			                     LOCTEXT("SuccessTitle", "Training Data Processor"));
		}

		return FReply::Handled();
	}
} // namespace UE::MLDeformer::TrainingDataProcessor

#undef LOCTEXT_NAMESPACE
