// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/TrainingDataProcessor/TrainingDataProcessor.h"
#include "MLDeformerModule.h"
#include "ReferenceSkeleton.h"
#include "Algo/MaxElement.h"
#include "Misc/ScopedSlowTask.h"
#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "BonePose.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/AttributesRuntime.h"
#include "Async/ParallelFor.h"
#include "Math/RandomStream.h"

#define LOCTEXT_NAMESPACE "TrainingDataProcessorAlgo"

namespace UE::MLDeformer::TrainingDataProcessor
{
	bool FTrainingDataProcessor::Execute(const UMLDeformerTrainingDataProcessorSettings& Settings, const USkeleton* Skeleton)
	{
		if (!IsValid(Skeleton))
		{
			UE_LOGF(LogMLDeformer, Error, "The Training Data Processor settings are not valid.");
			return false;
		}

		// Remove existing frame data.
		Clear();

		// Now iterate over all enabled and valid input animations, and sample their frames.
		if (!SampleFrames(Settings, *Skeleton))
		{
			UE_LOGF(LogMLDeformer, Warning, "Failed to sample frames, user probably cancelled.");
			return false;
		}

		// Perform pose remixing, if desired.
		// This basically randomizes keyframes for groups of bones, as defined in the bone groups that we set up.
		const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
		if (Settings.bRemixPoses && !Settings.BoneGroups.Groups.IsEmpty())
		{
			if (!RemixPoses(Settings, RefSkeleton))
			{
				UE_LOGF(LogMLDeformer, Warning, "No best frames found, user probably cancelled.");
				return false;
			}
		}

		// Reduce the number of frames, which essentially only keeps the frames that are as different as possible from each other.
		// It is not the optimal solution as it is a greedy algorithm, but it should pick a good set of diverse poses.
		TArray<int32> FramesToInclude;
		if (Settings.bReduceFrames)
		{
			FramesToInclude = FindBestFrameIndices(Settings.NumOutputFrames, RefSkeleton);
			if (FramesToInclude.IsEmpty())
			{
				UE_LOGF(LogMLDeformer, Warning, "No best frames found, user probably cancelled.");
				return false;
			}
		}

		// Now transfer our transforms data into the animation sequence we selected as output anim sequence.
		// If FramesToInclude is empty it will just include all the frames.
		if (!SaveAnimationDataInAnimSequence(Settings, RefSkeleton, FramesToInclude))
		{
			UE_LOGF(LogMLDeformer, Warning, "Failed to save results of the training data processor into the output animation sequence.");
			return false;
		}

		return true;
	}

	bool FTrainingDataProcessor::SampleFrames(const UMLDeformerTrainingDataProcessorSettings& Settings, const USkeleton& Skeleton)
	{
		// Make sure the animations are loaded.
		int32 TotalNumFrames = 0;
		{
			FScopedSlowTask Task(Settings.AnimList.Num(), LOCTEXT("AnimPreloadText", "Loading animations"));
			Task.MakeDialog();
			for (const FMLDeformerTrainingDataProcessorAnim& Anim : Settings.AnimList)
			{
				if (Anim.bEnabled)
				{
					UAnimSequence* AnimSequence = Anim.AnimSequence.LoadSynchronous();
					if (AnimSequence)
					{
						if (AnimSequence->IsCompressedDataOutOfDate())
						{
							AnimSequence->WaitOnExistingCompression(true);
						}

						TotalNumFrames += AnimSequence->GetDataModel()->GetNumberOfFrames();
					}
				}
				Task.EnterProgressFrame();
			}
		}

		// Build a list of existing bone names.
		AnimFrameData.BoneNames.Reset();
		AnimFrameData.BoneIndices.Reset();
		const FReferenceSkeleton& RefSkeleton = Skeleton.GetReferenceSkeleton();
		for (const FName BoneName : Settings.BoneList.BoneNames)
		{
			const int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);
			if (BoneIndex != INDEX_NONE)
			{
				AnimFrameData.BoneNames.Add(BoneName);
				AnimFrameData.BoneIndices.Add(BoneIndex);
			}
		}

		// Allocate space for the frames.
		AnimFrameData.NumRefSkelBones = RefSkeleton.GetNum();
		const int32 NumFrameElements = AnimFrameData.NumRefSkelBones * TotalNumFrames;
		AnimFrameData.Transforms.SetNum(NumFrameElements);
		check(GetNumFrames() == TotalNumFrames);

		// Sample the ref pose and store it.
		const TArray<FTransform>& RefBonePose = RefSkeleton.GetRefBonePose();
		AnimFrameData.RefPoseTransforms.SetNum(AnimFrameData.NumRefSkelBones);
		for (int32 Index = 0; Index < AnimFrameData.NumRefSkelBones; ++Index)
		{
			AnimFrameData.RefPoseTransforms[Index] = FTransform3f(RefBonePose[Index]);
		}

		// Sample all the frames.
		{
			FMemMark Mark(FMemStack::Get());
			FScopedSlowTask Task(TotalNumFrames, LOCTEXT("AnimSamplingText", "Sampling animations"));
			Task.MakeDialogDelayed(1.0f, true, false);

			TArray<FBoneIndexType> RequiredBoneIndexArray;
			RequiredBoneIndexArray.AddUninitialized(RefSkeleton.GetNum());
			for (int32 BoneIndex = 0; BoneIndex < RequiredBoneIndexArray.Num(); ++BoneIndex)
			{
				RequiredBoneIndexArray[BoneIndex] = BoneIndex;
			}

			int32 AnimSeqFrameOffset = 0;
			for (const FMLDeformerTrainingDataProcessorAnim& Anim : Settings.AnimList)
			{
				if (!Anim.bEnabled)
				{
					continue;
				}

				const UAnimSequence* AnimSequence = Anim.AnimSequence.Get();
				if (!AnimSequence)
				{
					continue;
				}

				const IAnimationDataModel* AnimDataModel = AnimSequence->GetDataModel();
				const int32 NumFramesInAnimSequence = AnimDataModel->GetNumberOfFrames();
				const FFrameRate FrameRate = AnimSequence->GetSamplingFrameRate();

				FBoneContainer RequiredBones;
				RequiredBones.InitializeTo(RequiredBoneIndexArray, UE::Anim::ECurveFilterMode::DisallowAll, Skeleton);
				RequiredBones.SetUseRAWData(false);

				// Sample all frames of this animation sequence.
				ParallelFor(NumFramesInAnimSequence, [&](int32 FrameNumber)
				{
					const double SampleTime = FrameNumber / FrameRate.AsDecimal();

					FCompactPose Pose;
					Pose.SetBoneContainer(&RequiredBones);
					Pose.ResetToRefPose(RequiredBones);

					FBlendedCurve TempCurve;
					UE::Anim::FStackAttributeContainer TempAttributes;

					FAnimExtractContext ExtractionContext(SampleTime);
					ExtractionContext.bExtractWithRootMotionProvider = false;
					FAnimationPoseData AnimPoseData(Pose, TempCurve, TempAttributes);
					AnimSequence->GetAnimationPose(AnimPoseData, ExtractionContext);

					const FCompactPose& CompactPose = AnimPoseData.GetPose();
					const TArray<FTransform, TMemStackAllocator<>>& PoseBoneTransforms = CompactPose.GetBones();
					check(PoseBoneTransforms.Num() == RefSkeleton.GetNum());

					// Store the transforms for the entire reference skeleton.
					TArrayView<FTransform3f> FrameTransforms = GetFrameTransforms(FrameNumber + AnimSeqFrameOffset);
					FTransform BoneTransform;
					for (int32 Index = 0; Index < AnimFrameData.NumRefSkelBones; ++Index)
					{
						const FCompactPoseBoneIndex CompactIndex = RequiredBones.GetCompactPoseIndexFromSkeletonIndex(Index);
						if (CompactIndex != INDEX_NONE)
						{
							const FTransform& LocalTransform = PoseBoneTransforms[CompactIndex.GetInt()];
							BoneTransform = LocalTransform;
						}
						else
						{
							BoneTransform = RefBonePose[Index];
						}

						// Store the transform in the frame data.
						FrameTransforms[Index] = FTransform3f(BoneTransform);
					}
				}); // For all frames in this anim sequence.

				Task.EnterProgressFrame();
				if (Task.ShouldCancel())
				{
					Clear();
					return false;
				}

				AnimSeqFrameOffset += NumFramesInAnimSequence;
			} // For all animations.

			const SIZE_T NumBytes = AnimFrameData.Transforms.NumBytes();
			UE_LOGF(LogMLDeformer, Display, "Sampled frame data: %zu Bytes (%.2f MB)", NumBytes, NumBytes/(double)(1000*1000));
		}

		return true;
	}

	FTransform3f FTrainingDataProcessor::GetRefPoseTransform(const FReferenceSkeleton& RefSkeleton, FName BoneName)
	{
		const int32 BoneIndex = RefSkeleton.GetRawRefBoneNames().Find(BoneName);
		if (BoneIndex != INDEX_NONE)
		{
			return FTransform3f(RefSkeleton.GetRefBonePose()[BoneIndex]);
		}

		return FTransform3f::Identity;
	}

	void FTrainingDataProcessor::Clear()
	{
		AnimFrameData.Transforms.Empty();
		AnimFrameData.BoneNames.Empty();
		AnimFrameData.BoneIndices.Empty();
		AnimFrameData.RefPoseTransforms.Empty();
		AnimFrameData.NumRefSkelBones = 0;
	}

	int32 FTrainingDataProcessor::GetNumFrames() const
	{
		if (AnimFrameData.NumRefSkelBones == 0)
		{
			return 0;
		}
		check(AnimFrameData.Transforms.Num() % AnimFrameData.NumRefSkelBones == 0);
		return AnimFrameData.Transforms.Num() / AnimFrameData.NumRefSkelBones;
	}

	TArrayView<FTransform3f> FTrainingDataProcessor::GetFrameTransforms(int32 FrameIndex)
	{
		const int32 NumBones = AnimFrameData.NumRefSkelBones;
		check(FrameIndex < GetNumFrames());
		check(NumBones > 0);
		return TArrayView<FTransform3f>(&AnimFrameData.Transforms[NumBones * FrameIndex], NumBones);
	}

	TConstArrayView<FTransform3f> FTrainingDataProcessor::GetFrameTransforms(const TArray<FTransform3f>& Transforms, int32 FrameIndex) const
	{
		const int32 NumBones = AnimFrameData.NumRefSkelBones;
		check(Transforms.Num() == AnimFrameData.Transforms.Num());
		check(FrameIndex < GetNumFrames());
		check(NumBones > 0);
		return TConstArrayView<FTransform3f>(&Transforms[NumBones * FrameIndex], NumBones);
	}

	TConstArrayView<FTransform3f> FTrainingDataProcessor::GetFrameTransforms(int32 FrameIndex) const
	{
		const int32 NumBones = AnimFrameData.NumRefSkelBones;
		check(FrameIndex < GetNumFrames());
		check(NumBones > 0);
		return TConstArrayView<FTransform3f>(&AnimFrameData.Transforms[NumBones * FrameIndex], NumBones);
	}

	TConstArrayView<FTransform3f> FTrainingDataProcessor::GetRefPoseTransforms() const
	{
		const int32 NumBones = AnimFrameData.NumRefSkelBones;
		check(NumBones > 0);
		return TConstArrayView<FTransform3f>(&AnimFrameData.Transforms[0], NumBones);
	}

	double FTrainingDataProcessor::CalculateMeanError(const TConstArrayView<FTransform3f> PoseA, const TConstArrayView<FTransform3f> PoseB) const
	{
		check(PoseA.Num() == PoseB.Num());
		check(PoseA.Num() == AnimFrameData.NumRefSkelBones);

		double Sum = 0.0;
		for (int32 Index = 0; Index < AnimFrameData.BoneIndices.Num(); ++Index)
		{
			const int32 BoneIndex = AnimFrameData.BoneIndices[Index];
			const FQuat RotA = FQuat(PoseA[BoneIndex].GetRotation());
			const FQuat RotB = FQuat(PoseB[BoneIndex].GetRotation());
			Sum += FMath::Square(RotA.X - RotB.X);
			Sum += FMath::Square(RotA.Y - RotB.Y);
			Sum += FMath::Square(RotA.Z - RotB.Z);
			Sum += FMath::Square(RotA.W - RotB.W);
		}

		const double Mean = Sum / static_cast<double>(AnimFrameData.BoneIndices.Num() * 4); // Times 4 because of the 4 quat values.
		return Mean;
	}

	double FTrainingDataProcessor::CalculateMeanError(int32 PreviousPoseIndex, int32 CurrentPoseIndex) const
	{
		if (PreviousPoseIndex != -1)
		{
			return CalculateMeanError(GetFrameTransforms(PreviousPoseIndex), GetFrameTransforms(CurrentPoseIndex));
		}

		return CalculateMeanError(GetRefPoseTransforms(), GetFrameTransforms(CurrentPoseIndex));
	}

	// Find the best MaxNumFrames number of frames.
	// With best frames we mean frames that are most far apart from each other.
	TArray<int32> FTrainingDataProcessor::FindBestFrameIndices(int32 MaxNumFrames, const FReferenceSkeleton& RefSkeleton) const
	{
		TArray<int32> BestFrames;

		// Now find the best frames.
		const int32 NumSampledFrames = GetNumFrames();
		const int32 NumFramesToFind = FMath::Min(MaxNumFrames, NumSampledFrames);
		BestFrames.Reserve(NumFramesToFind);
		int32 PreviousBestFrameIndex = -1; // The ref pose.

		TArray<double> FrameMeanErrors; // One error per frame.
		FrameMeanErrors.SetNum(NumSampledFrames);

		TArray<int32> FramesToCheck;
		FramesToCheck.SetNum(NumSampledFrames);
		for (int32 FrameIndex = 0; FrameIndex < NumSampledFrames; FrameIndex++)
		{
			FramesToCheck[FrameIndex] = FrameIndex;
		}

		FScopedSlowTask Task(NumFramesToFind, LOCTEXT("FrameReductionText", "Reducing the number of frames"));
		Task.MakeDialogDelayed(1.0f, true);

		for (int32 FrameToFindIndex = 0; FrameToFindIndex < NumFramesToFind; ++FrameToFindIndex)
		{
			// Calculate the mean errors to the previous best pose.
			check(FramesToCheck.Num() == FrameMeanErrors.Num())
			ParallelFor(TEXT("MLDeformer::TrainingDataProcessorAlgo::FindBestFrameIndices"), FramesToCheck.Num(), 5, [&](int32 Index)
			{
				const int32 FrameIndex = FramesToCheck[Index];
				FrameMeanErrors[Index] = CalculateMeanError(PreviousBestFrameIndex, FrameIndex);
			});

			// Find the frame with the highest error, which indicates it diverges most from the previous pose.
			const double* MaxDistancePtr = Algo::MaxElement(FrameMeanErrors);
			const int32 MaxErrorFrameIndex = MaxDistancePtr - FrameMeanErrors.GetData();
			const int32 BestSampledFrameIndex = FramesToCheck[MaxErrorFrameIndex];

			// Store this frame as best frame.
			BestFrames.Add(BestSampledFrameIndex);
			FramesToCheck.RemoveAt(MaxErrorFrameIndex);
			FrameMeanErrors.Pop(); // We can just pop here instead of RemoveAt, as we reinit all values later, and Pop is more efficient.
			PreviousBestFrameIndex = BestSampledFrameIndex;

			Task.EnterProgressFrame();
			if (Task.ShouldCancel())
			{
				BestFrames.Empty();
				break;
			}
		}

		return MoveTemp(BestFrames);
	}

	namespace
	{
		TArray<int32> GenerateShuffledFrameArray(int32 NumFrames, const FRandomStream& RandomStream)
		{
			// First initialize the list to the ordered list of frame numbers.
			TArray<int32> ShuffledFrames;
			ShuffledFrames.SetNum(NumFrames);
			for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
			{
				ShuffledFrames[FrameIndex] = FrameIndex;
			}

			// Perform the shuffle.
			for (int32 Index = NumFrames - 1; Index > 0; --Index)
			{
				const int32 RandomFrameIndex = RandomStream.RandRange(0, Index);
				ShuffledFrames.Swap(Index, RandomFrameIndex);
			}

			return MoveTemp(ShuffledFrames);
		}
	}

	bool FTrainingDataProcessor::RemixPoses(const UMLDeformerTrainingDataProcessorSettings& Settings,
	                                            const FReferenceSkeleton& RefSkeleton)
	{
		// We need a backup of our frame data for this to work correctly.
		const TArray<FTransform3f> OriginalTransforms = AnimFrameData.Transforms;

		const FRandomStream RandomStream(Settings.RandomSeed);
		for (int32 BoneGroupIndex = 0; BoneGroupIndex < Settings.BoneGroups.Groups.Num(); ++BoneGroupIndex)
		{
			const FMLDeformerTrainingDataProcessorBoneGroup& BoneGroup = Settings.BoneGroups.Groups[BoneGroupIndex];

			// Generate a shuffled frame order for this group. This just reorders the frame numbers.
			// For example frames [0, 1, 2, 3, 4] could be shuffled to turn into [2, 4, 0, 1, 3].
			const TArray<int32> ShuffledFrames = GenerateShuffledFrameArray(GetNumFrames(), RandomStream);

			// For every frame.
			for (int32 Index = 0; Index < ShuffledFrames.Num(); ++Index)
			{
				// Get the shuffled and current frame transform buffers.
				// With shuffled we mean the transform of another frame number.
				const int32 ShuffledFrameIndex = ShuffledFrames[Index];
				const TConstArrayView<FTransform3f> ShuffledFrameTransforms = GetFrameTransforms(OriginalTransforms, ShuffledFrameIndex);
				TArrayView<FTransform3f> FrameTransforms = GetFrameTransforms(Index);

				// Replace the transform for each bone in this bone group with the shuffled version.
				for (const FName BoneName : BoneGroup.BoneNames)
				{
					const int32 RefSkelBoneIndex = RefSkeleton.FindBoneIndex(BoneName);
					if (RefSkelBoneIndex != INDEX_NONE)
					{
						FrameTransforms[RefSkelBoneIndex] = ShuffledFrameTransforms[RefSkelBoneIndex];
					}
				}
			}
		}

		return true;
	}

	bool FTrainingDataProcessor::SaveAnimationDataInAnimSequence(const UMLDeformerTrainingDataProcessorSettings& Settings,
	                                                                 const FReferenceSkeleton& RefSkeleton,
	                                                                 const TArray<int32>& FramesToInclude)
	{
		UAnimSequence* OutputSequence = Settings.OutputAnimSequence.LoadSynchronous();
		if (!OutputSequence)
		{
			return false;
		}

		(void)OutputSequence->MarkPackageDirty();

		check(RefSkeleton.GetNum() == AnimFrameData.NumRefSkelBones);
		const int32 NumBones = AnimFrameData.NumRefSkelBones;

		FScopedSlowTask Task(NumBones, LOCTEXT("SavingMessage", "Generating Animation Sequence"));
		Task.MakeDialogDelayed(1.0f, true);

		// Clear the current animation data, so all bone and curve tracks etc.
		IAnimationDataController& Controller = OutputSequence->GetController();
		Controller.InitializeModel();
		OutputSequence->ResetAnimation();
		const int32 NumOutputFrames = FramesToInclude.IsEmpty() ? GetNumFrames() : FramesToInclude.Num();
		Controller.SetNumberOfFrames(NumOutputFrames, false);
		Controller.SetFrameRate(FFrameRate(30, 1), false);

		// Preallocate the pos/rot/scale buffers, as we will decompose the FTransforms.
		TArray<FVector3f> PosKeys;
		TArray<FQuat4f> RotKeys;
		TArray<FVector3f> ScaleKeys;
		PosKeys.SetNumUninitialized(NumOutputFrames, EAllowShrinking::No);
		RotKeys.SetNumUninitialized(NumOutputFrames, EAllowShrinking::No);
		ScaleKeys.SetNumUninitialized(NumOutputFrames, EAllowShrinking::No);

		// For all bones, generate a key track.
		Controller.OpenBracket(LOCTEXT("CreateNewAnimBracket", "Create Anim Sequence"));
		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			const FName BoneName = RefSkeleton.GetBoneName(BoneIndex);

			// Decompose the transforms into arrays for pos/rot/scale.
			for (int32 Index = 0; Index < NumOutputFrames; ++Index)
			{
				const int32 FrameIndex = FramesToInclude.IsEmpty() ? Index : FramesToInclude[Index];
				const TConstArrayView<FTransform3f> FrameTransforms = GetFrameTransforms(FrameIndex);
				const FTransform3f& Transform = FrameTransforms[BoneIndex];

				PosKeys[Index] = AnimFrameData.RefPoseTransforms[BoneIndex].GetTranslation();
				RotKeys[Index] = Transform.GetRotation();
				ScaleKeys[Index] = AnimFrameData.RefPoseTransforms[BoneIndex].GetScale3D();
			}

			Controller.AddBoneCurve(BoneName, false);
			Controller.SetBoneTrackKeys(BoneName, PosKeys, RotKeys, ScaleKeys, false);

			Task.EnterProgressFrame();
		}
		Controller.CloseBracket();

		Controller.NotifyPopulated();
		OutputSequence->RefreshCacheData();

		return true;
	}
} // namespace UE::MLDeformer::TrainingDataProcessor

#undef LOCTEXT_NAMESPACE
