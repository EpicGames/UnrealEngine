// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/TrainingDataProcessor/TrainingDataProcessorSettingsDetailCustomization.h"
#include "ContentBrowserModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IContentBrowserSingleton.h"
#include "MLDeformerGeomCacheModel.h"
#include "MLDeformerTrainingDataProcessorSettings.h"
#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/SkeletalMesh.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/MessageDialog.h"
#include "UObject/SavePackage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "PropertyCustomizationHelpers.h"
#include "SWarningOrErrorBox.h"

#define LOCTEXT_NAMESPACE "TrainingDataProcessorSettingsDetailCustomization"

namespace UE::MLDeformer::TrainingDataProcessor
{
	TSharedRef<IDetailCustomization> FTrainingDataProcessorSettingsDetailCustomization::MakeInstance()
	{
		return MakeShareable(new FTrainingDataProcessorSettingsDetailCustomization());
	}

	namespace
	{
		bool GetFolderFromPackagePath(const FString& PackagePath, FString& OutFolderPath)
		{
			int32 LastSlashIndex = INDEX_NONE;
			if (PackagePath.FindLastChar('/', LastSlashIndex) && LastSlashIndex != INDEX_NONE)
			{
				OutFolderPath = PackagePath.Left(LastSlashIndex);
				return true;
			}

			return false;
		}
	}

	void FTrainingDataProcessorSettingsDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
	{
		IDetailCategoryBuilder& OutputCategory = DetailBuilder.EditCategory("Output");
		IDetailCategoryBuilder& InputAnimCategory = DetailBuilder.EditCategory("Input Animations");
		IDetailCategoryBuilder& FrameReductionCategory = DetailBuilder.EditCategory("Frame Reduction");

		// Get the training data processor settings and the skeleton.
		TArray<TWeakObjectPtr<UObject>> Objects;
		DetailBuilder.GetObjectsBeingCustomized(Objects);
		if (Objects.Num() == 1 && Objects[0] != nullptr)
		{
			TrainingDataProcessorSettings = Cast<UMLDeformerTrainingDataProcessorSettings>(Objects[0].Get());
			check(TrainingDataProcessorSettings.Get());
		}

		// Hide the default output anim sequence property.
		const FName PropertyName = GET_MEMBER_NAME_CHECKED(UMLDeformerTrainingDataProcessorSettings, OutputAnimSequence);
		TSharedPtr<IPropertyHandle> OutputAnimSequencePropertyHandle = DetailBuilder.GetProperty(PropertyName);
		DetailBuilder.HideProperty(OutputAnimSequencePropertyHandle);

		OutputCategory.AddCustomRow(FText::FromString("SkeletonMismatchErrorRow"))
		              .Visibility(TAttribute<EVisibility>::CreateSP(
			              this, &FTrainingDataProcessorSettingsDetailCustomization::GetSkeletonMismatchErrorVisibility))
		              .WholeRowContent()
		[
			SNew(SBox)
			.Padding(FMargin(0.0f, 4.0f))
			[
				SNew(SWarningOrErrorBox)
				.MessageStyle(EMessageStyle::Error)
				.Message(LOCTEXT("SkeletonMismatchError", "The output anim sequence skeleton does not match the model's skeleton."))
			]
		];

		// Create our own version of the property.
		// Insert some Create New button behind the anim sequence property.
		OutputCategory.AddCustomRow(FText::FromName(PropertyName))
		              .NameContent()
			[
				OutputAnimSequencePropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SObjectPropertyEntryBox)
					.PropertyHandle(OutputAnimSequencePropertyHandle)
					.AllowedClass(UAnimSequence::StaticClass())
					.ObjectPath_Lambda([this]()
					{
						return TrainingDataProcessorSettings->OutputAnimSequence.LoadSynchronous()
							       ? TrainingDataProcessorSettings->OutputAnimSequence->GetPathName()
							       : FString();
					})
					.ThumbnailPool(DetailBuilder.GetThumbnailPool())
					.OnShouldFilterAsset(this, &FTrainingDataProcessorSettingsDetailCustomization::FilterAnimSequences)
				]
				+ SHorizontalBox::Slot()
				.Padding(FMargin(8.0f, 2.0f, 2.0f, 2.0f))
				.MaxWidth(200)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.Text(FText::FromString("Create New"))
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.OnClicked(this, &FTrainingDataProcessorSettingsDetailCustomization::OnCreateNewButtonClicked)
				]
			];

		// Show a warning when there are no input frames.
		InputAnimCategory.AddCustomRow(FText::FromString("NoInputAnimsWarning"))
		                 .Visibility(TAttribute<EVisibility>::CreateSP(
			                 this, &FTrainingDataProcessorSettingsDetailCustomization::GetNoFramesWarningVisibility))
		                 .WholeRowContent()
		[
			SNew(SBox)
			.Padding(FMargin(0.0f, 4.0f))
			[
				SNew(SWarningOrErrorBox)
				.MessageStyle(EMessageStyle::Warning)
				.Message(LOCTEXT("NoInputFramesError", "Please add some input animations to sample frames from."))
			]
		];

		// Add the animation list property.
		InputAnimCategory.AddProperty(
			                 GET_MEMBER_NAME_CHECKED(UMLDeformerTrainingDataProcessorSettings, AnimList),
			                 UMLDeformerTrainingDataProcessorSettings::StaticClass())
		                 .DisplayName(FText::Format(LOCTEXT("InputAnimsFrameCountString", "Animation List ({0} Frames)"), GetTotalNumInputFrames()));

		// Get the property handle for the animation inputs, and make it listen to changes.
		TSharedPtr<IPropertyHandle> InputAnimsArrayPropertyHandle = DetailBuilder.GetProperty(
			GET_MEMBER_NAME_CHECKED(UMLDeformerTrainingDataProcessorSettings, AnimList));
		if (InputAnimsArrayPropertyHandle.IsValid())
		{
			InputAnimsArrayPropertyHandle->SetOnPropertyValueChanged(
				FSimpleDelegate::CreateStatic(&FTrainingDataProcessorSettingsDetailCustomization::Refresh, &DetailBuilder));
		}

		// Show a warning when there are no input bones while frame reduction is enabled.
		FrameReductionCategory.AddCustomRow(FText::FromString("NoInputBonesWarning"))
		                      .Visibility(TAttribute<EVisibility>::CreateSP(
			                      this, &FTrainingDataProcessorSettingsDetailCustomization::GetNoInputBonesWarningVisibility))
		                      .WholeRowContent()
		[
			SNew(SBox)
			.Padding(FMargin(0.0f, 4.0f))
			[
				SNew(SWarningOrErrorBox)
				.MessageStyle(EMessageStyle::Warning)
				.Message(LOCTEXT("NoInputBonesError", "Please add input bones."))
			]
		];
	}

	EVisibility FTrainingDataProcessorSettingsDetailCustomization::GetNoInputBonesWarningVisibility() const
	{
		if (TrainingDataProcessorSettings.IsValid())
		{
			return (TrainingDataProcessorSettings->bReduceFrames && TrainingDataProcessorSettings->BoneList.BoneNames.IsEmpty())
				       ? EVisibility::Visible
				       : EVisibility::Collapsed;
		}
		return EVisibility::Collapsed;
	}

	EVisibility FTrainingDataProcessorSettingsDetailCustomization::GetNoFramesWarningVisibility() const
	{
		return GetTotalNumInputFrames() > 0 ? EVisibility::Collapsed : EVisibility::Visible;
	}

	int32 FTrainingDataProcessorSettingsDetailCustomization::GetTotalNumInputFrames() const
	{
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		int32 TotalNumFrames = 0;
		if (TrainingDataProcessorSettings.IsValid())
		{
			for (const FMLDeformerTrainingDataProcessorAnim& Anim : TrainingDataProcessorSettings->AnimList)
			{
				if (Anim.bEnabled)
				{
					const FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(Anim.AnimSequence.ToSoftObjectPath());
					if (AssetData.IsValid())
					{
						const FAssetTagValueRef Tag = AssetData.TagsAndValues.FindTag(TEXT("Number Of Frames"));
						if (Tag.IsSet())
						{
							TotalNumFrames += FCString::Atoi(*Tag.GetValue());
						}
					}
				}
			}
		}
		return TotalNumFrames;
	}

	void FTrainingDataProcessorSettingsDetailCustomization::Refresh(IDetailLayoutBuilder* DetailBuilder)
	{
		if (DetailBuilder)
		{
			DetailBuilder->ForceRefreshDetails();
		}
	}

	FString FTrainingDataProcessorSettingsDetailCustomization::FindDefaultAnimSequencePath(const UMLDeformerModel* Model)
	{
		// Try to find a good path to create a new animation in.
		// Do this by checking if we already have training animations set up, if so, use that same path.
		if (Model)
		{
			FString Folder;
			const UMLDeformerGeomCacheModel* GeomCacheModel = Cast<UMLDeformerGeomCacheModel>(Model);
			if (GeomCacheModel)
			{
				// Try to use the training animations first.
				for (const FMLDeformerGeomCacheTrainingInputAnim& Anim : GeomCacheModel->GetTrainingInputAnims())
				{
					if (Anim.GetAnimSequenceSoftObjectPtr().ToSoftObjectPath().IsValid())
					{
						if (GetFolderFromPackagePath(Anim.GetAnimSequenceSoftObjectPtr().GetLongPackageName(), Folder))
						{
							return Folder;
						}
					}
				}
			}

			// Since we didn't find a good training animation path, try the input animations to the training data processor settings.
			const UMLDeformerTrainingDataProcessorSettings* Settings = Model->GetTrainingDataProcessorSettings();
			if (Settings)
			{
				for (const FMLDeformerTrainingDataProcessorAnim& Anim : Settings->AnimList)
				{
					if (GetFolderFromPackagePath(Anim.AnimSequence.GetLongPackageName(), Folder))
					{
						return Folder;
					}
				}
			}

			// If that also failed, use the model path.
			if (GetFolderFromPackagePath(Model->GetPathName(nullptr), Folder))
			{
				return Folder;
			}
		}

		return FString("/Game");
	}

	FReply FTrainingDataProcessorSettingsDetailCustomization::OnCreateNewButtonClicked() const
	{
		// Get the skeleton from our model.
		TStrongObjectPtr<UMLDeformerTrainingDataProcessorSettings> StrongSettings = TrainingDataProcessorSettings.Pin();
		check(StrongSettings);
		USkeleton* Skeleton = nullptr;
		UMLDeformerModel* Model = Cast<UMLDeformerModel>(StrongSettings->GetOuter());
		if (Model)
		{
			Skeleton = Model->GetSkeletalMesh() ? Model->GetSkeletalMesh()->GetSkeleton() : nullptr;
		}

		// We need a skeleton.
		const FText MessageTitle = LOCTEXT("MessageBoxTitle", "Training Data Processor");
		if (!Skeleton)
		{
			FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok,
			                     LOCTEXT("FailMessage",
			                             "Cannot create new anim sequence as we don't have a skeleton to assign.\nPlease make sure you select a skeletal mesh in your model."),
			                     MessageTitle);
			return FReply::Handled();
		}

		// Create the save asset dialog, don't allow picking existing assets.
		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		FSaveAssetDialogConfig SaveAssetConfig;
		SaveAssetConfig.DialogTitleOverride = LOCTEXT("CreateWindowTitle", "Create New Animation Sequence Asset");
		SaveAssetConfig.DefaultPath = FindDefaultAnimSequencePath(Model);
		SaveAssetConfig.DefaultAssetName = TEXT("NewAnimSequence");
		SaveAssetConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Type::Disallow;
		const FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetConfig);
		if (SaveObjectPath.IsEmpty())
		{
			UE_LOGF(LogMLDeformer, Warning, "User canceled the save dialog.");
			return FReply::Handled();
		}

		// Create the package.
		FString PackageName;
		FString AssetName;
		SaveObjectPath.Split(TEXT("."), &PackageName, &AssetName);
		UPackage* Package = CreatePackage(*PackageName);
		if (!Package)
		{
			FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok, LOCTEXT("CreatePackageFailMessage", "Failed to create the package."),
			                     MessageTitle);
			return FReply::Handled();
		}
		Package->FullyLoad();

		// Create the animation asset.
		UAnimSequence* NewAnimSequence = NewObject<UAnimSequence>(Package, FName(*AssetName), RF_Public | RF_Standalone);
		if (NewAnimSequence)
		{
			// Set up the animation sequence.
			NewAnimSequence->SetSkeleton(Skeleton);
			NewAnimSequence->SetPreviewMesh(Model->GetSkeletalMesh());
			NewAnimSequence->GetController().InitializeModel();
			// Needed, as otherwise there will be issues and crashes in the anim data model validation code.
			NewAnimSequence->RefreshCacheData();
			(void)Package->MarkPackageDirty();
			FAssetRegistryModule::AssetCreated(NewAnimSequence);

			// Update the animation sequence inside our training data processor settings UObject to point to the newly created one.
			// That will also refresh the UI to show our newly created anim sequence when returning from this dialog.
			StrongSettings->OutputAnimSequence = NewAnimSequence;
			UE_LOGF(LogMLDeformer, Display, "Successfully created AnimSequence asset at: %ls", *SaveObjectPath);
		}
		else
		{
			FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok,
			                     LOCTEXT("AnimCreationErrorMessage", "Failed to create new Anim Sequence. Please check the log for more details."),
			                     MessageTitle);
			UE_LOGF(LogMLDeformer, Error, "Failed to create the anim instance asset at: %ls", *SaveObjectPath);
		}

		return FReply::Handled();
	}

	bool FTrainingDataProcessorSettingsDetailCustomization::FilterAnimSequences(const FAssetData& AssetData) const
	{
		const USkeleton* Skeleton = TrainingDataProcessorSettings.IsValid() ? TrainingDataProcessorSettings->FindSkeleton() : nullptr;
		if (Skeleton && Skeleton->IsCompatibleForEditor(AssetData))
		{
			return false;
		}

		return true;
	}

	EVisibility FTrainingDataProcessorSettingsDetailCustomization::GetSkeletonMismatchErrorVisibility() const
	{
		// Get the skeleton from our model.
		if (TrainingDataProcessorSettings.IsValid())
		{
			const USkeleton* ModelSkeleton = TrainingDataProcessorSettings->FindSkeleton();

			// Get the skeleton used by the output animation.
			USkeleton* OutputAnimSkeleton = nullptr;
			if (TrainingDataProcessorSettings->OutputAnimSequence.IsValid())
			{
				if (TrainingDataProcessorSettings->OutputAnimSequence.LoadSynchronous())
				{
					OutputAnimSkeleton = TrainingDataProcessorSettings->OutputAnimSequence->GetSkeleton();
				}
			}

			// If both skeletons are known and they mismatch, show the error.
			if (ModelSkeleton && OutputAnimSkeleton && ModelSkeleton != OutputAnimSkeleton)
			{
				return EVisibility::Visible;
			}
		}

		return EVisibility::Collapsed;
	}
} // namespace UE::MLDeformer::TrainingDataProcessor

#undef LOCTEXT_NAMESPACE
