// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/TrainingDataProcessor/TrainingDataProcessorTool.h"
#include "Tools/TrainingDataProcessor/STrainingDataProcessorMainWidget.h"
#include "MLDeformerEditorToolkit.h"
#include "MLDeformerTrainingDataProcessorSettings.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "IDocumentation.h"
#include "Widgets/Docking/SDockTab.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/Skeleton.h"
#include "PropertyHandle.h"

#define LOCTEXT_NAMESPACE "TrainingDataProcessorTool"

namespace UE::MLDeformer::TrainingDataProcessor
{
	static FName TrainingDataProcessorToolName = FName("Training Data Processor");
	static FText TrainingDataProcessorToolTip = LOCTEXT("TrainingDataProcessorToolToolTip",
	                                                    "The training data processor tool, which allows us to generate training data from a set of animations.");

	// The tab summoner for the tool.
	class FTrainingDataProcessorToolTabSummoner
		: public FWorkflowTabFactory
	{
	public:
		FTrainingDataProcessorToolTabSummoner(const TSharedRef<FMLDeformerEditorToolkit>& InEditor)
			: FWorkflowTabFactory(TrainingDataProcessorToolName, InEditor)
		{
			bIsSingleton = true;
			TabRole = ETabRole::NomadTab;
			TabLabel = LOCTEXT("TabLabel", "Training Data");
			ViewMenuDescription = TrainingDataProcessorToolTip;
			ViewMenuTooltip = TrainingDataProcessorToolTip;
		}

		virtual ~FTrainingDataProcessorToolTabSummoner() override = default;

		virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override
		{
			TSharedPtr<FMLDeformerEditorToolkit> Toolkit = StaticCastSharedPtr<FMLDeformerEditorToolkit>(HostingApp.Pin());
			if (!Toolkit.IsValid())
			{
				return SNullWidget::NullWidget;
			}

			TSharedPtr<FMLDeformerEditorModel> EditorModel = Toolkit->GetActiveModelPointer().Pin();
			if (!EditorModel.IsValid())
			{
				return SNullWidget::NullWidget;
			}

			// Create the training data processor settings if it doesn't exist yet.
			UMLDeformerModel* Model = EditorModel->GetModel();
			if (!Model->GetTrainingDataProcessorSettings())
			{
				UMLDeformerTrainingDataProcessorSettings* TrainingDataProcessorSettings = NewObject<UMLDeformerTrainingDataProcessorSettings>(
					Model, NAME_None, RF_Transactional);
				Model->SetTrainingDataProcessorSettings(TrainingDataProcessorSettings);
			}

			return SNew(STrainingDataProcessorMainWidget)
				.Model(Model);
		}

		virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override
		{
			return IDocumentation::Get()->CreateToolTip(
				TrainingDataProcessorToolTip,
				nullptr,
				TEXT("Shared/Editors/Persona"),
				"TrainingDataProcessorTool_Window");
		}
	};


	// The tool menu extender, which extends the tools menu inside the ML Deformer asset editor.
	class FMLDeformerTrainingDataProcessorToolsMenuExtender
		: public FToolsMenuExtender
	{
	public:
		virtual FMenuEntryParams GetMenuEntry(FMLDeformerEditorToolkit& Toolkit) const override
		{
			FMenuEntryParams Params;
			Params.DirectActions = FUIAction(
				FExecuteAction::CreateLambda([this, &Toolkit]()
				{
					Toolkit.GetAssociatedTabManager()->TryInvokeTab(TrainingDataProcessorToolName);
				}),
				FCanExecuteAction::CreateLambda([]()
				{
					return true;
				})
			);
			Params.LabelOverride = FText::FromName(TrainingDataProcessorToolName);
			Params.ToolTipOverride = TrainingDataProcessorToolTip;
			return Params;
		}

		virtual TSharedPtr<FWorkflowTabFactory> GetTabSummoner(const TSharedRef<FMLDeformerEditorToolkit>& Toolkit) const override
		{
			return MakeShared<FTrainingDataProcessorToolTabSummoner>(Toolkit);
		}
	};

	// Register to the ML Deformer asset editor tools menu.
	void RegisterTool()
	{
		FMLDeformerEditorToolkit::AddToolsMenuExtender(MakeUnique<FMLDeformerTrainingDataProcessorToolsMenuExtender>());
	}

	USkeleton* FindSkeletonForProperty(const TSharedPtr<IPropertyHandle>& PropertyHandle)
	{
		if (!PropertyHandle.IsValid())
		{
			return nullptr;
		}

		TArray<UObject*> Objects;
		PropertyHandle->GetOuterObjects(Objects);

		auto FindSkeletonForObject = [&PropertyHandle](UObject* InObject) -> USkeleton* {
			for (; InObject; InObject = InObject->GetOuter())
			{
				if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(InObject))
				{
					return SkeletalMesh->GetSkeleton();
				}

				if (const UAnimationAsset* AnimationAsset = Cast<UAnimationAsset>(InObject))
				{
					if (AnimationAsset->IsAsset())
					{
						return AnimationAsset->GetSkeleton();
					}
				}

				if (IBoneReferenceSkeletonProvider* SkeletonProvider = Cast<IBoneReferenceSkeletonProvider>(InObject))
				{
					bool bInvalidSkeletonIsError = false;
					return SkeletonProvider->GetSkeleton(bInvalidSkeletonIsError, PropertyHandle.Get());
				}
			}

			return nullptr;
		};

		for (UObject* Object : Objects)
		{
			if (USkeleton* Skeleton = FindSkeletonForObject(Object))
			{
				return Skeleton;
			}
		}

		return nullptr;
	}
} // namespace UE::MLDeformer::TrainingDataProcessor

#undef LOCTEXT_NAMESPACE
