// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class FWorkflowCentricApplication;
class IPersonaPreviewScene;

namespace UE::MLDeformer
{
	class FMLDeformerEditorToolkit;

	/**
	 * The application mode for the ML Deformer editor.
	 * This defines the layout of the UI. It basically spawns all the tabs, such as the viewport, details panels, etc.
	 */
	class FMLDeformerApplicationMode
		: public FApplicationMode
	{
	public:
		/** The name of this mode. */
		static UE_API FName ModeName;

		UE_API FMLDeformerApplicationMode(TSharedRef<FWorkflowCentricApplication> InHostingApp, TSharedRef<IPersonaPreviewScene> InPreviewScene);

		// FApplicationMode overrides.
		UE_API virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
		// ~END FApplicationMode overrides.

	protected:
		/** The hosting app. */
		TWeakPtr<FMLDeformerEditorToolkit> EditorToolkit = nullptr;

		/** The tab factories we support. */
		FWorkflowAllowedTabSet TabFactories;
	};

}	// namespace UE::MLDeformer

#undef UE_API
