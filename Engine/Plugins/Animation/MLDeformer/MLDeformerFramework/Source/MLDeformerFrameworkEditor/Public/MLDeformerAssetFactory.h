// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Factories/Factory.h"
#include "MLDeformerAssetFactory.generated.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

/**
 * The factory for the ML Deformer asset type.
 * This basically integrates the new asset type into the editor, so you can right click and create a new ML Deformer asset.
 */
UCLASS(MinimalAPI, hidecategories=Object)
class UMLDeformerFactory
	: public UFactory
{
	GENERATED_BODY()

public:
	UE_API UMLDeformerFactory();

	// UFactory overrides.
	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual uint32 GetMenuCategories() const override;
	UE_API virtual FText GetToolTip() const override;
	UE_API virtual FString GetDefaultNewAssetName() const override;
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	UE_API virtual bool ConfigureProperties() override;
	UE_API virtual bool ShouldShowInNewMenu() const override;
	UE_API virtual const TArray<FText>& GetMenuCategorySubMenus() const override;
	// ~END UFactory overrides.
};

#undef UE_API
