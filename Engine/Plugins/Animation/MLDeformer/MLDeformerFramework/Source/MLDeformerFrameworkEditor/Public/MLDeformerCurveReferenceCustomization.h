// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"
#include "DetailWidgetRow.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class USkeletalMesh;

namespace UE::MLDeformer
{
	/**
	 * The curve reference property detail customization.
	 */
	class FMLDeformerCurveReferenceCustomization
		: public IPropertyTypeCustomization
	{
	public:
		static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance();

		// IPropertyTypeCustomization overrides.
		UE_API virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
		virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}
		// ~END IPropertyTypeCustomization overrides.

	protected:
		UE_API void SetSkeleton(TSharedRef<IPropertyHandle> StructPropertyHandle);
		UE_API virtual void SetPropertyHandle(TSharedRef<IPropertyHandle> StructPropertyHandle);
		UE_API TSharedPtr<IPropertyHandle> FindStructMemberProperty(TSharedRef<IPropertyHandle> PropertyHandle, const FName& PropertyName);

		/** Property to change after curve has been picked. */
		TSharedPtr<IPropertyHandle> CurveNameProperty = nullptr;

		/** The skeletal mesh we get the curves from. */
		TObjectPtr<USkeletalMesh> SkeletalMesh = nullptr;

	private:
		UE_API virtual void OnCurveSelectionChanged(const FString& Name);
		UE_API virtual FString OnGetSelectedCurve() const;
		UE_API virtual USkeletalMesh* OnGetSkeletalMesh() const;
	};
}	// namespace UE::MLDeformer

#undef UE_API
