// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "GameFramework/Actor.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class UDebugSkelMeshComponent;
class UTextRenderComponent;
class UMLDeformerComponent;

namespace UE::MLDeformer
{
	/**
	 * The default ID's for editor actors.
	 * You can assign any ID to the editor actors, although some code in the editor model classes will assume
	 * that some of these exist. You can always overload certain methods to change that behavior though.
	 * @see FMLDeformerEditorActor
	 */
	enum : int32
	{
		/** The linear skinned (base) actor in training mode. */
		ActorID_Train_Base,

		/** The ground truth (target) actor in training mode. This is our training target that we try to approximate. */
		ActorID_Train_GroundTruth,

		/** The linear skinned (base) actor, in test mode. */
		ActorID_Test_Base,

		/** The ML Deformed actor, in test mode. This one should have an UMLDeformerComponent on it. */
		ActorID_Test_MLDeformed,

		/** The ground truth test actor, in test mode. This will represent your test anim sequence ground truth. */
		ActorID_Test_GroundTruth,

		/** The comparison actor, in test mode. This can be used to compare the results to another pretrained ML Deformer model. */
		ActorID_Test_Compare
	};

	/**
	 * An actor in the ML Deformer asset editor viewport.
	 * The UE Actor object itself is created by the FMLDeformerEditorModel class, after which this editor actor class
	 * can create components on it.
	 * It also contains some methods to control visibility and playback, among some other helper methods.
	 */
	class FMLDeformerEditorActor
	{
	public:
		/**
		 * The construction settings, which can be used as info when creating new components.
		 */
		struct FConstructSettings
		{
			/** The UE actor object that we should add components to. */
			AActor* Actor = nullptr;

			/** The ID of the actor. Look at the UE::MLDeformer::ActorID_Train_Base etc for example. */
			int32 TypeID = -1;

			/** 
			 * The instance index. There can for example be multiple editor actors with a TypeID of ActorID_Test_Compare. 
			 * This index separates those. If you have three test compare editor actors, on will have index 0, another 1, and the third one will have a value of 2 for this member.
			 */
			int32 ActorTypeInstanceIndex = 0;

			/** The color of the label text. */
			FLinearColor LabelColor = FLinearColor(1.0f, 0.0f, 0.0f);

			/** The label text, which should be something like "Linear Skinned" or "Ground Truth", basically describing the actor. This will appear inside the render viewport. */
			FText LabelText;

			/** Is this an actor used during training? Set to false if it is a testing mode actor. */
			bool bIsTrainingActor = false;
		};

		UE_API FMLDeformerEditorActor(const FConstructSettings& Settings);
		UE_API virtual ~FMLDeformerEditorActor();

		// Main methods you can override.
		UE_API virtual void SetVisibility(bool bIsVisible);
		UE_API virtual bool IsVisible() const;
		UE_API virtual void SetPlayPosition(float TimeInSeconds, bool bAutoPause = true);
		UE_API virtual float GetPlayPosition() const;
		UE_API virtual void SetPlaySpeed(float PlaySpeed);
		UE_API virtual float GetPlaySpeed();

		UE_API virtual FBox GetBoundingBox() const;
		UE_API virtual void Pause(bool bPaused);
		UE_API virtual bool IsPlaying() const;
		UE_API virtual bool IsGroundTruthActor() const;
		UE_API virtual bool HasVisualMesh() const;

		AActor* GetActor() const													{ return Actor; }
		int32 GetTypeID() const														{ return TypeID; }
		int32 GetActorTypeInstanceIndex() const										{ return ActorTypeInstanceIndex; }
		UDebugSkelMeshComponent* GetSkeletalMeshComponent() const					{ return SkeletalMeshComponent.Get(); }
		UTextRenderComponent* GetLabelComponent() const								{ return LabelComponent.Get(); }
		UMLDeformerComponent* GetMLDeformerComponent() const						{ return MLDeformerComponent.Get(); }
		bool IsTrainingActor() const												{ return bIsTrainingActor; }
		bool IsTestActor() const													{ return !bIsTrainingActor; }
		float GetMeshOffsetFactor() const											{ return MeshOffsetFactor; }

		void SetSkeletalMeshComponent(UDebugSkelMeshComponent* SkelMeshComponent)	{ SkeletalMeshComponent = SkelMeshComponent; }
		void SetMLDeformerComponent(UMLDeformerComponent* Component)				{ MLDeformerComponent = Component; }
		void SetMeshOffsetFactor(float OffsetFactor)								{ MeshOffsetFactor = OffsetFactor; }
		void SetActorTypeInstanceIndex(int32 Index)									{ ActorTypeInstanceIndex = Index; }

	protected:
		UE_API UTextRenderComponent* CreateLabelComponent(AActor* InActor, FLinearColor Color, const FText& Text) const;

	protected:
		/**
		 * The ID of the editor actor type.
		 * This can be used to identify what actor we are dealing with, for example the base actor or ML Deformed one etc.
		 */
		int32 TypeID = -1;

		/** 
 		 * The instance index. There can for example be multiple editor actors with a TypeID of ActorID_Test_Compare. 
		 * This index separates those. If you have three test compare editor actors, on will have index 0, another 1, and the third one will have a value of 2 for this member.
		 */
		int32 ActorTypeInstanceIndex = 0;

		/** The label component, which shows above the actor. */
		TObjectPtr<UTextRenderComponent> LabelComponent = nullptr;

		/** The actual actor pointer. */
		TObjectPtr<AActor> Actor = nullptr;

		/** The skeletal mesh component (can be nullptr). */
		TObjectPtr<UDebugSkelMeshComponent> SkeletalMeshComponent = nullptr;

		/** The ML Deformer component (can be nullptr). */
		TObjectPtr<UMLDeformerComponent> MLDeformerComponent = nullptr;

		/** The position offset factor of the actor. A value of 1.0 would offset the actor with the Mesh Spacing amount, a value of 2.0 would be two times the mesh spacing offset, etc. */
		float MeshOffsetFactor = 0.0f;

		/** Is this actor used for training? */
		bool bIsTrainingActor = true;
	};
} // namespace UE::MLDeformer

#undef UE_API
