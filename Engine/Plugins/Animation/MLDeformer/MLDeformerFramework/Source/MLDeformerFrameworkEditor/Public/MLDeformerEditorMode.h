// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IPersonaEditMode.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class UMLDeformerVizSettings;

namespace UE::MLDeformer
{
	class FMLDeformerEditorToolkit;
	class FMLDeformerPreviewScene;

	/**
	 * The ML Deformer Persona editor mode.
	 * This handles ticking and rendering of the render viewport and camera focus.
	 */
	class FMLDeformerEditorMode
		: public IPersonaEditMode
	{
	public:
		/** The name of the mode. */
		static UE_API FName ModeName;

		void SetEditorToolkit(FMLDeformerEditorToolkit* InToolkit) { DeformerEditorToolkit = InToolkit; }

		// IPersonaEditMode overrides.
		UE_API virtual bool GetCameraTarget(FSphere& OutTarget) const override;
		UE_API virtual class IPersonaPreviewScene& GetAnimPreviewScene() const override;
		virtual void GetOnScreenDebugInfo(TArray<FText>& OutDebugInfo) const override {}
		// ~END IPersonaEditMode overrides.

		// FEdMode overrides.
		UE_API virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
		UE_API virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
		UE_API virtual void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;
		virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override { return true; }
		virtual bool AllowWidgetMove() override { return false; }
		virtual bool ShouldDrawWidget() const override { return false; }
		virtual bool UsesTransformWidget() const override { return false; }
		virtual bool UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const override { return false; }
		virtual bool RequiresLegacyViewportInteractions() const override { return false; }
		// ~END FEdMode overrides.

	protected:
		FMLDeformerEditorToolkit* DeformerEditorToolkit = nullptr;
	};
}	// namespace UE::MLDeformer

#undef UE_API
