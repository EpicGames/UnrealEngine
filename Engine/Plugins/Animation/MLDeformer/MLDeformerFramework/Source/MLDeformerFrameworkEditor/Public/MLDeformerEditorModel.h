// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/GCObject.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "MLDeformerEditorActor.h"
#include "Misc/FrameTime.h"
#include "MLDeformerVizSettings.h"
#include "MLDeformerModel.h"
#include "MLDeformerSampler.h"
#include "MeshDescription.h"
#include "MLDeformerEditorModel.generated.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class UMLDeformerModel;
class UMLDeformerInputInfo;
class AAnimationEditorPreviewActor;
class IPersonaPreviewScene;
class FEditorViewportClient;
class FSceneView;
class FViewport;
class FPrimitiveDrawInterface;
class UWorld;
class UMeshDeformer;
class USkeletalMesh;
class UAnimSequence;
class UMaterial;
class UGeometryCache;
class UMorphTarget;
class FMorphTargetVertexInfoBuffers;
struct FMLDeformerTrainingInputAnim;
struct FMLDeformerTrainingInputAnimName;

/** Training process return codes. */
UENUM()
enum class ETrainingResult : uint8
{
	/** The training successfully finished. */
	Success = 0,

	/** The user has aborted the training process. */
	Aborted,

	/** The user has aborted the training process, and we can't use the resulting network. */
	AbortedCantUse,

	/** The input or output data to the network has issues, which means we cannot train. */
	FailOnData,

	/** The python script has some error (see output log). Or it's missing some required model training class. Or an actual python code syntax error or failed imports. */
	FailPythonError,

	/** Any other error, for example when some exception got thrown during training. */
	Other
};

namespace UE::MLDeformer
{
	class FMLDeformerEditorToolkit;
	class FMLDeformerSampler;
	class FMLDeformerEditorModel;
	class SMLDeformerInputWidget;

	/**
	 * The base class for the editor side of an UMLDeformerModel.
	 * The editor model class handles most of the editor interactions, such as property changes, triggering the training process,
	 * creation of the viewport actors, and many more things.
	 * In most cases you will want to inherit your own editor model from either FMLDeformerGeomCacheEditorModel or FMLDeformerMorphModelEditorModel though.
	 * These classes will already implement a lot of base functionality in case you work with geometry caches as ground truth data etc.
	 */
	class FMLDeformerEditorModel
		: public TSharedFromThis<FMLDeformerEditorModel>
		, public FGCObject
	{
	public:
		/** 
		 * The editor model initialization settings.
		 * This is used in the Init call.
		 */
		struct InitSettings
		{
			FMLDeformerEditorToolkit* Editor = nullptr;
			UMLDeformerModel* Model = nullptr;
		};

		/**
		 * The destructor, which unregisters the instance of this editor model from the model registry.
		 * It also deletes the editor actors and unregisters from the OnPostEditChangeProperty delegate.
		 */
		UE_API virtual ~FMLDeformerEditorModel() override;

		// FGCObject overrides.
		UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override { return TEXT("FMLDeformerEditorModel"); }
		// ~END FGCObject overrides.

		/**
		 * Copy the common property values from a source model into this model.		 
		 * This includes the skeletal mesh, input animations, and shared visualization settings.
		 * @param SourceEditorModel The source model to copy the shared settings from.
		 */
		UE_API virtual void CopyBaseSettingsFromModel(const FMLDeformerEditorModel* SourceEditorModel);

		/**
		 * Change the skeletal mesh on a specific skeletal mesh component.
		 * This internally will not only call SkelMeshComponent->SetSkeletalMesh(Mesh), but also will reassign the materials.
		 * The reason for this is that if there are material instances set already, the skeletal mesh component will not override them to preserve changes done by the user.
		 * This helper function will force these to be updated as well. This was needed in case of Metahumans.
		 * Metahumans use a PostProcessAnimBP with a ControlRig node in it, which drives material curves.
		 * In order to drive this, material instances are needed instead of regular materials. But as mentioned before, when switching the mesh the 
		 * code internally will not override the material instances. So we simply have to force this manually, hence this function.
		 * @param SkelMeshComponent The skeletal mesh component to change the mesh for.
		 * @param Mesh The new mesh to use.
		 */
		static UE_API void ChangeSkeletalMeshOnComponent(USkeletalMeshComponent* SkelMeshComponent, USkeletalMesh* Mesh);

		/**
		 * Get the number of training frames.
		 * This must not be clamped yet to the maximum training frame limit. It must return the full amount of frames available in your training data.
		 * For example this could return the number of frames inside your geometry cache.
		 * @return The number of frames in the training data.
		 */
		UE_API virtual int32 GetNumTrainingFrames() const;

		/**
		 * Updates the number of available training frames as returned by GetNumTrainingFrames().
		 * This should update the NumTrainingFrames class member with the total number of frames that can be included during training.
		 * So this must be the sum of all frames of all enabled training input animations.
		 * This should not take into account the maximum number of frames we want to train with.
		 */
		UE_API virtual void UpdateNumTrainingFrames();

		/**
		 * Launch the training. This gets executed when the Train button is pressed.
		 * Training can succeed, or be aborted, and there can be training errors. The result is returned by this method.
		 * You generally want to implement a class inherited from MLDeformerTrainingModel, and put a Train function inside of that, which you then execute through
		 * the TrainModel function.
		 * 
		 * A code example:
		 * 
		 * @code{.cpp}
		 * ETrainingResult FYourEditorModel::Train()
		 * {
		 *     return TrainModel<UYourTrainingModel>(this);
		 * }
		 * @endcode
		 * 
		 * That will internally call the Train function of your python class.
		 * Now inside your Python class you do something like:
		 * 
		 * @code{.py}
		 * @unreal.uclass()
		 * class YourModelPythonTrainingModel(unreal.YourTrainingModel):
		 *     @unreal.ufunction(override=True)
		 *     def train(self):
		 *         # ...do training here...
		 *         return 0   # A value of 0 is success, 1 means aborted, see ETrainingResult.
		 * @endcode
		 * 
		 * Calling the TrainModel function inside will then trigger the Python train method to be called.
		 * 
		 * @return The training result.
		 */
		UE_API virtual ETrainingResult Train();

		// Optional overrides.
		/**
		 * Initialize the model.
		 * This will mainly create the delta sampler, editor input info and some other things.
		 * @param Settings The initialization settings.
		 */
		UE_API virtual void Init(const InitSettings& Settings);

		/**
		 * Create the editor actors. These are the actors that will appear in the viewport of the asset editor.
		 * Every actor has some specific ID. The base implementation of certain methods will assume there are specific actors, such as
		 * a linear skinned actor, ML Deformed actor, and a ground truth one. You can find the ID values of those inside the FMLDeformerEditorActor.h file.
		 * The ID values are things like: ActorID_Train_Base, ActorID_Train_GroundTruth, ActorID_Test_Base, ActorID_Test_MLDeformed and ActorID_Test_GroundTruth.
		 * Nothing prevents you from adding more or less actors though.
		 * @param InPersonaPreviewScene The persona scene.
		 */
		UE_API virtual void CreateActors(const TSharedRef<IPersonaPreviewScene>& InPersonaPreviewScene);

		/**
		 * Clears the world, which basically means it removes all the editor actors and engine actors from the editor world.
		 */
		UE_API virtual void ClearWorld();

		/**
		 * Clears the Persona preview scene. Resetting the preview mesh, preview mesh component etc.
		 */
		UE_API virtual void ClearPersonaPreviewScene();

		/**
		 * Clear both the world and preview scene.
		 * This calls ClearWorld and ClearPersonaPreviewScene.
		 */
		UE_API virtual void ClearWorldAndPersonaPreviewScene();

		/**
		 * Create an actor in the editor viewport.
		 * Each actor will have a specific ID and a label that is displayed above it, with a specific color.
		 * The ID values are things like: ActorID_Train_Base, ActorID_Train_GroundTruth, ActorID_Test_Base, ActorID_Test_MLDeformed and ActorID_Test_GroundTruth.
		 * You can find the ID values of those inside the FMLDeformerEditorActor.h file.
		 * @param Settings The creation settings for the editor actor you are creating.
		 * @return A pointer to the editor actor object that was created.
		 */
		UE_API virtual FMLDeformerEditorActor* CreateEditorActor(const FMLDeformerEditorActor::FConstructSettings& Settings) const;

		/**
		 * Create the vertex delta sampler object.
		 * You can create your own sampler in case you use anything else than say a Geometry Cache as ground truth target data.
		 * A geometry cache based editor model would create a new FMLDeformerGeomCacheSampler for example.
		 * @return A pointer to the newly created sampler.
		 */
		UE_API virtual TSharedPtr<FMLDeformerSampler> CreateSamplerObject() const;

		/**
		 * Get the number of training input animations.
		 * Each training input anim is a pair of a source animation and a target mesh deformation with the same poses.
		 * This data is fed into the training process.
		 * @return The number of training input anims.
		 */
		UE_API virtual int32 GetNumTrainingInputAnims() const;

		/**
		 * Get a given training input animation.
		 * Each training input anim is a pair of a source animation and a target mesh deformation with the same poses.
		 * This data is fed into the training process.
		 * @param Index The training input animation to get.
		 * @see GetNumTrainingInputAnims
		 */
		UE_API virtual FMLDeformerTrainingInputAnim* GetTrainingInputAnim(int32 Index) const;

		/**
		 * Update the timeline's training input animation list.
		 * This is the list of animations that show inside the timeline when in training mode.
		 * Each training input animation pair needs some name that is shown inside the timeline.
		 */
		UE_API virtual void UpdateTimelineTrainingAnimList();

		/**
		 * Get the time in seconds, for a given frame in the training data.
		 * @param FrameNumber The frame number to get the time in seconds for.
		 * @return The time offset of this frame, in seconds.
		 */
		UE_API virtual double GetTrainingTimeAtFrame(int32 FrameNumber) const;

		/**
		 * Get the training data frame number for a specific time offset in seconds.
		 * @param TimeInSeconds The time offset, in seconds, to get the frame number for.
		 * @return The frame number for this time offset.
		 */
		UE_API virtual int32 GetTrainingFrameAtTime(double TimeInSeconds) const;

		/**
		 * Get the time in seconds, for a given frame in the test data.
		 * @param FrameNumber The frame number to get the time in seconds for.
		 * @return The time offset of this frame, in seconds.
		 */
		UE_API virtual double GetTestTimeAtFrame(int32 FrameNumber) const;

		/**
		 * Get the test data frame number for a specific time offset in seconds.
		 * @param TimeInSeconds The time offset, in seconds, to get the frame number for.
		 * @return The frame number for this time offset.
		 */
		UE_API virtual int32 GetTestFrameAtTime(double TimeInSeconds) const;

		/**
		 * Get the total number of frames in the test anim sequence.
		 * @return The number of frames.
		 */
		UE_API virtual int32 GetNumTestFrames() const;

		/**
		 * Tick the editor viewport.
		 * @param ViewportClient The editor viewport client object.
		 * @param DeltaTime The delta time of this tick, in seconds.
		 */
		UE_API virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime);

		/**
		 * Create the linear skinned actor that is used as training base.
		 * This is the actor you see marked as "Training Base" in the training mode.
		 * An editor actor needs to be created by this method.
		 * The default implementation will directly set the persona actor and preview mesh and component to this actor.
		 * @param InPersonaPreviewScene The personal preview scene.
		 */
		UE_API virtual void CreateTrainingLinearSkinnedActor(const TSharedRef<IPersonaPreviewScene>& InPersonaPreviewScene);

		/**
		 * Create the linear skinned test actor.
		 * This is the "Linear Skinned" actor you see in testing mode.
		 * An editor actor needs to be created by this method.
		 * @param World The world to create the actor in.
		 */
		UE_API virtual void CreateTestLinearSkinnedActor(UWorld* World);

		/**
		 * Create the test ML Deformed actor.
		 * This is the actor you see in test mode marked as "ML Deformed".
		 * So this actor will have the ML Deformer component added to it as well.
		 * An editor actor needs to be created by this method.
		 * @param World The world to create the actor in.
		 */
		UE_API virtual void CreateTestMLDeformedActor(UWorld* World);

		/**
		 * Create the compare actors for the testing mode.
		 * @param World The world to create the actors in.
		 */
		UE_API virtual void CreateTestCompareActors(UWorld* World);

		/**
		 * Check if this model is compatible with a specific deformer asset.
		 * For example, it will check whether the other deformer asset's skeletal mesh is the same as the one used
		 * by this model.
		 * @param Deformer The deformer asset to check against.
		 * @return Returns true if the specified asset is compatible with this one, otherwise false is returned.
		 */
		UE_API virtual bool IsCompatibleDeformer(UMLDeformerAsset* Deformer) const;

		/**
		 * Update the mesh offset factors, which is basically controlling where to place the actors in the scene.
		 * The spacing between the characters is the mesh offset factor multiplied by the mesh spacing.
		 */
		UE_API virtual void UpdateMeshOffsetFactors();

		/**
		 * Create the training ground truth actor.
		 * This is the training target, so the actor that has the complex deformations, for example using a geometry cache.
		 * An editor actor needs to be created by this method.
		 * @param World The world to create the actor in.
		 */
		virtual void CreateTrainingGroundTruthActor(UWorld* World) {}

		/**
		 * Create the test ground truth actor.
		 * This is the actor typically marked as "Ground Truth" in the editor.
		 * It represents the test animation sequence ground truth deformation. So this could contain a geometry cache
		 * equivalent to the animation sequence.
		 * An editor actor needs to be created by this method.
		 * @param World The world to create the actor in.
		 */
		virtual void CreateTestGroundTruthActor(UWorld* World) {}

		/**
		 * This is triggered when the training frame is changed by the user.
		 * For example when they click the timeline in the editor, or enter a specific frame number.
		 * This happens only in training mode.
		 */
		UE_API virtual void OnTrainingDataFrameChanged();

		/**
		 * Update the transforms of all the editor actors.
		 * This should apply the mesh offsets to space the actors in the world/viewport.
		 */
		UE_API virtual void UpdateActorTransforms();

		/**
		 * Update the visibility status of the editor actors.
		 * This should hide test actors when the editor is in training mode for example, and the other way around.
		 */
		UE_API virtual void UpdateActorVisibility();

		/**
		 * Update the visibility of the labels, as well as their positions.
		 * For example labels of invisible actors should be made invisible as well.
		 */
		UE_API virtual void UpdateLabels();

		/**
		 * This is called when training data inputs change.
		 * This includes changing things such as the Skeletal Mesh, training anim sequence and target mesh.
		 * It will re-initialize the components. For example a skeletal mesh change will update the skeletal mesh component to the new mesh.
		 */
		UE_API virtual void OnInputAssetsChanged();

		/**
		 * This is executed after the input assets have changed.
		 * On default this will refresh the deformer graphs, update the timeline ranges, select the right frames, modify the training button ready state, etc.
		 */
		UE_API virtual void OnPostInputAssetChanged();

		/**
		 * This is called whenever a property value changes in the UI.
		 * The default implementation handles all the shared property changes already.
		 * You can overload this and call the base class method inside it, and then handle your own model specific property changes.
		 * @param PropertyChangedEvent The event information object that contains info about the property change.
		 */
		UE_API virtual void OnPropertyChanged(FPropertyChangedEvent& PropertyChangedEvent);

		/**
		 * This is called before undo is called on the model.
		 */
		virtual void OnPreEditUndo() {}

		/**
		 * This is called after undo is performed on the model.
		 */
		virtual void OnPostEditUndo() {}

		/**
		 * This is called whenever a transaction happened on the model.
		 * @param Event Information about the event.
		 */
		UE_API virtual void OnPostTransacted(const FTransactionObjectEvent& Event);

		/**
		 * Executed when the user presses the play button in the timeline.
		 * On default this will start playing the anim sequence related to whether we are in training or testing mode.
		 */
		UE_API virtual void OnPlayPressed();

		/**
		 * This is called just before we start the training process.
		 * You can use it to back up any specific values that might be modified by the training process.
		 * Sometimes you want to restore specific values when the user aborts the training, while the training already modified those values.
		 */
		virtual void OnPreTraining() {}

		/**
		 * This method is executed after training.
		 * @param TrainingResult The result of the training process, such as whether it was a success or the user aborted, etc.
		 * @param bUsePartiallyTrainedWhenAborted When the user aborts training, they can get offered to use the partially trained neural network. This boolean 
		 *                                        specifies whether they picked "Yes" (true) or "No" (false) to that question.
		 */
		UE_API virtual void OnPostTraining(ETrainingResult TrainingResult, bool bUsePartiallyTrainedWhenAborted);

		/**
		 * This is executed when the training process got aborted by the user.
		 * You can use this method to restore any values you had to back up during OnPreTraining for example.
		 * @param bUsePartiallyTrainedData When the user aborts training, they can get offered to use the partially trained neural network. This boolean 
		 *                                 specifies whether they picked "Yes" (true) or "No" (false) to that question.
		 */
		virtual void OnTrainingAborted(bool bUsePartiallyTrainedData) {}

		/**
		 * Check whether we are playing an animation or not.
		 * This looks at whether we are in training or testing mode. It will look at the correct anim sequence based on this mode.
		 * So if you are in training mode and the testing animation is playing, while the training sequence isn't playing, then it returns false.
		 * It would in that case only return true when the training anim sequence is playing.
		 * The same goes in testing mode, if we are in testing mode it will only return true when the test anim sequence is playing.
		 * @return Returns true whether the mode specific animation sequence is playing.
		 */
		UE_API virtual bool IsPlayingAnim() const;

		/**
		 * Check whether our anim sequence is playing forward or backward.
		 * Just like the IsPlayingAnim method, it is specific to the mode we are in.
		 * @return Returns true when we are playing the animation forwards, and not backward.
		 */
		UE_API virtual bool IsPlayingForward() const;

		/**
		 * Get the current playback position of our training mode.
		 * This will use the target mesh if there is any, and uses the anim sequence play offset in case no target mesh has been set yet.
		 * So if you have a geometry cache or so specified as target mesh, it would extract the current time value from that.
		 * @return The time offset, in seconds, of our training data.
		 */
		UE_API virtual double CalcTrainingTimelinePosition() const;

		/**
		 * Calculate the test mode timeline position.
		 * This will use the ground truth test data if there is any, and uses the test anim sequence play offset in case no test ground truth has been set yet.
		 * So if you have a geometry cache or so specified as test ground truth, it would extract the current time value from that.
		 * @return The time offset, in seconds, of our test data.
		 */
		UE_API virtual double CalcTestTimelinePosition() const;

		/**
		 * This is executed when the timeline has been scrubbed.
		 * Sometimes this is also executed without user interaction, but to update the same data when there was scrubbed.
		 * @param NewScrubTime This is the new time value, in seconds.
		 * @param bIsScrubbing Specifies whether the user is scrubbing or just clicked.
		 */
		UE_API virtual void OnTimeSliderScrubPositionChanged(double NewScrubTime, bool bIsScrubbing);

		/**
		 * Update the playback speed of the test data.
		 * This will iterate over all the editor actors, check whether they are test mode actors, and if so
		 * it will update the play speed. For example a linear skinned mesh will update its skeletal mesh component's play speed, while
		 * a geometry cache ground truth actor would update the play speed set on the geometry cache component.
		 */
		UE_API virtual void UpdateTestAnimPlaySpeed();

		/**
		 * Clamp the current training frame index to be in a valid range.
		 * A valid range would be between 0 and the number of frames in the training data.
		 * So if the training data contains 100 frames, so index 0..99, and we set the TrainingFrameNumber member to 500, this method
		 * will turn that into 99.
		 */
		UE_API virtual void ClampCurrentTrainingFrameIndex();

		/**
		 * Clamp the current test frame index to be in a valid range.
		 * A valid range would be between 0 and the number of frames in the test data (anim sequence / ground truth).
		 * So if the test data contains 100 frames, so index 0..99, and we set the TestingFrameNumber member to 500, this method
		 * will turn that into 99.
		 */
		UE_API virtual void ClampCurrentTestFrameIndex();

		/**
		 * Get the total number of frames used for training, but limited by the maximum number of training frames we have setup.
		 * So if the training data contains 10000 frames, but we set to train on a maximum of 2000 frames, then this will return 2000.
		 * If the number of training frames is 10000 but we set the max number of training frames to 1 million, then it will still return only 10000.
		 * @return The number of frames we should be training on.
		 */
		UE_API virtual int32 GetNumFramesForTraining() const;

		/**
		 * Go to a given frame in the training data.
		 * Internally this will also call OnTimeSliderScrubPositionChanged, to go to the specific frame.
		 * @param FrameNumber The training frame to go to. This will automatically be clamped to a valid range in case it goes out of bounds.
		 * @param bIsScrubbing This is set to true when we are scrubbing the timeline. It is set to false when the mouse button is released.
		 */
		UE_API virtual void SetTrainingFrame(int32 FrameNumber, bool bIsScrubbing);

		/**
		 * Go to a given frame in the test data.
		 * Internally this will also call OnTimeSliderScrubPositionChanged, to go to the specific frame.
		 * @param FrameNumber The test frame to go to. This will automatically be clamped to a valid range in case it goes out of bounds.
		 * @param bIsScrubbing This is set to true when we are scrubbing the timeline. It is set to false when the mouse button is released.
		 */
		UE_API virtual void SetTestFrame(int32 FrameNumber, bool bIsScrubbing);

		/**
		 * Render additional (debug) things in the viewport.
		 * This will draw the vertex deltas when we are in training mode and aren't playing the training data.
		 * @param View The scene view object.
		 * @param Viewport The viewport object.
		 * @param PDI The draw primitive interface used for drawing things.
		 */
		UE_API virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI);

		/**
		 * Update the state that is used to decide whether the Train button in the UI is enabled or not.
		 * A criteria could be to check whether we have all required inputs, such as a skeletal mesh, target mesh and anim sequence.
		 * This should set the bIsReadyForTraining bool to true when it is ready, or false if not.
		 */
		virtual void UpdateIsReadyForTrainingState() { bIsReadyForTraining = false; }

		/**
		 * Get the text that we should display as overlay on top of the asset editor viewport.
		 * @return The text, or an empty text object if there is nothing to show.
		 */
		UE_API virtual FText GetOverlayText() const;

		/**
		 * Initialize the input information, which basically describes what the inputs to the neural network are going to be.
		 * This contains things such as bone and curve names. The input info is used during both training and inference time.
		 * It is used to see what bone transforms and curve values to feed into the network inputs.
		 * Next to bones and curve reference it also stores the vertex counts of the base and target meshes, so we can later do some
		 * compatibility checks when applying this ML Deformer onto some character.
		 * @param InputInfo The input info object to initialize.
		 */
		UE_API virtual void InitInputInfo(UMLDeformerInputInfo* InputInfo);

		/**
		 * Call the SetupComponent on every ML Deformer component on our editor actors and update the compatibility status of the deformer components.
		 * This gets triggered when some input assets change.
		 */
		UE_API virtual void RefreshMLDeformerComponents();

		/**
		 * Create the heat map material for this model.
		 * This updates the HeatMapMaterial member.
		 */
		UE_API virtual void CreateHeatMapMaterial();

		/**
		 * Create the heat map deformer graph for this model.
		 * This updates the HeatMapDeformerGraph member.
		 */
		UE_API virtual void CreateHeatMapDeformerGraph();

		/**
		 * Create the heat map assets for this model.
		 * This calls CreateHeatMapMaterial and CreateHeatMapDeformerGraph on default.
		 */
		UE_API virtual void CreateHeatMapAssets();

		/**
		 * Get the heat map deformer that should currently be used (when heatmaps are enabled).
		 * On default this will return the HeatMapDeformerGraph.
		 * However, if your deformer supports let's say linear blend skinning and dual quaternion skinning, you could
		 * make it return the appropriate deformer graph for skinning method that is currently used.
		 * @return Returns a pointer to the mesh deformer to use when heatmaps are enabled.
		 */
		UE_API virtual UMeshDeformer* GetActiveHeatMapDeformer() const;

		/**
		 * Specify whether the heat map material should be enabled or not.
		 * This will activate the heat map visualization, or deactivate it.
		 * @param bEnabled Set to true to enable heat map mode.
		 */
		UE_API virtual void SetHeatMapMaterialEnabled(bool bEnabled);

		/**
		 * Load the default Optimus deformer graph to use on the skeletal mesh when using this model.
		 * @return The mesh deformer graph.
		 */
		UE_API virtual UMeshDeformer* LoadDefaultDeformerGraph() const;

		/**
		 * This makes sure that the deformer graph selected in the visualization settings is not a nullptr.
		 * If the current graph hasn't been set, it will assign the default deformer graph to it.
		 */
		UE_API virtual void SetDefaultDeformerGraphIfNeeded();

		/**
		 * This applies the right mesh deformer to the editor actors that have an ML Deformer component on them.
		 * Basically this just ensures that the heat map deformer graph is active when this is enabled. Otherwise the default
		 * mesh deformer is used.
		 */
		UE_API virtual void UpdateDeformerGraph();

		/**
		 * Executed when the maximum number of LOD levels changed.
		 * Other models could use this to reinitialize the morph targets for all LOD levels for example.
		 */
		virtual void OnMaxNumLODsChanged() {}

		/**
		 * Sample the vertex deltas between the training base model and target model.
		 * This will initialize the sampler if needed and set the sampler space to post-skinning deltas, and then samples them using the sampler.
		 * So this will update the internal state of the Sampler member. It will use the current training frame number to sample from.
		 * If this number is somehow out of range it will automatically clamp it to be inside a valid range.
		 */
		UE_API virtual void SampleDeltas();

		/**
		 * Load the trained network from an onnx file.
		 * The filename of the onnx file is determined by the GetTrainedNetworkOnnxFile method.
		 */
		virtual bool LoadTrainedNetwork() const { return false; }

		/**
		 * Get the onnx file name of the trained network.
		 * This is the filename that your python script will save the onnx file to.
		 * On default this will return the following filename: "<intermediatedir>/YourModel/YourModel.onnx".
		 * Where "intermediatedir" is your project's Intermediate folder. And "YourModel" is the name of your runtime model class.
		 * So if you have a model class named UFancyModel, on default this method will return "<intermediatedir>/FancyModel/FancyModel.onnx"
		 */
		UE_API virtual FString GetTrainedNetworkOnnxFile() const;

		/**
		 * Check whether our model is trained or not.
		 * This is determined by looking whether the neural network pointer is nullptr or not.
		 * @return Returns true when the model is trained already, or false if it hasn't been trained yet.
		 */
		virtual bool IsTrained() const { return Model->IsTrained(); }

		/**
		 * Get the editor actor that defines the timeline play position.
		 * In training mode this is the training target actor, while in test mode it is the test ground truth actor.
		 * @return A pointer to the ground truth actor depending on whether we're in test or training mode.
		 */
		UE_API virtual FMLDeformerEditorActor* GetTimelineEditorActor() const;

		/**
		 * Get the path to the heat map material asset.
		 * @return The path to the heat map asset.
		 */
		UE_API virtual FString GetHeatMapMaterialPath() const;

		/**
		 * Get the path to the heat map deformer graph asset, when using linear skinning.
		 * @return The path to the heat map deformer graph asset.
		 */
		UE_API virtual FString GetHeatMapDeformerGraphPath() const;

		/**
		 * Get the path to the heat map deformer graph asset, when using dual quaternion skinning.
		 * @return The path to the heat map deformer graph asset.
		 */
		UE_API virtual FString GetHeatMapDeformerGraphDualQuatPath() const;

		/**
		 * Refresh the contents of the input widget.
		 * This includes the bone and curve list contents.
		 */
		UE_API virtual void RefreshInputWidget();

		/**
		 * Create the input widget.
		 * The default widget that is created contains the bone and curve list.
		 * You can inherit from the SMLDeformerInputWidget base class and override this method to return your custom widget.
		 * This can be useful when you want to add new types of inputs to your model.
		 * @return A shared pointer to the input widget that will appear in the UI.
		 */
		UE_API virtual TSharedPtr<SMLDeformerInputWidget> CreateInputWidget();

		/** Update the persona preview scene's actor, skeletal mesh component, mesh and anim asset. These are based on the current visualization mode. */
		UE_API virtual void UpdatePreviewScene();

		/**
		 * Called whenever the actor to debug has been changed.
		 * This can be changed throughout the UI, in testing mode.
		 * When debugging gets disabled, this will contain a nullptr.
		 * @param DebugActor The new actor we want to debug.
		 */
		UE_API virtual void OnDebugActorChanged(TObjectPtr<AActor> DebugActor);

		/** 
		 * Apply the transforms of the actor we debug to the editor actors in our world.
		 * @param DebugActorComponentSpaceTransforms The component space transforms from the actor we are debugging.
		 */
		UE_API virtual void ApplyDebugActorTransforms(const TArray<FTransform>& DebugActorComponentSpaceTransforms);

		/**
		 * Update the character's pose in paint mode.
		 * This will check whether we are in training or testing mode and choose the editor actor based on that.
		 * The paint mode will then use this specific editor actor's skeletal mesh component to calculate skinned vertex positions at its current pose.
		 * After that it will update the dynamic mesh in the paint mode in order to paint in this same pose.
		 * This method is called when you start painting, or scrub the timeline.
		 * @param bFullUpdate When set to true, this can internally update the acceleration structures, which is slow. When set to false it can quickly update the visual mesh
		 *                    It is still important to once call it with bFullUpdate set to true though, before you start painting.
		 */
		UE_API virtual void UpdatePaintModePose(bool bFullUpdate=true);

		/**
		 * Debug draw helpers inside the PIE viewport that highlight debuggable actors.
		 * On default this will draw a bounding box around the actors that can be debugged by this model.
		 * Also it will render the actor names.
		 * NOTE: This renders inside the PIE viewport, not our own MLD asset editor viewport.
		 */
		UE_API virtual void DrawPIEDebugActors();

		/**
		 * Check whether this model supports a per training input animation vertex mask.
		 * If this is enabled, each training intput animation should show something where the user can select an optional mask.
		 * @return Returns true when vertex masking per training input is supported, false otherwise.
		 */
		virtual bool GetSupportsPerTrainingInputAnimVertexMask() const	{ return false; }

		/** 
		 * Update LOD levels of the actors in editor world.
		 * This can be used to for example sync the LOD levels of the compare actors with the main actor.
		 */
		UE_API virtual void UpdateActorLODs();

		/**
		 * This should update the list of available training devices.
		 * If not implemented, the user won't be able to pick a training device.
		 * With training device we mean the device used to store the tensors at. Typically you want to make it build a list of GPU's and have the CPU in there as well.
		 *
		 * <code>
		 * UYourTrainingModel* TrainingModel = NewDerivedObject<UYourTrainingModel>();
		 * if (TrainingModel)
		 * {
		 *     TrainingModel->Init(this);
		 *     TrainingModel->UpdateAvailableDevices();
		 *     TrainingModel->ConditionalBeginDestroy();
		 * }
		 * </code>
		 * 
		 * You can implement the following in Python as an example:
		 * <code>
		 * @unreal.ufunction(override=True)
         * def update_available_devices(self):
         *     reload(mldeformer.training_helpers)
         *     mldeformer.training_helpers.update_training_device_list(self)
		 * </code>
		 * 
		 * Then inside the python code you can do something like:
		 * 
		 * <code>
		 * training_device = model.get_training_device()
         * device_index = mldeformer.training_helpers.find_cuda_device_index(device_name=training_device)
         * if torch.cuda.is_available() and device_index != -1:
         *      torch.cuda.set_device(device_index)
		 * </code>
		 */
		virtual void UpdateTrainingDeviceList() {}

		/** Apply the transforms of the debug actor to the actors in the asset editor world. This will internally call ApplyDebugActorTransforms(DebugActorComponentSpaceTransforms). */
		UE_API void ApplyDebugActorTransforms();

		/** Invalidate the memory usage, so it gets updated in the UI again. */
		UE_API void UpdateMemoryUsage();

		/** Get the currently active training input animation sequence, which is the one that the viewport is showing the timeline for in training mode. */
		UE_API UAnimSequence* GetActiveTrainingInputAnimSequence() const;

		/** Get the current view range. */
		UE_API TRange<double> GetViewRange() const;

		/** Set the current view range. */
		UE_API void SetViewRange(TRange<double> InRange);

		/** Get the working range of the model's data. */
		UE_API TRange<double> GetWorkingRange() const;

		/** Get the playback range of the model's data. */
		UE_API TRange<FFrameNumber> GetPlaybackRange() const;

		/** Get the current scrub position. */
		UE_API FFrameNumber GetTickResScrubPosition() const;

		/** Get how many ticks there are per frame. */
		UE_API int32 GetTicksPerFrame() const;

		/** Get the current scrub time. */
		UE_API float GetScrubTime() const;

		/** Set the current scrub position. */
		UE_API void SetScrubPosition(FFrameTime NewScrubPosition, bool bIsScrubbing);

		/** Set the current scrub position. */
		UE_API void SetScrubPosition(FFrameNumber NewScrubPosition, bool bIsScrubbing);

		/** Set if frames are displayed. */
		UE_API void SetDisplayFrames(bool bDisplayFrames);

		/** Is the timeline displaying frames rather than seconds? */
		UE_API bool IsDisplayingFrames() const;

		/** Handle the timeline changes when our model changes. */
		UE_API void HandleModelChanged();

		/** Handle the timeline changes required when changing the visualization mode, so when we switch between training and testing mode. */
		UE_API void HandleVizModeChanged(EMLDeformerVizMode Mode);

		/** Handle the view range being changed. */
		UE_API void HandleViewRangeChanged(TRange<double> InRange);

		/** Handle the working range being changed. */
		UE_API void HandleWorkingRangeChanged(TRange<double> InRange);

		/** Get the framerate specified by the anim sequence. */
		UE_API double GetFrameRate() const;

		/** Get the tick resolution we are displaying at. */
		UE_API int32 GetTickResolution() const;

		/** Get a pointer to the ML Deformer editor tookit. Cannot be a nullptr. */
		FMLDeformerEditorToolkit* GetEditor() const { return Editor; }

		/** Get a pointer to the runtime ML Deformer model. Cannot really be a nullptr. */
		UMLDeformerModel* GetModel() const { return Model.Get(); }

		/** Get a pointer to the world that our editor is displaying. Cannot be nullptr. */
		UE_API UWorld* GetWorld() const;

		/** Get all the created editor actors. */
		const TArray<FMLDeformerEditorActor*>& GetEditorActors() const { return EditorActors; }

		/** Find an editor actor with a specific ID, or return nullptr when not found. */
		UE_API FMLDeformerEditorActor* FindEditorActor(int32 ActorTypeID) const;

		/** Check whether we are ready to train. The Train button in the UI will be enabled or disabled based on this return value. */
		bool IsReadyForTraining() const { return bIsReadyForTraining; }

		/**
		 * Get the vertex delta sampler for a given training input animation.
		 * @param AnimIndex The training input animation index.
		 * @return A pointer to the sampler for this specific input anim, or nullptr in case the index is out of range.
		 */
		FMLDeformerSampler* GetSamplerForTrainingAnim(int32 AnimIndex) const	{ return Samplers.IsValidIndex(AnimIndex) ? Samplers[AnimIndex].Get() : nullptr; }

		/**
		 * Set whether we need to resample inputs or not. This can be used by the training code to determine if we need to resample inputs and outputs, or if
		 * some cache can be used. Resampling is generally needed when input assets change, or other specific settings.
		 * @param bNeeded Set to true to mark that inputs or other cached data need resampling.
		 */
		void SetResamplingInputOutputsNeeded(bool bNeeded) { bNeedToResampleInputOutputs = bNeeded; }

		/**
		 * Check whether we need to resample inputs and outputs.
		 * @return Returns true if resampling of the inputs and outputs is needed, for example because input assets have changed.
		 * @see SetResamplingInputOutputsNeeded
		 */
		bool GetResamplingInputOutputsNeeded() const { return bNeedToResampleInputOutputs; }

		/**
		 * Check whether the vertex counts of the skeletal mesh have changed compared to the one we have trained on.
		 * @return An error string if there is an error, otherwise an empty text object is returned.
		 */
		UE_API FText GetBaseAssetChangedErrorText() const;

		/**
		 * Check whether the vertex map has changed or not.
		 * This basically checks whether the vertex order has changed.
		 * @return An error string if there is an error, otherwise an empty text object is returned.
		 */
		UE_API FText GetVertexMapChangedErrorText() const;

		/**
		 * Check for errors in the inputs. This basically checks if there are bones or curves to train on.
		 * If there aren't any it will give some error.
		 * @return An error string if there is an error, otherwise an empty text object is returned.
		 */
		UE_API FText GetInputsErrorText() const;

		/**
		 * Check for incompatible skeletons between the skeletal mesh and anim sequence skeleton.
		 * @return An error string if there is an error, otherwise an empty text object is returned.
		 */
		UE_API FText GetIncompatibleSkeletonErrorText(const USkeletalMesh* InSkelMesh, const UAnimSequence* InAnimSeq) const;

		/**
		 * Check to see if the skeletal mesh has to be reimported because it misses some newly added data.
		 * We have added imported mesh information to the Fbx importer. Older assets won't have this info, while this is required
		 * to make the ML Deformer work properly.
		 * @return An error string if there is an error, otherwise an empty text object is returned.
		 */
		UE_API FText GetSkeletalMeshNeedsReimportErrorText() const;

		/**
		 * Check whether the vertex count in the target mesh has changed or not.
		 * If this is the case you would need to re-train the model.
		 * @return An error string if there is an error, otherwise an empty text object is returned.
		 */
		UE_API FText GetTargetAssetChangedErrorText() const;

		/**
		 * Get the editor's input info object.
		 * This is the information the user currently has setup in the UI, so not the input info of the runtime model.
		 * The runtime model's input info contains the data inputs the model has been trained on, which can be different from what the user currently has set up in the UI.
		 * @return A pointer to the editor's input info object.
		 */
		UMLDeformerInputInfo* GetEditorInputInfo() const { return EditorInputInfo.Get(); }	

		/**
		 * Update the editor's input info based on the current UI settings.
		 */
		UE_API void UpdateEditorInputInfo();

		/**
		 * This will call OnInputAssetsChanged first, followed by OnPostInputAssetChanged.
		 * It then will update the editor only flags for the assets, and then forces a refresh of the model details panel and optionally also the visualization settings panel.
		 * @param bRefreshVizSettings Set to true if you want the visualization settings details panel to also refresh.
		 */
		UE_API void TriggerInputAssetChanged(bool bRefreshVizSettings=false);

		/**
		 * Initialize the bone include list of the runtime model to the set of bones that are animated.
		 * A bone is seen as animated if its animation data has keyframes that have a different value than the first keyframe.
		 * So if the bone's animation data has 1000 keyframes, and they are all the same value, it won't see it as animated.
		 */
		UE_API void InitBoneIncludeListToAnimatedBonesOnly();

		/**
		 * Initialize the curve include list of the runtime model to the set of curves that are animated.
		 * A curve is seen as animated if its animation data has keyframes that have a different value than the first keyframe.
		 * So if the curve's animation data has 1000 keyframes, and they are all the same value, it won't see it as animated.
		 */
		UE_API void InitCurveIncludeListToAnimatedCurvesOnly();

		/**
		 * Add all animated bones to the bone include list.
		 */
		UE_API void AddAnimatedBonesToBonesIncludeList();

		/**
		 * Add all animated curves to the curve include list.
		 */
		UE_API void AddAnimatedCurvesToCurvesIncludeList();

		/**
		 * Get the number of curves on a specific skeletal mesh.
		 * @param SkelMesh The skeletal mesh to get the curves count from.
		 * @return The number of curves found on the skeletal mesh.
		 */
		UE_API int32 GetNumCurvesOnSkeletalMesh(USkeletalMesh* SkelMesh) const;

		/**
		 * Get the currently desired training frame number.
		 * You can call CheckTrainingFrameChanged in order to make the desired frame the actual frame.
		 * @return The currently desired training frame number, which might not be the same as the real training frame number.
		 */
		int32 GetCurrentTrainingFrame() const { return CurrentTrainingFrame; }

		/**
		 * Check whether our training data frame number changed.
		 * We have a TrainingFrameNumber and a CurrentTrainingFrame. If they don't match it will call OnTrainingDataFrameChanged.
		 */
		UE_API void CheckTrainingDataFrameChanged();

		/**
		 * Find the ML Deformer component of a given editor actor.
		 * @param ActorID The actor ID to get the component for.
		 * @return A pointer to the ML Deformer component, if it exists on the actor, otherwise nullptr is returned.
		 */
		UE_API UMLDeformerComponent* FindMLDeformerComponent(int32 ActorID = ActorID_Test_MLDeformed) const;

		/**
		 * Correct floating point errors that can cause issues when sampling animation using step timing.
		 * @param FrameNumber The desired frame number.
		 * @param FrameTimeForFrameNumber The frame time computed for the frame number, which may have conversion errors.
		 * @param FrameRate The frame rate for conversion.
		 * @return A corrected frame time, which is equal to FrameTimeForFrameNumber unless a floating point error means the floor is not equal to the FrameNumber.
		 */
		static UE_API float CorrectedFrameTime(int32 FrameNumber, float FrameTimeForFrameNumber, FFrameRate FrameRate);

		TSharedPtr<SMLDeformerInputWidget> GetInputWidget() const				{ return InputWidget; }
		void SetInputWidget(const TSharedPtr<SMLDeformerInputWidget>& Widget)	{ InputWidget = Widget; }

		UE_API FMLDeformerSampler* GetSamplerForActiveAnim() const;

		UE_API void SetActiveTrainingInputAnimIndex(int32 Index);
		UE_API int32 GetActiveTrainingInputAnimIndex() const;

		/** Mark the deltas to be updated on next Tick. */
		UE_API void InvalidateDeltas();

		/** 
		 * Find the float based vertex attributes on the skeletal map that have a given name.
		 * You can check whether the attribute data has been found using ReturnedValue.IsValid().
		 */
		UE_API TVertexAttributesConstRef<float> FindVertexAttributes(FName AttributeName) const;

		/** Are we scrubbing on the timeline? */
		bool IsScrubbingTimeline() const									{ return bIsScrubbingTimeline; }

		/** Get the array of vertex attribute names that can be used for UI widget elements. Refresh this list with UpdateVertexAttributeNames(). */
		const TArray<TSharedPtr<FName>>& GetVertexAttributeNames() const	{ return VertexAttributeNames; }

		/** Update the vertex attribute names array as returned by GetVertexAttributeNames(). This is some array that can be used for UI elements like combo boxes. */
		UE_API void UpdateVertexAttributeNames();

		/**
		 * Request a call to TriggerInputAssetsChanged on the next tick, so not directly.
		 * This can be useful to prevent multiple nested calls to TriggerInputAssetsChanged.
		 */
		void RequestTriggerInputAssetsChangedNextTick()						{ bNeedsAssetReinit = true; }

	protected:
		UE_API virtual void CreateSamplers();

		UE_API void AddAnimatedBonesToBonesIncludeList(const UAnimSequence* AnimSequence);
		UE_API void AddAnimatedCurvesToCurvesIncludeList(const UAnimSequence* AnimSequence);

		/**
		 * Executed when a property changes by a change in the UI.
		 * This internally calls OnPropertyChanged.
		 */
		UE_API void OnPostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent);

		/**
		 * Trigger a PostEditChangeProperty using a ValueSet event type, on the model, which will also trigger the OnPostEditChangeProperty.
		 * @param PropertyName The name of the property to trigger it for.
		 */
		UE_API void BroadcastPropertyChanged(const FName PropertyName);

		/**
		 * Called whenever an engine object is being modified.
		 * Use this to set the reinitialize assets flag. For example when a skeletal mesh or anim sequence change (being modified or reimported), we should 
		 * set the bNeedsAssetReinit member to true, to trigger the actor components and UI etc to be reinitialized.
		 * The default implementation of this method will check for changes in the skeletal mesh and training and test anim sequence.
		 * If you want to check more things (your target mesh), you need to check for that.
		 */
		UE_API virtual void OnObjectModified(UObject* Object);

		/**
		 * Delete all editor actors.
		 * This does not actually delete the actors in the world.
		 * You can call ClearWorld to include that.
		 */
		UE_API void DeleteEditorActors();

		/**
		 * Set the names of the anim sequences in the timeline. 
		 */
		UE_API void SetTimelineAnimNames(const TArray<TSharedPtr<FMLDeformerTrainingInputAnimName>>& AnimNames);

		/**
		 * Perform some basic checks to see if the editor can be ready to train the model.
		 * This checks whether there is an anim sequence, target mesh, and skeletal mesh.
		 * It can be called inside the UpdateIsReadyForTrainingState method, which can then do additional checks afterwards.
		 * @return Returns true when at least the basic required inputs have been selected, false if not.
		 */
		UE_API bool IsEditorReadyForTrainingBasicChecks();

		/**
		 * Get the base actor depending on the currently active mode (train or test mode).
		 * The base actor is the linear-skinned one.
		 * @return A pointer to the linear skinned actor in either test or training mode.
		 */
		UE_API FMLDeformerEditorActor* GetVisualizationModeBaseActor() const;

		/**
		 * Get the animation sequence, either the test or training one, depending on in which mode we are in.
		 * This can return a nullptr if no anim sequence has been set up yet.
		 * @return A pointer to the test sequence when we are in test mode, or the training anim sequence if we are in training mode.
		 */
		UE_API const UAnimSequence* GetAnimSequence() const;

		/**
		 * Calculate the current timeline position offset, of the currently selected frame.
		 * This depends on whether we are in test or training mode.
		 * @return The timeline offset, in seconds.
		 */
		UE_API double CalcTimelinePosition() const;

		/**
		 * Update the timeline related ranges, based on the length of the training or testing data.
		 */
		UE_API void UpdateRanges();

		UE_API void AddCompareActor(int32 ArrayIndex);
		UE_API void RemoveCompareActor(int32 ArrayIndex);
		UE_API void RemoveAllCompareActors();
		UE_API void UpdateCompareActorLabels();		

		UE_API int32 CalcNumValidCompareActorsPriorTo(int32 CompareActorIndex) const;
		UE_API virtual bool IsAnimIndexValid(int32 AnimIndex) const;

		UE_API void UpdateLODMappings();

		/**
		 * Update whether we force to use step interpolation on the ML Deformed model or not.
		 * Step interpolation is needed when we are enabled heatmaps and have a ground truth mesh and are in ground truth heatmap mode.
		 * This will modify the AnimInstance's ForceStepInterpolation state on just the ML Deformed model. It can also disable it when needed.
		 */
		UE_API void UpdateStepInterpolationMode();

		/**
		 * Because of some technical reasons we cannot display a correct ground truth heat map at the moment while playing an animation.
		 * Scrubbing works fine though, because we can align on the right keyframes then.
		 * This method will look whether we are having heatmaps enabled or not, and if we are playing or not and if we're in ground truth mode.
		 * It then disables or enables the heatmap material based on the current state.
		 */
		UE_API void UpdateHeatMapMaterialBasedOnMode();

	protected:
		struct FLODInfo
		{
			/** 
			 * Map all vertices in this LOD into an imported vertex number in LOD 0. 
			 * This basically tells us which vertex to get the vertex delta for, for each vertex in this LOD.
			 */
			TArray<int32> VtxMappingToLODZero;
		};

		/** The LOD information. The size of this array is the number of LODs on the skeletal mesh, unless less LOD levels are desired using MLD. */
		TArray<FLODInfo> LODMappings;

		/** A reusable array to store current LOD skinned vertex positions. */
		TArray<FVector3f> SkinnedPositions;

		/** The runtime model associated with this editor model. */
		TObjectPtr<UMLDeformerModel> Model = nullptr;

		/** The set of actors that can appear inside the editor viewport. */
		TArray<FMLDeformerEditorActor*> EditorActors;

		/** A pointer to the editor toolkit. */
		FMLDeformerEditorToolkit* Editor = nullptr;

		/** A sampler for every input animation. These samplers are used to calculate the vertex deltas. */
		TArray<TSharedPtr<FMLDeformerSampler>> Samplers;

		/** The inputs widget. */
		TSharedPtr<SMLDeformerInputWidget> InputWidget;

		/**
		 * The input info as currently setup in the editor.
		 * This is different from the runtime model's input info, as that is the one that was used to train with.
		 */
		TObjectPtr<UMLDeformerInputInfo> EditorInputInfo = nullptr;

		/** The heatmap material. */
		TObjectPtr<UMaterial> HeatMapMaterial = nullptr;

		/** The heatmap deformer graph. */
		TObjectPtr<UMeshDeformer> HeatMapDeformerGraph = nullptr;

		/** The dual quaternion based heat map. */
		TObjectPtr<UMeshDeformer> HeatMapDeformerGraphDualQuat = nullptr;

		/** A list of vertex attribute names that can be used inside combobox UI elements. */
		TArray<TSharedPtr<FName>> VertexAttributeNames;

		/** The delegate handle to the post edit property event. */
		FDelegateHandle PostEditPropertyDelegateHandle;

		/** The delegate handle linked to PostTransacted of the model. */
		FDelegateHandle PostTransactedDelegateHandle;

		/** The delegate handle linked to PreEditUndo of the model. */
		FDelegateHandle PreEditUndoDelegateHandle;

		/** The delegate handle linked to PostEditUndo of the model. */
		FDelegateHandle PostEditUndoDelegateHandle;

		/** The current training frame. */
		int32 CurrentTrainingFrame = -1;

		/** The range we are currently viewing */
		TRange<double> ViewRange;

		/** The working range of this model, encompassing the view range */
		TRange<double> WorkingRange;

		/** The playback range of this model for each timeframe */
		TRange<double> PlaybackRange;

		/** The current scrub position. */
		FFrameTime ScrubPosition;

		/**
		 * Are we ready for training? 
		 * The training button in the editor will be enabled or disabled based on this on default.
		 */
		bool bIsReadyForTraining = false;	

		/** Do we need to resample all input/output data? */
		bool bNeedToResampleInputOutputs = true;

		/** Display frame numbers? */
		bool bDisplayFrames = true;

		/** The delegate that handles when an object got modified (any object). */
		FDelegateHandle InputObjectModifiedHandle;

		/** The delegate that handles when an object property got changed (any object). */
		FDelegateHandle InputObjectPropertyChangedHandle;

		/** Set to true when on next tick we need to trigger an input assets changed event. */
		bool bNeedsAssetReinit = false;

		/**
		 * The total number of frames that can be used for training, based on the training input animations list.
		 * Call UpdateNumTrainingFrames() to refresh this value.
		 */
		int32 NumTrainingFrames = 0;

		/** The training input animation that is selected in the timeline. */
		int32 ActiveTrainingInputAnimIndex = INDEX_NONE;

		/** Are we currently scrubbing the timeline? */
		bool bIsScrubbingTimeline = false;
	};

	/**
	 * Train the model.
	 * This will execute the train function inside your python code.
	 * Provide the UMLDeformerTrainingModel inherited class type as template argument.
	 */
	template<class TrainingModelClass>
	ETrainingResult TrainModel(FMLDeformerEditorModel* EditorModel)
	{
		// Find class, which will include our Python class, generated from the python script.
		TArray<UClass*> TrainingModels;
		GetDerivedClasses(TrainingModelClass::StaticClass(), TrainingModels, false);
		if (TrainingModels.IsEmpty())
		{
			// We didn't define a derived class in Python.
			return ETrainingResult::FailPythonError;
		}

		// Perform the training.
		// This will trigger the Python class train function to be called.
		EditorModel->TriggerInputAssetChanged(false);
		TrainingModelClass* TrainingModel = Cast<TrainingModelClass>(TrainingModels.Last()->GetDefaultObject());
		TrainingModel->Init(EditorModel);
		const int32 ReturnCode = TrainingModel->Train();
		return static_cast<ETrainingResult>(ReturnCode);
	}
}	// namespace UE::MLDeformer

#undef UE_API
