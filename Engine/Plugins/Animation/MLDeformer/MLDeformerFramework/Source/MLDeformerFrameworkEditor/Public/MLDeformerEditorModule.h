// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Modules/ModuleInterface.h"
#include "MLDeformerModelRegistry.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

namespace UE::MLDeformer
{
	namespace TrainingDataProcessor
	{
		class FTrainingDataProcessorTool;
	}

	/**
	 * The ML Deformer editor module.
	 * This registers the editor mode and custom property customizations.
	 * It also contains the model registry, which you will use to register your custom models to the editor.
	 */
	class FMLDeformerEditorModule
		: public IModuleInterface
	{
	public:
		// IModuleInterface overrides.
		UE_API virtual void StartupModule() override;
		UE_API virtual void ShutdownModule() override;
		// ~END IModuleInterface overrides.

		/**
		 * Get a reference to the ML Deformer model registry.
		 * You use this to see what kind of model types ther are, register new models, or create editor models for specific types of runtime models.
		 * @return A reference to the ML Deformer model registry.
		 */
		FMLDeformerEditorModelRegistry& GetModelRegistry()						{ return ModelRegistry; }
		const FMLDeformerEditorModelRegistry& GetModelRegistry() const			{ return ModelRegistry; }

		/**
		 * Manually assign a new model registry.
		 * This can be used in some automated tests where we want to test with different registries, for example an empty one.
		 * @param Registry The model registry to use.
		 */
		void SetModelRegistry(const FMLDeformerEditorModelRegistry& Registry)	{ ModelRegistry = Registry; }

	private:
		/** The model registry, which keeps track of all model types and instances created of these models, and how to create the editor models for specific runtime model types. */
		FMLDeformerEditorModelRegistry ModelRegistry;
	};
}	// namespace UE::MLDeformer

#undef UE_API
