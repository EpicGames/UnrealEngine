// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

namespace UE::MLDeformer
{
	/**
	 * The editor style class that describes specific UI style related settings for the ML Deformer editor.
	 */
	class FMLDeformerEditorStyle
		: public FSlateStyleSet
	{
	public:
		UE_API FMLDeformerEditorStyle();
		UE_API ~FMLDeformerEditorStyle();

		static UE_API FMLDeformerEditorStyle& Get();
	};
}	// namespace UE::MLDeformer

#undef UE_API
