// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerTrainingModel.h"
#include "MLDeformerEditorModel.h"
#include "MLDeformerVizSettings.h"
#include "IHasPersonaToolkit.h"
#include "IPersonaPreviewScene.h"
#include "IPersonaViewport.h"
#include "PersonaAssetEditorToolkit.h"
#include "EditorUndoClient.h"
#include "Framework/MultiBox/MultiBox.h"
#include "Math/Color.h"
#include "Widgets/Notifications/SNotificationList.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

struct FMenuEntryParams;
class FTabManager;
class IDetailsView;
class UMLDeformerAsset;
class SSimpleTimeSlider;

namespace UE::MLDeformer
{
	class SMLDeformerTimeline;
	class FMLDeformerApplicationMode;
	class SMLDeformerDebugSelectionWidget;

	namespace MLDeformerEditorModes
	{
		extern const FName Editor;
	}

	class FMLDeformerEditorToolkit;
	class FToolsMenuExtender
	{
	public:
		virtual ~FToolsMenuExtender() = default;
		virtual FMenuEntryParams GetMenuEntry(FMLDeformerEditorToolkit& Toolkit) const = 0;
		virtual TSharedPtr<FWorkflowTabFactory> GetTabSummoner(const TSharedRef<FMLDeformerEditorToolkit>& Toolkit) const { return nullptr; }
	};

	/**
	 * The ML Deformer asset editor toolkit.
	 * This is the editor that opens when you double click an ML Deformer asset.
	 */
	class FMLDeformerEditorToolkit :
		public FPersonaAssetEditorToolkit,
		public IHasPersonaToolkit,
		public FGCObject,
		public FEditorUndoClient,
		public FTickableEditorObject
	{
	public:
		friend class FMLDeformerApplicationMode;
		friend struct FMLDeformerVizSettingsTabSummoner;

		UE_API ~FMLDeformerEditorToolkit();

		/** Initialize the asset editor. This will register the application mode, init the preview scene, etc. */
		UE_API void InitAssetEditor(
			const EToolkitMode::Type Mode,
			const TSharedPtr<IToolkitHost>& InitToolkitHost,
			UMLDeformerAsset* InDeformerAsset);

		// FAssetEditorToolkit overrides.
		UE_API virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
		UE_API virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
		UE_API virtual FName GetToolkitFName() const override;
		UE_API virtual FText GetBaseToolkitName() const override;
		UE_API virtual FText GetToolkitName() const override;
		UE_API virtual FLinearColor GetWorldCentricTabColorScale() const override;
		UE_API virtual FString GetWorldCentricTabPrefix() const override;
		UE_API virtual void AddViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget, int32 ZOrder = INDEX_NONE) override;
		UE_API virtual void RemoveViewportOverlayWidget(TSharedRef<SWidget> InViewportOverlayWidget) override;
		// ~END FAssetEditorToolkit overrides.

		// FGCObject overrides.
		virtual FString GetReferencerName() const override							{ return TEXT("FMLDeformerEditorToolkit"); }
		UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		// ~END FGCObject overrides.

		// FTickableEditorObject overrides.
		UE_API virtual void Tick(float DeltaTime) override;
		virtual ETickableTickType GetTickableTickType() const override				{ return ETickableTickType::Always; }
		UE_API virtual TStatId GetStatId() const override;
		// ~END FTickableEditorObject overrides.

		// FEditorUndoClient overrides.
		UE_API void PostUndo(bool bSuccess) override;
		UE_API void PostRedo(bool bSuccess) override;
		// ~END FEditorUndoClient overrides.

		// IHasPersonaToolkit overrides.
		UE_API virtual TSharedRef<IPersonaToolkit> GetPersonaToolkit() const override;
		IPersonaToolkit* GetPersonaToolkitPointer() const							{ return PersonaToolkit.Get(); }
		// ~END IHasPersonaToolkit overrides.

		void SetVizSettingsDetailsView(TSharedPtr<IDetailsView> InDetailsView)		{ VizSettingsDetailsView = InDetailsView; }
		UE_API IDetailsView* GetVizSettingsDetailsView() const;

		UE_API IDetailsView* GetModelDetailsView() const;

		UE_API void SetTimeSlider(TSharedPtr<SMLDeformerTimeline> InTimeSlider);
		UE_API SMLDeformerTimeline* GetTimeSlider() const;

		UMLDeformerAsset* GetDeformerAsset() const									{ return DeformerAsset.Get(); }
		FMLDeformerEditorModel* GetActiveModel()									{ return ActiveModel.Get(); }
		TWeakPtr<FMLDeformerEditorModel> GetActiveModelPointer()					{ return TWeakPtr<FMLDeformerEditorModel>(ActiveModel); }
		const FMLDeformerEditorModel* GetActiveModel() const						{ return ActiveModel.Get(); }
		FMLDeformerApplicationMode* GetApplicationMode() const						{ return ApplicationMode; }
		TSharedPtr<IPersonaViewport> GetViewport() const							{ return PersonaViewport; }

		void SetNeedsPaintModeDisable(bool bNeedsDisable)							{ bNeedsPaintModeDisable = bNeedsDisable; }

		UE_API double CalcTimelinePosition() const;
		UE_API void OnTimeSliderScrubPositionChanged(double NewScrubTime, bool bIsScrubbing);
		UE_API void UpdateTimeSliderRange();
		UE_API void SetTimeSliderRange(double StartTime, double EndTime);

		UE_API void EnablePaintMode();
		UE_API void DisablePaintMode();
		UE_API bool IsDefaultModeActive() const;
		UE_API bool IsPaintModeActive() const;

		bool IsInitialized() const { return bIsInitialized; }

		/**
		 * Switch the editor to a given model type.
		 * @param ModelType The model type you want to switch to, for example something like: UNeuralMorphModel::StaticClass().
		 * @param bForceChange Force changing to this model? This will suppress any UI popups.
		 * @return Returns true in case we successfully switched model types, or otherwise false is returned.
		 */
		UE_API bool SwitchModelType(UClass* ModelType, bool bForceChange);

		/**
		 * Switch the editor's visualization mode.
		 * This essentially allows you to switch the UI between testing and training modes.
		 * @param Mode The mode to switch to.
		 */
		UE_API void SwitchVizMode(EMLDeformerVizMode Mode);

		UE_API bool Train(bool bSuppressDialogs);
		UE_API bool IsTrainButtonEnabled() const;
		UE_API bool IsTraining() const;

		/** Get the actor we want to debug, if any. Returns a nullptr when we don't want to debug anything. */
		UE_API AActor* GetDebugActor() const;

		/** Get the component space transforms of the actor we want to debug. Returns an empty array if GetDebugActor returns a nullptr. */
		UE_API TArray<FTransform> GetDebugActorComponentSpaceTransforms() const;

		UE_API void ZoomOnActors();

		TSharedPtr<SMLDeformerDebugSelectionWidget> GetDebugWidget() const { return DebugWidget; }

		static UE_API void AddToolsMenuExtender(TUniquePtr<FToolsMenuExtender> Extender);
		static UE_API TConstArrayView<TUniquePtr<FToolsMenuExtender>> GetToolsMenuExtenders();

	private:
		UE_API void OnModelChanged(int Index, bool bForceChange);

		/* Toolbar related. */
		UE_API void ExtendToolbar();
		UE_API void FillToolbar(FToolBarBuilder& ToolbarBuilder);

		/** Preview scene setup. */
		UE_API void HandlePreviewSceneCreated(const TSharedRef<IPersonaPreviewScene>& InPersonaPreviewScene);
		UE_API void HandleViewportCreated(const TSharedRef<IPersonaViewport>& InPersonaViewport);
		UE_API void HandleDetailsCreated(const TSharedRef<class IDetailsView>& InDetailsView);
		UE_API void OnFinishedChangingDetails(const FPropertyChangedEvent& PropertyChangedEvent);

		/** Helpers. */
		UE_API void ShowNotification(const FText& Message, SNotificationItem::ECompletionState State, bool PlaySound) const;
		UE_API FText GetOverlayText() const;
		UE_API void OnSwitchedVisualizationMode();
		UE_API bool HandleTrainingResult(ETrainingResult TrainingResult, double TrainingDuration, bool& bOutUsePartiallyTrained, bool bSuppressDialogs, bool& bOutSuccess);
		UE_API FText GetActiveModelName() const;
		UE_API FText GetCurrentVizModeName() const;
		UE_API FText GetVizModeName(EMLDeformerVizMode Mode) const;
		UE_API void ShowNoModelsWarningIfNeeded();
		UE_API EVisibility GetDebuggingVisibility() const;
		UE_API FSlateIcon GetTrainingStatusIcon() const;

		UE_API TSharedRef<SWidget> GenerateModelButtonContents(TSharedRef<FUICommandList> InCommandList);
		UE_API TSharedRef<SWidget> GenerateVizModeButtonContents(TSharedRef<FUICommandList> InCommandList);
		UE_API TSharedRef<SWidget> GenerateToolsMenuContents(TSharedRef<FUICommandList> InCommandList);


	private:
		/** The persona toolkit. */	
		TSharedPtr<IPersonaToolkit> PersonaToolkit;

		/** Model details view. */
		TSharedPtr<IDetailsView> ModelDetailsView;

		/** Model viz settings details view. */
		TSharedPtr<IDetailsView> VizSettingsDetailsView;

		/** The timeline slider widget. */
		TSharedPtr<SMLDeformerTimeline> TimeSlider;

		/** The currently active editor model. */
		TSharedPtr<FMLDeformerEditorModel> ActiveModel;

		// Persona viewport.
		TSharedPtr<IPersonaViewport> PersonaViewport;

		/** The ML Deformer Asset. */
		TObjectPtr<UMLDeformerAsset> DeformerAsset;

		/** The widget where you select which actor to debug. */
		TSharedPtr<SMLDeformerDebugSelectionWidget> DebugWidget;

		/** The active application mode. */
		FMLDeformerApplicationMode* ApplicationMode = nullptr;

		/** Has the asset editor been initialized? */
		bool bIsInitialized = false;

		/** Are we currently in a training process? */
		bool bIsTraining = false;

		/** 
		 * When set to true, the paint mode will be disabled automatically.
		 * This is used to defer the disable of the paint mode to a later stage in the frame, as sometimes this
		 * can cause issues by deactivating or activating modes while in a loop over all modes.
		 */
		bool bNeedsPaintModeDisable = false;

		/** Extenders for Tools menu */
		static UE_API TArray<TUniquePtr<FToolsMenuExtender>> ToolsMenuExtenders;

		/** Mutex for adding extenders */
		static UE_API FCriticalSection ExtendersMutex;
	};
}	// namespace UE::MLDeformer

#undef UE_API
