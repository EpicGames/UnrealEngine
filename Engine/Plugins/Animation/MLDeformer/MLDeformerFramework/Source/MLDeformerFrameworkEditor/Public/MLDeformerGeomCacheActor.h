// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MLDeformerEditorActor.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class UMLDeformerComponent;
class UGeometryCacheComponent;

namespace UE::MLDeformer
{
	/**
	 * An editor actor with a geometry cache component on it.
	 * This can for example be used as ground truth viewport actors.
	 */
	class FMLDeformerGeomCacheActor
		: public FMLDeformerEditorActor
	{
	public:
		UE_API FMLDeformerGeomCacheActor(const FConstructSettings& Settings);

		void SetGeometryCacheComponent(UGeometryCacheComponent* Component)	{ GeomCacheComponent = Component; }
		UGeometryCacheComponent* GetGeometryCacheComponent() const			{ return GeomCacheComponent; }

		// FMLDeformerEditorActor overrides.
		UE_API virtual void SetVisibility(bool bIsVisible) override;
		UE_API virtual bool IsVisible() const override;
		UE_API virtual bool HasVisualMesh() const override;
		UE_API virtual void SetPlayPosition(float TimeInSeconds, bool bAutoPause = true) override;
		UE_API virtual float GetPlayPosition() const override;
		UE_API virtual void SetPlaySpeed(float PlaySpeed) override;
		UE_API virtual void Pause(bool bPaused) override;
		UE_API virtual bool IsPlaying() const override;
		UE_API virtual FBox GetBoundingBox() const override;
		// ~END FMLDeformerEditorActor overrides.

	protected:
		/** The geometry cache component. */
		TObjectPtr<UGeometryCacheComponent> GeomCacheComponent = nullptr;
	};
}	// namespace UE::MLDeformer

#undef UE_API
