// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerEditorModel.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class UMLDeformerGeomCacheModel;
class UMLDeformerGeomCacheVizSettings;

namespace UE::MLDeformer
{
	class FMLDeformerGeomCacheActor;
	class FMLDeformerGeomCacheSampler;

	/**
	 * An editor model for the a model derived from the UMLDeformerGeomCacheModel runtime model.
	 * This editor model automatically handles sampling of the geometry caches to calculate deltas.
	 * It also handles the creation of ground truth models with a geometry cache on them.
	 */
	class FMLDeformerGeomCacheEditorModel
		: public FMLDeformerEditorModel
	{
	public:
		// FGCObject overrides.
		virtual FString GetReferencerName() const override { return TEXT("FMLDeformerGeomCacheEditorModel"); }
		// ~END FGCObject overrides.

		// FMLDeformerEditorModel overrides.
		UE_API virtual void Init(const InitSettings& Settings) override;
		UE_API virtual void CopyBaseSettingsFromModel(const FMLDeformerEditorModel* SourceEditorModel) override;
		UE_API virtual FMLDeformerEditorActor* CreateEditorActor(const FMLDeformerEditorActor::FConstructSettings& Settings) const override;
		UE_API virtual TSharedPtr<FMLDeformerSampler> CreateSamplerObject() const override;
		UE_API virtual void CreateTrainingGroundTruthActor(UWorld* World) override;
		UE_API virtual void CreateTestGroundTruthActor(UWorld* World) override;
		UE_API virtual int32 GetNumTrainingInputAnims() const override;
		UE_API virtual FMLDeformerTrainingInputAnim* GetTrainingInputAnim(int32 Index) const override;
		UE_API virtual void UpdateNumTrainingFrames() override;
		UE_API virtual void UpdateIsReadyForTrainingState() override;
		UE_API virtual void OnPropertyChanged(FPropertyChangedEvent& PropertyChangedEvent) override;
		UE_API virtual void OnInputAssetsChanged() override;
		UE_API virtual void OnObjectModified(UObject* Object) override;
		UE_API virtual ETrainingResult Train() override;
		// ~END FMLDeformerEditorModel overrides.
		
		// Helpers.
		UE_API UMLDeformerGeomCacheModel* GetGeomCacheModel() const;
		UE_API UMLDeformerGeomCacheVizSettings* GetGeomCacheVizSettings() const;
		UE_API FMLDeformerGeomCacheActor* FindGeomCacheEditorActor(int32 ID) const;
		UE_API UGeometryCache* GetActiveGeometryCache() const;


	protected:
		/**
		 * Creates a geometry cache editor actor (FMLDeformerGeomCacheActor) of a specific ID with a specific geometry cache and label.
		 * This will add a new editor actor to the list of editor actors.
		 * @param World The world to create the actor in.
		 * @param ActorID The ID that the actor will have after creation, for example UE::MLDeformer::ActorID_Train_GroundTruth.
		 * @param Name The name of the actor in the scene.
		 * @param GeomCache The geometry cache to use on this actor.
		 * @param LabelColor The color of the label that is rendered with the editor actor inside the editor viewport.
		 * @param WireframeColor The color of this actor when wireframe rendering is enabled.
		 * @param LabelText The text of the label.
		 * @param bIsTrainingActor Set this to true when the actor is an actor to be used in training mode, or set to false when it is to be used in testing mode.
		 */
		UE_API void CreateGeomCacheActor(UWorld* World, int32 ActorID, const FName& Name, UGeometryCache* GeomCache, FLinearColor LabelColor, FLinearColor WireframeColor, const FText& LabelText, bool bIsTrainingActor);
	};
}	// namespace UE::MLDeformer

#undef UE_API
