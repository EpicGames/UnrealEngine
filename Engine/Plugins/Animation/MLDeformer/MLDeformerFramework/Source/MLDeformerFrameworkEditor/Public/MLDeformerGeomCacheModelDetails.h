// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerModelDetails.h"
#include "IDetailCustomization.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class IDetailLayoutBuilder;
class UMLDeformerGeomCacheModel;
class UGeometryCache;
class USkeletalMesh;

namespace UE::MLDeformer
{
	class FMLDeformerGeomCacheEditorModel;
	class FMLDeformerEditorModel;

	/**
	 * The detail customization of a geometry cache based model.
	 * The model it customizes needs to be derived from UMLDeformerGeomCacheModel.
	 * It will automatically insert some error when the geom cache doesn't match the animation sequence etc.
	 * If your model inherits from a UMLDeformerGeomCacheModel then you can also inherit your model detail customization from this FMLDeformerGeomCacheModelDetails class.
	 */
	class FMLDeformerGeomCacheModelDetails
		: public FMLDeformerModelDetails
	{
	public:
		// FMLDeformerModelDetails overrides.
		UE_API virtual bool UpdateMemberPointers(const TArray<TWeakObjectPtr<UObject>>& Objects) override;
		UE_API virtual void AddTrainingInputAnims() override;
		// ~END FMLDeformerModelDetails overrides.

	protected:
		/** The geometry cache based runtime model. This value gets updated once UpdateMemberPointers gets called. */
		TObjectPtr<UMLDeformerGeomCacheModel> GeomCacheModel = nullptr;

		/** The geometry cache based editor model. This value gets updated once UpdateMemberPointers gets called. */
		FMLDeformerGeomCacheEditorModel* GeomCacheEditorModel = nullptr;
	};
}	// namespace UE::MLDeformer

#undef UE_API
