// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerSampler.h"
#include "MLDeformerModel.h"
#include "MLDeformerGeomCacheHelpers.h"
#include "UObject/ObjectPtr.h"
#include "GeometryCacheMeshData.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class UGeometryCacheComponent;
class UGeometryCache;

namespace UE::MLDeformer
{
	/**
	 * The input data sampler, which is used to sample vertex positions from geometry caches.
	 * It can then also calculate deltas between the sampled skeletal mesh data and geometry cache data.
	 */
	class FMLDeformerGeomCacheSampler
		: public FMLDeformerSampler
	{
	public:
		// FVertexDeltaSampler overrides.
		UE_API virtual void Init(FMLDeformerEditorModel* InModel, int32 InAnimIndex) override;
		UE_API virtual void Sample(int32 AnimFrameIndex) override;
		UE_API virtual float GetTimeAtFrame(int32 InAnimFrameIndex) const override;
		// ~END FVertexDeltaSampler overrides.

		/**
		 * Get the array of mesh names that we cannot do any sampling for.
		 * The reason for this can be that the sampler cannot find a matching mesh in the skeletal mesh for specific geometry cache tracks.
		 * For example if there is some geometry cache track that is named "Head", but there is no such mesh inside the skeletal mesh, then
		 * the returned array of names will include a string with the value "Head".
		 * @return An array of geometry cache track names for which no mesh could be found inside the SkeletalMesh / linear skinned actor.
		 */
		const TArray<FString>& GetFailedImportedMeshNames() const	{ return FailedImportedMeshNames; }

		UGeometryCacheComponent* GetGeometryCacheComponent()		{ return GeometryCacheComponent; }
		UGeometryCacheComponent* GetGeometryCacheComponent() const	{ return GeometryCacheComponent; }

		const TArray<FMLDeformerGeomCacheMeshMapping>& GetMeshMappings() const	{ return MeshMappings; }

	protected:
		/** The geometry cache component used to sample the geometry cache. */
		TObjectPtr<UGeometryCacheComponent> GeometryCacheComponent = nullptr;

		/** Maps skeletal meshes imported meshes to geometry tracks. */
		TArray<FMLDeformerGeomCacheMeshMapping> MeshMappings;

		/** The geometry cache mesh data reusable buffers. One for each MeshMapping.*/
		TArray<FGeometryCacheMeshData> GeomCacheMeshDatas;

		/** Geom cache track names for which no mesh can be found inside the skeletal mesh. */
		TArray<FString> FailedImportedMeshNames; 

		/** Imported mesh names in the skeletal mesh for which the geometry track had a different vertex count. */
		TArray<FString> VertexCountMisMatchNames;
	};
}	// namespace UE::MLDeformer

#undef UE_API
