// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
#include "AssetRegistry/AssetData.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class USkeleton;
class UMLDeformerGeomCacheModel;

namespace UE::MLDeformer
{
	/**
	 * The ML Deformer's training input animation (that uses a geometry cache) type customization.
	 * This is used to show a custom UI when displaying the FMLDeformerGeomCacheTrainingInputAnim struct.
	 * That struct is shown in the list of training input animations in the details panel.
	 * @see FMLDeformerGeomCacheTrainingInputAnim
	 */
	class FMLDeformerGeomCacheTrainingInputAnimCustomization
		: public IPropertyTypeCustomization
	{
	public:
		static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance();

		// IPropertyTypeCustomization overrides.
		UE_API virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
		UE_API virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
		// ~END IPropertyTypeCustomization overrides.

	private:
		UE_API bool FilterAnimSequences(const FAssetData& AssetData, USkeleton* Skeleton);
		UE_API UMLDeformerGeomCacheModel* FindMLDeformerModel(TSharedRef<IPropertyHandle> StructPropertyHandle) const;
	};
}	// namespace UE::MLDeformer

#undef UE_API
