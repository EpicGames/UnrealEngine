// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "MLDeformerTrainingModel.h"
#include "MLDeformerGeomCacheTrainingModel.generated.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

/**
 * The training model base class for geometry cache based models.
 * This class is our link to the Python training.
 */
UCLASS(MinimalAPI, Blueprintable)
class UMLDeformerGeomCacheTrainingModel
	: public UMLDeformerTrainingModel
{
	GENERATED_BODY()

public:
	// UMLDeformerTrainingModel overrides.
	UE_API virtual void Init(UE::MLDeformer::FMLDeformerEditorModel* InEditorModel) override;
	UE_API virtual bool SampleNextFrame() override;
	// ~END UMLDeformerTrainingModel overrides;

	/**
	 * Sample all input frames.
	 */
	UFUNCTION(BlueprintCallable, Category = "ML Deformer Training")
	UE_API bool GenerateBasicInputsAndOutputBuffers(const FString& InputsFilePath, const FString& OutputsFilePath);

protected:
	/**
	 * Find the next input animation to sample from.
	 * This is an index inside the training input animations list.
	 * This assumes the SampleAnimIndex member as starting point. The method does not modify this member directly, unless passed in as parameter.
	 * @param OutNextAnimIndex The next animation index to sample from when we take our next sample.
	 * @return Returns true when we found our next animation to sample. Returns false when we already sampled everything.
	 */
	UE_API virtual bool FindNextAnimToSample(int32& OutNextAnimIndex) const override;
};

#undef UE_API
