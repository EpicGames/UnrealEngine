// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerVizSettingsDetails.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class UMLDeformerGeomCacheModel;
class UMLDeformerGeomCacheVizSettings;

namespace UE::MLDeformer
{
	/**
	 * The visualization settings detail customization for a geometry cache based model.
	 * A geometry cache based model is one inherited from the UMLDeformerGeomCacheModel base class.
	 * You can inherit your own model detail customization from this class if you also inherited your model from the UMLDeformerGeomCacheModel class.
	 */
	class FMLDeformerGeomCacheVizSettingsDetails
		: public FMLDeformerVizSettingsDetails
	{
	public:
		// FMLDeformerVizSettingsDetails overrides.
		UE_API virtual bool UpdateMemberPointers(const TArray<TWeakObjectPtr<UObject>>& Objects) override;
		UE_API virtual void AddGroundTruth() override;
		// ~END FMLDeformerVizSettingsDetails overrides.

	protected:
		/** The geometry cache based runtime model pointer. This is updated once UpdateMemberPointers is called. */
		TObjectPtr<UMLDeformerGeomCacheModel> GeomCacheModel = nullptr;

		/** A pointer to the geom cache based visualization settings. This is updated once UpdateMemberPointers is called. */
		TObjectPtr<UMLDeformerGeomCacheVizSettings> GeomCacheVizSettings = nullptr;
	};
}	// namespace UE::MLDeformer

#undef UE_API
