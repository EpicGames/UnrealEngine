// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IDetailCustomization.h"
#include "AssetRegistry/AssetData.h"
#include "Input/Reply.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class IDetailLayoutBuilder;
class USkeleton;
class IDetailCategoryBuilder;
class UMLDeformerModel;
class UGeometryCache;
class USkeletalMesh;

namespace UE::MLDeformer
{
	class FMLDeformerEditorModel;

	/**
	 * The ML Deformer model detail customization base class.
	 * This adds the shared properties, inserts error messages, creates some groups, etc.
	 * Model detail customizations should all inherit from this class. If you use a geometry cache based model however, you will 
	 * likely inherit from other classes such as FMLDeformerGeomCacheModelDetails or FMLDeformerMorphModelDetails. 
	 * Those classes are also inherited from this one though.
	 */
	class FMLDeformerModelDetails
		: public IDetailCustomization
	{
	public:
		// ILayoutDetails overrides.
		UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
		// ~END ILayoutDetails overrides.

		/**
		 * Update the class member pointers, which includes the pointer to the model, and its editor model.
		 * @param Objects The array of objects that the detail customization is showing.
		 */
		UE_API virtual bool UpdateMemberPointers(const TArray<TWeakObjectPtr<UObject>>& Objects);

		/**
		 * Create the categories that we can add properties to. This will update the category member pointers. */
		UE_API virtual void CreateCategories();

		/** Add additional errors related to the base mesh. */
		virtual void AddBaseMeshErrors() {}

		/** Add additional errors related to bone inputs. */
		virtual void AddBoneInputErrors() {}

		/** Add additional errors related to curve inputs. */
		virtual void AddCurveInputErrors() {}

		/** Add the property that shows the input animations. */
		UE_API virtual void AddTrainingInputAnims();

		/** Add training input flags, which basically are things like the check boxes that specify whether bones or curves (or both) should be included. */
		virtual void AddTrainingInputFlags() {}

		/** Add additional training input errors. */
		virtual void AddTrainingInputErrors() {}

		/** Add training settings errors. */
		virtual void AddTrainingSettingsErrors() {}

	protected:
		/** The filter that only shows anim sequences that are compatible with the given skeleton. */
		UE_API bool FilterAnimSequences(const FAssetData& AssetData, USkeleton* Skeleton);

	protected:
		/** Associated detail layout builder. */
		IDetailLayoutBuilder* DetailLayoutBuilder = nullptr;

		/** A pointer to the model we are customizing details for. */
		TObjectPtr<UMLDeformerModel> Model = nullptr;

		/** A pointer to the editor model of the runtime model. */
		FMLDeformerEditorModel* EditorModel = nullptr;

		/** The category related to the base mesh. */
		IDetailCategoryBuilder* BaseMeshCategoryBuilder = nullptr;

		/** The category related to the target mesh. */
		IDetailCategoryBuilder* TargetMeshCategoryBuilder_DEPRECATED = nullptr;

		/** The category related to the inputs and outputs. */
		IDetailCategoryBuilder* InputOutputCategoryBuilder = nullptr;

		/** The training settings category. You most likely add your model properties to this. */
		IDetailCategoryBuilder* TrainingSettingsCategoryBuilder = nullptr;

		/** The LOD generation category. This category can be hidden, based on whether the model supports LOD or not. */
		IDetailCategoryBuilder* LODSettingsCategoryBuilder = nullptr;
	};
}	// namespace UE::MLDeformer

#undef UE_API
