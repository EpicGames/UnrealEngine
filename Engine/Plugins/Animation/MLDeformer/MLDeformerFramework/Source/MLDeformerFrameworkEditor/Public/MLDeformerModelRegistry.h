// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class UMLDeformerModel;


namespace UE::MLDeformer
{
	class FMLDeformerEditorModel;

	DECLARE_DELEGATE_RetVal(FMLDeformerEditorModel*, FOnGetEditorModelInstance);

	/**
	 * The editor model registry, which keeps track of different types of runtime ML Deformer models and their related editor models.
	 * When you create a new ML Deformer model type, you register it to this registry using the RegisterEditorModel method.
	 * You typically do this in your editor module's StartupModule method. And you unregister it again in the editor module's ShutdownModule method.
	 * Models can have priorities. The one model with the highest priority level will be the model that is created by default when creating a new ML Deformer asset.
	 */
	class FMLDeformerEditorModelRegistry
	{
		friend class FMLDeformerEditorToolkit;
		friend class FMLDeformerEditorModel;

	public:
		UE_API ~FMLDeformerEditorModelRegistry();

		/**
		 * Register a new model. This is needed to make the model appear inside the UI of the editor as selectable model.
		 * Call this method in your editor module's StartupModule call.
		 * @param ModelType The type of the runtime model, for example UNeuralMorphModel::StaticClass().
		 * @param Delegate The function that creates the editor model that's related to this editor model, 
		 *        for example FOnGetEditorModelInstance::CreateStatic(&FNeuralMorphEditorModel::MakeInstance). This MakeInstance function must return the new editor model associated with this model.
		 * @param ModelPriority The priority level of this model. The model with the highest priority is created by default when creating a new ML Deformer Asset.
		 *        If there is a model with the same priority level already, and it happens to be the highest priority model, then this new model will take over as priority model.
		 */
		UE_API void RegisterEditorModel(UClass* ModelType, FOnGetEditorModelInstance Delegate, int32 ModelPriority = 0);

		/**
		 * Unregister the editor model.
		 * Call this method in your editor module's ShutdownModule call.
		 * @param ModelType The model type to unregister, for example UNeuralMorphModel::StaticClass().
		 */
		UE_API void UnregisterEditorModel(const UClass* ModelType);

		/**
		 * Get the editor model related to a given runtime model. 
		 * @param Model The runtime model to get the editor model object for.
		 * @return A pointer to the editor model object for this specific runtime model instance, or nullptr when not found.
		 */
		UE_API FMLDeformerEditorModel* GetEditorModel(UMLDeformerModel* Model);

		/**
		 * Get the number of registered models.
		 * @return The number of models that have been registered through RegisterEditorModel.
		 */
		UE_API int32 GetNumRegisteredModels() const;

		/**
		 * Get the number of instanced models.
		 * @return The number of instanced models.
		 */
		UE_API int32 GetNumInstancedModels() const;

		/**
		 * Get the highest priority runtime model class.
		 * The model with the highest priority is created by default when creating a new ML Deformer Asset.
		 * @return Returns the model class of the highest priority model, or nullptr when there are none.
		 */
		UE_API UClass* GetHighestPriorityModel() const;

		/**
		 * Get the priority level for a given runtime model type.
		 * @param ModelType The runtime model type, for example UNeuralMorphModel::StaticClass().
		 * @return The model priority level, or if it cannot be found it will return MIN_int32.
		 */
		UE_API int32 GetPriorityForModel(UClass* ModelType) const;

		/**
		 * Get the read-only map of registered models.
		 * @return The map of registered models that have been registered through RegisterEditorModel.
		 */
		UE_API const TMap<UClass*, FOnGetEditorModelInstance>& GetRegisteredModels() const;

		/**
		 * Get the read-only map that maps instanced models with their editor models.
		 * @return The map that provides the editor model for a specific runtime model.
		 */
		UE_API const TMap<UMLDeformerModel*, FMLDeformerEditorModel*>& GetModelInstances() const;

		/**
		 * Get the highest priority model index into the map.
		 * It is the 'index'-th item in the array that you can iterate by doing something like: for (auto& Item : Map).
		 * @return The highest priority model index.
		 */
		UE_API int32 GetHighestPriorityModelIndex() const;

	private:
		/**
		 * Update the highest priority model by looking at the priority map.
		 * This updates the value returned by GetHighestPriorityModel().
		 */
		UE_API void UpdateHighestPriorityModel();

		/**
		 * Remove a given editor model instance.
		 * @param EditorModel A pointer to the editor model to remove.
		 */
		UE_API void RemoveEditorModelInstance(FMLDeformerEditorModel* EditorModel);

		/**
		 * Create a new editor model for a given runtime model.
		 * @param Model The runtime model object to create an editor model for.
		 * @return A pointer to the newly created editor model.
		 */
		UE_API FMLDeformerEditorModel* CreateEditorModel(UMLDeformerModel* Model);

	private:
		/** A map that maps runtime model types with a function that creates its editor model. */
		TMap<UClass*, FOnGetEditorModelInstance> Map;

		/** A map of runtime model instances, and the related editor models. */
		TMap<UMLDeformerModel*, FMLDeformerEditorModel*> InstanceMap;

		/** The map of model type priority levels. */
		TMap<UClass*, int32> ModelPriorities;

		/** The registered model type that has the highest priority level. */
		UClass* HighestPriorityModelType = nullptr;
	};
}	// namespace UE::MLDeformer

#undef UE_API
