// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerGeomCacheModelDetails.h"
#include "IDetailCustomization.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class IDetailLayoutBuilder;
class UMLDeformerMorphModel;

namespace UE::MLDeformer
{
	class FMLDeformerMorphModelEditorModel;

	/**
	 * The detail customization for models inherited from the UMLDeformerMorphModel class.
	 * You can inherit the detail customization for your own model from this class if your model inherited from the UMLDeformerMorphModel class.
	 */
	class FMLDeformerMorphModelDetails
		: public FMLDeformerGeomCacheModelDetails
	{
	public:
		// ILayoutDetails overrides.
		UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
		// ~END ILayoutDetails overrides.

		// FMLDeformerModelDetails overrides.
		UE_API virtual void CreateCategories() override;
		UE_API virtual bool UpdateMemberPointers(const TArray<TWeakObjectPtr<UObject>>& Objects) override;
		// ~END FMLDeformerModelDetails overrides.

	protected:
		/**
		 * Check whether we should show an error in the UI about possible shading artifacts.
		 * The ML Deformer cannot run properly if all of the following things are true:
		 * - Skin cache is disabled
		 * - Material settings have "Use with morph targets" disabled.
		 * - No deformer graph is being used.
		 * 
		 * This method checks those points.
		 * If not, we can show a warning in the UI.
		 * 
		 * @return Returns true if an error should be displayed in the UI about this issue.
		 */
		UE_API bool ShouldShowShadingError() const;

	protected:
		/** A pointer to the morph model. This is updated when UpdateMemberPointers is called. */
		TObjectPtr<UMLDeformerMorphModel> MorphModel = nullptr;

		/** A pointer to the editor model for the morph model. This is updated when UpdateMemberPointers is called. */
		FMLDeformerMorphModelEditorModel* MorphModelEditorModel = nullptr;

		/** The morph settings category. */
		IDetailCategoryBuilder* MorphTargetCategoryBuilder = nullptr;
	};
}	// namespace UE::MLDeformer

#undef UE_API
