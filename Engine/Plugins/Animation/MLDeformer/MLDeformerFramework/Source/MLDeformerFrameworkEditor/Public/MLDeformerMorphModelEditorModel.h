// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerGeomCacheEditorModel.h"
#include "MLDeformerGeomCacheSampler.h"
#include "MLDeformerEditorActor.h"
#include "MLDeformerMasking.h"
#include "Rendering/ColorVertexBuffer.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class UMLDeformerMorphModel;
class UMLDeformerMorphModelVizSettings;

namespace UE::MLDeformer
{
	/**
	 * The editor model related to the runtime UMLDeformerMorphModel class, or models inherited from that.
	 */
	class FMLDeformerMorphModelEditorModel
		: public FMLDeformerGeomCacheEditorModel
	{
	public:
		// We need to implement this static MakeInstance method.
		static UE_API FMLDeformerEditorModel* MakeInstance();

		// FGCObject overrides.
		virtual FString GetReferencerName() const override { return TEXT("FMLDeformerMorphModelEditorModel"); }
		// ~END FGCObject overrides.

		// FMLDeformerEditorModel overrides.
		UE_API virtual void CopyBaseSettingsFromModel(const FMLDeformerEditorModel* SourceEditorModel) override;
		UE_API virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
		UE_API virtual void OnPropertyChanged(FPropertyChangedEvent& PropertyChangedEvent) override;
		UE_API virtual FString GetHeatMapDeformerGraphPath() const override;
		UE_API virtual FString GetHeatMapDeformerGraphDualQuatPath() const override;
		UE_API virtual void OnPreTraining() override;
		UE_API virtual void OnPostTraining(ETrainingResult TrainingResult, bool bUsePartiallyTrainedWhenAborted) override;
		UE_API virtual void OnMaxNumLODsChanged() override;
		UE_API virtual void OnObjectModified(UObject* Object) override;
		// ~END FMLDeformerEditorModel overrides.

		/**
		 * Returns whether this model supports morph output weight clamping or not.
		 * This is used to make sure the morph target weights are within valid ranges.
		 * Some models might choose not to do this, because they might already clamp input values.
		 * When this returns false, the "Clamp morph weights" option in the UI will also be hidden.
		 * @return Returns true when the model supports morph weight clamping, otherwise false is returned.
		 */
		virtual bool IsMorphWeightClampingSupported() const		{ return true; }

		/**
		 * Get the mask buffer for a given morph target.
		 * @param MorphTargetIndex The morph target index, excluding the first 'means' morph target that we always have, so 0 would mean the actual first real morph target.
		 */
		UE_API virtual const TArrayView<const float> GetMaskForMorphTarget(int32 MorphTargetIndex) const;

		/**
		 * Check if input masking is supported in this model.
		 * This allows bones and curves to specify a given mask. These masks can be used during training.
		 * On default this is disabled.
		 * @return Returns true if supported, or false otherwise.
		 */
		UE_API virtual bool IsInputMaskingSupported() const;

		/**
		 * Fill the mask buffer for a given item with a specific value.
		 * An item can be a bone, curve, bone group or curve group.
		 * The number of values written is always Model->GetNumBaseMeshVerts().
		 * @param MaskBuffer The mask buffer to write to. The mask buffer view has Model->GetBaseNumVerts() elements.
		 * @param Value The value to write for all vertices.
		 */
		UE_API void FillMaskValues(TArrayView<float> ItemMaskBuffer, float Value) const;

		/**
		 * Add bone influences to the mask.
		 * Every vertex that is linked to the specified bone will write its skinning weight to the mask buffer.
		 * @param SkeletonBoneIndex The bone index inside our reference skeleton.
		 * @param MaskBuffer The mask buffer to write to.
		 */
		UE_API void ApplyBoneToMask(int32 SkeletonBoneIndex, TArrayView<float> MaskBuffer);

		/**
		 * Recursively add bone masks for this bone and its parents.
		 * The number of parents is controlled by the MaxHierarchyDepth.
		 * @param RefSkel The reference skeleton.
		 * @param SkeletonBoneIndex The bone index of our current bone to add to the mask.
		 * @param MaxHierarchyDepth The maximum hierarchy depth. If this is 1, it means that we visit only up to the parent of the specified bone. If it is 2, we also visit the parent of the parent, etc.
		 * @param OutBoneAdded The list of bones that we already added to the mask. Bones that are in this list will not be added to the mask again.
		 * @param CurHierarchyDepth The current recursive hierarchy depth. This is used to track how deep we are when moving up the hierarchy. A value of 1 means we are currently adding the parent of the initial bone.
		 */
		UE_API void RecursiveAddBoneToMaskUpwards(const FReferenceSkeleton& RefSkel, int32 SkeletonBoneIndex, int32 MaxHierarchyDepth, TArray<int32>& OutBonesAdded, int32 CurHierarchyDepth = 0);

		/**
		 * Recursively add bone masks for this bone and its children.
		 * The number of child steps down the hierarchy is controlled by the MaxHierarchyDepth.
		 * @param RefSkel The reference skeleton.
		 * @param SkeletonBoneIndex The bone index of our current bone to add to the mask.
		 * @param MaxHierarchyDepth The maximum hierarchy depth. If this is 1, it means that we visit only down to the children of the specified bone. If it is 2, we also visit the children of the children, etc.
		 * @param OutBoneAdded The list of bones that we already added to the mask. Bones that are in this list will not be added to the mask again.
		 * @param CurHierarchyDepth The current recursive hierarchy depth. This is used to track how deep we are when moving down the hierarchy. A value of 1 means we are currently adding a child node of the initial bone.
		 */
		UE_API void RecursiveAddBoneToMaskDownwards(const FReferenceSkeleton& RefSkel, int32 SkeletonBoneIndex, int32 MaxHierarchyDepth, TArray<int32>& OutBonesAdded, int32 CurHierarchyDepth = 0);

		/**
		 * Apply the mask info to the mask buffer of floats.
		 * @param SkeletalMesh The skeletal mesh that the mask relates to.
		 * @param MaskInfo The mask info object to apply to the float mask buffer.
		 * @param ItemMaskBuffer The mask buffer for this item. This should be an array view of size NumBaseMeshVerts.
		 */
		UE_API void ApplyMaskInfoToBuffer(const USkeletalMesh* SkeletalMesh, const FMLDeformerMaskInfo& MaskInfo, TArrayView<float> ItemMaskBuffer);

		/**
		 * Apply a generated mask to a specific vertex attribute.
		 * This copy the generated values one to one to the vertex attribute, but it applies some scaling to the attributes to give better results.
		 * If we don't do this scaling then the values often look too small and the vertex attribute visually looks very light, rather than clearly defining the area.
		 * This will also ensure the mask gets generated, even if the MaskInfo masking mode is set to something non-generated.
		 * @param SkeletalMesh The skeletal mesh that the vertex attribute is on.
		 * @param MaskInfo The mask info object that we generate the mask from.
		 * @param AttributeRef The output vertex attributes to write to.
		 */
		UE_API void ApplyGeneratedMaskToVertexAttributes(USkeletalMesh* SkeletalMesh, FMLDeformerMaskInfo& MaskInfo, TVertexAttributesRef<float> AttributeRef);

		// Helpers.
		UE_API UMLDeformerMorphModel* GetMorphModel() const;
		UE_API UMLDeformerMorphModelVizSettings* GetMorphModelVizSettings() const;

	protected:
		UE_API void TransferMorphTargets(TArray<UMorphTarget*> MorphTargetsLODZero);

		/**
		 * Initialize a set of engine morph targets and compress them to GPU friendly buffers.
		 * These morph targets are initialized from a set of deltas. Each morph target needs to have Model->GetNumBaseVerts() number of deltas.
		 * All deltas are concatenated in one big array. So all deltas of all vertices for the second morph target are appended to the deltas for
		 * the first morph target, etc. In other words, the layout is: [morph0_deltas][morph1_deltas][morph2_deltas][...].
		 * @param Deltas The deltas for all morph targets concatenated. So the number of items in this array is a multiple of Model->GetNumBaseVerts().
		 */
		UE_API virtual void InitEngineMorphTargets(const TArray<FVector3f>& Deltas);

		/**
		 * Clamp the current morph target number inside the visualization settings to a valid range.
		 * This won't let the number of the morph target that we are visualizing go above the total number of morph targets.
		 */
		UE_API void ClampMorphTargetNumber();

		/**
		 * Update the morph target error values. This assigns an error value for each morph target.
		 * Internally this calls SetMorphTargetsErrorOrder to set the actual order and errors.
		 * @param MorphTargets The morph targets to calculate the errors and order for.
		 */
		UE_API void UpdateMorphErrorValues(TArrayView<UMorphTarget*> MorphTargets);

		/**
		 * Calculate the delta normals for a given morph target.
		 * @param LOD The LOD level.
		 * @param SkelMesh The skeletal mesh to get the mesh data from.
		 * @param MorphTargetIndex The morph target to calculate the normals for.
		 * @param Deltas The per vertex deltas. The number of elements in this array is NumMorphTargets * NumImportedVertices.
		 * @param BaseVertexPositions The positions of the base/neutral mesh, basically the unskinned vertices.
		 * @param BaseNormals The normals of the neutral base mesh.
		 * @param ImportedVertexToRenderVertexMapping The array of size 'NumImportedVerts' (NumBaseMeshVerts) that maps to a render vertex.
		 * @param ColorBuffer The color buffer to use when calculating the global mask.
		 * @param MaskChannel The global mask channel, for example which color channel to use from the color buffer.
		 * @param bInvertGlobalMask Set to true when you want the global mask to be inverted.
		 * @param GlobalMaskWeights The array of weights, one for each render vertex.
		 * @param OutDeltaNormals The array that we will write the generated normals to. This will automatically be resized by this method.
		 */
		UE_API virtual void CalcMorphTargetNormals(
			int32 LOD,
			const USkeletalMesh* SkelMesh,
			int32 MorphTargetIndex,
			const TArrayView<const FVector3f> Deltas,
			const TArrayView<const FVector3f> BaseVertexPositions,
			const TArrayView<const FVector3f> BaseNormals,
			const TArrayView<const int32> ImportedVertexToRenderVertexMapping,
			const FColorVertexBuffer& ColorBuffer,
			EMLDeformerMaskChannel MaskChannel,
			bool bInvertGlobalMaskChannel,
			const TArray<float>& GlobalMaskWeights,
			TArray<FVector3f>& OutDeltaNormals);

		/**
		 * Debug draw specific morph targets using lines and points.
		 * This can show the user what deltas are included in which morph target.
		 * @param PDI A pointer to the draw interface.
		 * @param MorphDeltas A buffer of deltas for ALL morph targets. The size of the buffer must be a multiple of Model->GetBaseNumVerts().
		 *        So the layout of this buffer is [Morph0_Deltas][Morph1_Deltas][Morph2_Deltas] etc.
		 * @param DeltaThreshold Deltas with a length  larger or equal to the given threshold value will be colored differently than the ones smaller than this threshold.
		 * @param MorphTargetIndex The morph target number to visualize.
		 * @param DrawOffset An offset to perform the debug draw at.
		 */
		UE_API void DebugDrawMorphTarget(FPrimitiveDrawInterface* PDI, const TArray<FVector3f>& MorphDeltas, float DeltaThreshold, int32 MorphTargetIndex, const FVector& DrawOffset);

		/** Zero all deltas with a length equal to, or smaller than the threshold value. */
		UE_API void ZeroDeltasByLengthThreshold(TArray<FVector3f>& Deltas, float Threshold);

		/**
		 * Generate engine morph targets from a set of deltas.
		 * @param OutMorphTargets The output array with generated morph targets. This array will be reset, and then filled with generated morph targets.
		 * @param Deltas The per vertex deltas for all morph targets, as one big buffer. Each morph target has 'GetNumBaseMeshVerts()' number of deltas.
		 * @param NamePrefix The morph target name prefix. If set to "MorphTarget_" the names will be "MorphTarget_000", "MorphTarget_001", "MorphTarget_002", etc.
		 * @param LOD The LOD index to generate the morphs for.
		 * @param DeltaThreshold Only include deltas with a length larger than this threshold in the morph targets.
		 * @param bIncludeNormals Include normals inside the morph targets? This can be an alternative to recalculating normals at the end, although setting this to true and not 
		 *        recomputing normals can lead to lower quality results, in trade for faster performance.
		 * @param MaskChannel The weight mask mode, which specifies what channel to get the weight data from. Such channel allows the user to define what areas the deformer should for example not be active in.
		 * @param bInvertMaskChannel Specifies whether the weight mask should be inverted or not.
		 * @param MaskBuffer An optional mask buffer that contains 'Model->GetNumBaseMeshVerts() * (MorphModel->GetNumMorphTargets() - 1)' number of floats. Deltas will be multiplied by this value. 
		 *        When the mask buffer is an empty array, it will be ignored.
		 */
		UE_API void CreateMorphTargets(
			TArray<UMorphTarget*>& OutMorphTargets,
			const TArray<FVector3f>& Deltas,
			const FString& NamePrefix = TEXT("MorphTarget_"),
			int32 LOD = 0,
			float DeltaThreshold = 0.01f,
			bool bIncludeNormals=false,
			const EMLDeformerMaskChannel MaskChannel = EMLDeformerMaskChannel::Disabled,
			bool bInvertMaskChannel = false);

		/** 
		 * Compress morph targets into GPU based morph buffers.
		 * @param OutMorphBuffers The output compressed GPU based morph buffers. If this buffer is already initialized it will be released first.
		 * @param MorphTargets The morph targets to compress into GPU friendly buffers.
		 * @param LOD The LOD index to generate the morphs for.
		 * @param MorphErrorTolerance The error tolerance for the delta compression, in cm. Higher values compress better but can result in artifacts.
		 */
		UE_API void CompressMorphTargets(FMorphTargetVertexInfoBuffers& OutMorphBuffers, const TArray<UMorphTarget*>& MorphTargets, int32 LOD = 0, float MorphErrorTolerance = 0.01f);

		/**
		 * Calculate the normals for each vertex, given the triangle data and positions.
		 * It computes this by summing up the face normals for each vertex using that face, and normalizing them at the end.
		 * @param VertexPositions The buffer with vertex positions. This is the size of the number of imported vertices.
		 * @param IndexArray The index buffer, which contains NumTriangles * 3 number of integers.
		 * @param VertexMap For each render vertex, an imported vertex number. For example, for a cube these indices go from 0..7.
		 * @param OutNormals The array that will contain the normals. This will automatically be resized internally by this method.
		 */
		UE_API void CalcVertexNormals(TArrayView<const FVector3f> VertexPositions, TArrayView<const uint32> IndexArray, TArrayView<const int32> VertexMap, TArray<FVector3f>& OutNormals) const;

		/**
		 * Process and filter a vertex delta position and normal and apply scaling to it.
		 * The scaling of these deltas is based on a given morph mask weight, a global mask weight, and a delta zero threshold, which filters out small deltas entirely.
		 * @param OutScaledDelta The scaled version of the delta position.
		 * @param OutScaledDeltaNormal The scaled version of the delta normal.
		 * @param RawDelta The unscaled raw delta position.
		 * @param RawDeltaNormal The unscaled raw delta normal.
		 * @param DeltaThreshold Any scaled vertex delta position vector with a length smaller or equal to this value, will cause this method to return false. This basically can be used to filter out small deltas.
		 * @param MorphMaskWeight The weight value of the morph target mask for this vertex. This can be values larger than 1 as well, but always larger or equal to 0.
		 * @param GlobalMaskWeight The global weight value for this vertex. This global mask / weight is used to disable the deformer in specific regions on the mesh, for example where the head or neck connects with the body.
		 * @return Returns true if this vertex should be included in the morph target, or false if it has been filtered out based on the provided delta threshold.
		 */
		UE_API bool ProcessVertexDelta(FVector3f& OutScaledDelta, FVector3f& OutScaledDeltaNormal, const FVector3f RawDelta, const FVector3f RawDeltaNormal, float DeltaThreshold, float MorphMaskWeight, float GlobalMaskWeight) const;

		/**
		 * Calculate the global mask weight for a specific render vertex, using a color buffer and specified mask channel settings.
		 * @param VertexMap The render vertex to import vertex map.
		 * @param ColorBuffer The color buffer which we can grab color values from, which are used to calculate the mask weight.
		 * @param MaskChannel The channel to grab the weight value from. This specifieds whether the mask is enabled and if so, from what channel (r, g, b, a) we grab the weight value.
		 * @param bInvertMaskChannel Set to true when we should invert the weight value.
		 * @return An array with a weight per render vertex. A value of 1.0 is returned when the mask channel is set to disable the mask. The return value is always between 0 and 1.
		 */
		UE_API TArray<float> CalcGlobalMaskWeights(const TArray<int32>& VertexMap, const FColorVertexBuffer& ColorBuffer, EMLDeformerMaskChannel MaskChannel, bool bInvertMaskChannel) const;

		/**
		 * Tell the editor model that we should update the mask buffer on the next Tick call.
		 */
		void SetNeedsRebuildMaskBuffer() { bNeedsRebuildMaskInfo = true; }

	protected:
		/**
		 * The entire set of morph target deltas, 3 per vertex, for each morph target, as one flattened buffer.
		 * So the size of this buffer is: (NumVertsPerMorphTarget * NumMorphTargets).
		 */
		TArray<FVector3f> MorphTargetDeltasBackup;

		/** The backup of the minimum and maximum weights for each morph target. */
		TArray<FFloatInterval> MorphTargetsMinMaxWeightsBackup;

		/** The backup of the input item mask buffer. This contains all the bone and bone group masks. */
		TArray<float> InputItemMaskBufferBackup;

		/** Indicates whether we need to rebuild the mask buffer next Tick or not. */
		bool bNeedsRebuildMaskInfo = false;
	};
}	// namespace UE::MLDeformer

#undef UE_API
