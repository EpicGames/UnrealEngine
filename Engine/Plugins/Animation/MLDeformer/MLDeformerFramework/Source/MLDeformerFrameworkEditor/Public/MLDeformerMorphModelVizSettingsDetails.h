// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerGeomCacheVizSettingsDetails.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class UMLDeformerMorphModel;
class UMLDeformerMorphModelVizSettings;

namespace UE::MLDeformer
{
	/**
	 * The visualization settings for models inherited from the UMLDeformerMorphModel class.
	 */
	class FMLDeformerMorphModelVizSettingsDetails
		: public FMLDeformerGeomCacheVizSettingsDetails
	{
	public:
		// FMLDeformerVizSettingsDetails overrides.
		UE_API virtual bool UpdateMemberPointers(const TArray<TWeakObjectPtr<UObject>>& Objects) override;
		UE_API virtual void AddAdditionalSettings() override;
		// ~END FMLDeformerVizSettingsDetails overrides.

	protected:
		/** Is the morph target visualization option/checkbox enabled? */
		UE_API bool IsMorphTargetsEnabled() const;

		/** A pointer to the runtime morph model. This is updated when UpdateMemberPointers is called. */
		TObjectPtr<UMLDeformerMorphModel> MorphModel = nullptr;

		/** A pointer to the morph model visualization settings. This is updated when UpdateMemberPointers is called. */
		TObjectPtr<UMLDeformerMorphModelVizSettings> MorphModelVizSettings = nullptr;
	};
}	// namespace UE::MLDeformer

#undef UE_API
