// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EdMode.h"
#include "MLDeformerPaintMode.generated.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class USkeletalMeshComponent;

namespace UE::MLDeformer
{
	class FMLDeformerEditorToolkit;
}

UCLASS(MinimalAPI)
class UMLDeformerPaintMode
	: public UBaseLegacyWidgetEdMode
{
	GENERATED_BODY()

public:
	UE_API const static FEditorModeID Id;	

	UE_API UMLDeformerPaintMode();

	// UEdMode overrides
	UE_API virtual void Enter() override;
	UE_API virtual void CreateToolkit() override;
	virtual bool UsesTransformWidget() const override	{ return false; };
	virtual bool UsesPropertyWidgets() const override	{ return false; };
	virtual bool UsesToolkits() const override			{ return true; }
	virtual bool RequiresLegacyViewportInteractions() const override	{ return false; }
	// ~END UEdMode overrides

	UE_API void SetMLDeformerEditor(UE::MLDeformer::FMLDeformerEditorToolkit* Editor);
	UE_API void UpdatePose(USkeletalMeshComponent* SkeletalMeshComponent, bool bFullUpdate=true);

private:
	TArray<FVector3f> BindPosePositions;
};

#undef UE_API
