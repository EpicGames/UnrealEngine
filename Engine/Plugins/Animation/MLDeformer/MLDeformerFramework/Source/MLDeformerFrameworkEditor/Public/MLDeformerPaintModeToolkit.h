// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "StatusBarSubsystem.h"
#include "Toolkits/BaseToolkit.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class IToolkitHost;
class STextBlock;
class SButton;

namespace UE::MLDeformer
{
	class FMLDeformerEditorToolkit;

	class FMLDeformerPaintModeToolkit
		: public FModeToolkit
	{
	public:
		UE_API virtual ~FMLDeformerPaintModeToolkit() override;
	
		// IToolkit overrides
		UE_API virtual void Init(const TSharedPtr<IToolkitHost>& InToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode) override;
		UE_API virtual FName GetToolkitFName() const override;
		UE_API virtual FText GetBaseToolkitName() const override;
		virtual TSharedPtr<SWidget> GetInlineContent() const override			{ return ToolkitWidget; }

		// FModeToolkit overrides
		UE_API virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
		UE_API virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
		UE_API virtual FText GetActiveToolDisplayName() const override;
		UE_API virtual FText GetActiveToolMessage() const override;

		void SetMLDeformerEditor(FMLDeformerEditorToolkit* Editor)				{ MLDeformerEditor = Editor; }
		UE::MLDeformer::FMLDeformerEditorToolkit* GetMLDeformerEditor() const	{ return MLDeformerEditor; }

	private:
		UE_API void PostNotification(const FText& Message);
		UE_API void ClearNotification();

		UE_API void PostWarning(const FText& Message);
		UE_API void ClearWarning();

		UE_API void UpdateActiveToolProperties(UInteractiveTool* Tool);

		UE_API void RegisterPalettes();
		FDelegateHandle ActivePaletteChangedHandle;

		UE_API void MakeToolAcceptCancelWidget();

		FText ActiveToolName;
		FText ActiveToolMessage;
		FStatusBarMessageHandle ActiveToolMessageHandle;

		TSharedPtr<SWidget> ToolkitWidget;

		FMLDeformerEditorToolkit* MLDeformerEditor = nullptr;

		TSharedPtr<SWidget> ViewportOverlayWidget;
		const FSlateBrush* ActiveToolIcon = nullptr;
	
		TSharedPtr<STextBlock> ModeWarningArea;
		TSharedPtr<STextBlock> ModeHeaderArea;
		TSharedPtr<STextBlock> ToolWarningArea;
	};
} // namespace UE::MLDeformer

#undef UE_API
