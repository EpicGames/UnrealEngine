// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MLDeformerModel.h"
#include "UObject/ObjectPtr.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class UDebugSkelMeshComponent;
class UGeometryCacheComponent;
class UWorld;
class FSkeletalMeshLODRenderData;
class FSkinWeightVertexBuffer;
class AActor;

namespace UE::MLDeformer
{
	class FMLDeformerEditorModel;

	/**
	 * The space in which the sampler will calculate its vertex deltas.
	 */
	enum class EVertexDeltaSpace : int8
	{
		/** Vertex deltas will be in pre-skinning space. We train on pre-skinning deltas. */
		PreSkinning,

		/** Vertex deltas will be in post-skinning space. Mostly used to visualize deltas in the viewport. */
		PostSkinning
	};

	/**
	 * The input data sampler.
	 * This class can sample bone rotations, curve values and vertex deltas.
	 * It does this by creating two temp actors, one with skeletal mesh component and one with geom cache component.
	 */
	class FMLDeformerSampler
	{
	public:
		UE_API virtual ~FMLDeformerSampler();

		/** Call Init whenever assets or other relevant settings change. */
		UE_API virtual void Init(FMLDeformerEditorModel* Model, int32 AnimIndex);

		/** Call this every time the frame changes. This will update all buffer contents. */
		UE_API virtual void Sample(int32 AnimFrameIndex);

		/** Get the time in seconds, at a given frame index. */
		virtual float GetTimeAtFrame(int32 InAnimFrameIndex) const		{ return 0.0f; }

		/** Get the skinned vertex positions of the skeletal mesh. */
		const TArray<FVector3f>& GetSkinnedVertexPositions() const		{ return SkinnedVertexPositions; }

		/** Get the unskinned vertex positions of the skeletal mesh. */
		const TArray<FVector3f>& GetUnskinnedVertexPositions() const	{ return UnskinnedVertexPositions; }

		/** Get the calculated vertex deltas. The space they will be in depends on the setup vertex deltas space. */
		const TArray<float>& GetVertexDeltas() const					{ return VertexDeltas; }

		/** 
		 * Get the current bone rotations that we can feed to a neural network. 
		 * The number of floats in the buffer equals to NumBones * 6.
		 * The six floats represent two columns of the 3x3 rotation matrix of the bone.
		 * @return The array of floats that we can pass to the neural network as inputs.
		 */
		const TArray<float>& GetBoneRotations() const					{ return BoneRotations; }

		/** Get the float values for the curves. One float per curve. */
		const TArray<float>& GetCurveValues() const						{ return CurveValues; }

		/** Get the number of imported vertices. */
		int32 GetNumImportedVertices() const							{ return NumImportedVertices; }

		/** Get the space in which we will calculate the vertex deltas. */
		EVertexDeltaSpace GetVertexDeltaSpace() const					{ return VertexDeltaSpace; }

		/** Set the space in which we should calculate the vertex deltas. */
		void SetVertexDeltaSpace(EVertexDeltaSpace DeltaSpace)			{ VertexDeltaSpace = DeltaSpace; }

		/** Get the skinning mode. */
		EMLDeformerSkinningMode GetSkinningMode() const					{ return SkinningMode; }

		/** Set the skinning mode. This defines the type of skinning we use, like Linear Blend Skinning or Dual Quaternion skinning. */
		void SetSkinningMode(const EMLDeformerSkinningMode Mode)		{ SkinningMode = Mode; }

		/** Have we already initialized this sampler? Is it ready for sampling? */
		bool IsInitialized() const										{ return SkelMeshActor.Get() != nullptr; }

		/** Get the number of bones. */
		UE_API int32 GetNumBones() const;

		/** Calculate the memory usage of the sampler. */
		UE_API SIZE_T CalcMemUsagePerFrameInBytes() const;
		UDebugSkelMeshComponent* GetSkeletalMeshComponent() { return SkeletalMeshComponent; }

		/**
		 * Set the number of floats used per curve. Values larger than 1 will cause padding with zeros. 
		 * The first float is always the value. Floats that come after that will contain zeros on default.
		 */
		void SetNumFloatsPerCurve(int32 NumFloats)						{ check(NumFloats > 0); NumFloatsPerCurve = NumFloats; }

		/** Get the number of floats per curve. */
		int32 GetNumFloatsPerCurve() const								{ return NumFloatsPerCurve; }

		/** Get the skinned positions for a given LOD level. */
		UE_API void ExtractSkinnedPositions(int32 LODIndex, TArray<FVector3f>& OutPositions);

	protected:
		/** Create the actors used for sampling. This creates two actors, one for the base skeletal mesh and one for the target mesh. */
		UE_API virtual void CreateActors();

		/** Extract the skinned vertex positions, of the skeletal mesh. */
		UE_API void ExtractSkinnedPositions(int32 LODIndex, TArray<FMatrix44f>& InBoneMatrices, TArray<FVector3f>& TempPositions, TArray<FVector3f>& OutPositions) const;

		/** Extract the unskinned vertex positions, of the skeletal mesh. */
		UE_API void ExtractUnskinnedPositions(int32 LODIndex, TArray<FVector3f>& OutPositions) const;

		/** Calculate the inverse skinning transform, which is used to convert from post-skinning to pre-skinning space. */
		UE_API FMatrix44f CalcInverseSkinningTransform(int32 VertexIndex, const FSkeletalMeshLODRenderData& SkelMeshLODData, const FSkinWeightVertexBuffer& SkinWeightBuffer) const;

		/** Calculate the delta using dual quaternions, taking a world space delta vector as input. */
		UE_API FVector3f CalcDualQuaternionDelta(int32 VertexIndex, const FVector3f& WorldDelta, const FSkeletalMeshLODRenderData& SkelMeshLODData, const FSkinWeightVertexBuffer& SkinWeightBuffer) const;

		/** Create a new actor with a specific name, in a specified world. */
		UE_API AActor* CreateNewActor(UWorld* InWorld, const FName& Name) const;

		/** Update the skeletal mesh component's play position, and refresh its bone transforms internally. */
		UE_API void UpdateSkeletalMeshComponent();

		/** Update the skinned vertex positions. */
		UE_API void UpdateSkinnedPositions();

		/** Update the bone rotation buffers. */
		UE_API void UpdateBoneRotations();

		/** Update the curve value buffers. */
		UE_API void UpdateCurveValues();

	protected:
		/** The vertex delta editor model used to sample. */
		FMLDeformerEditorModel* EditorModel = nullptr;

		/** The skeletal mesh actor used to sample the skinned vertex positions. */
		TObjectPtr<AActor> SkelMeshActor = nullptr;

		/** The actor used for the target mesh. */
		TObjectPtr<AActor> TargetMeshActor = nullptr;

		/** The vertex delta model associated with this sampler. */
		TObjectPtr<UMLDeformerModel> Model = nullptr;

		/** The skeletal mesh component used to sample skinned positions. */
		TObjectPtr<UDebugSkelMeshComponent>	SkeletalMeshComponent = nullptr;

		/** The skinned vertex positions. */
		TArray<FVector3f> SkinnedVertexPositions;

		/** The unskinned vertex positions. */
		TArray<FVector3f> UnskinnedVertexPositions;

		/** A temp array to store vertex positions in. */
		TArray<FVector3f> TempVertexPositions;

		/** The sampled bone matrices. */
		TArray<FMatrix44f> BoneMatrices;

		/**
		 * The vertex deltas as float buffer. The number of floats equals: NumImportedVerts * 3.
		 * The layout is (xyz)(xyz)(xyz)(...)
		 */
		TArray<float> VertexDeltas;	

		/**
		 * The bone rotation floats.
		 * The number of floats in the buffer equals to NumBones * 6.
		 * The six floats represent two columns of the 3x3 rotation matrix of the bone.
		 */
		TArray<float> BoneRotations;

		/** A float for each Skeleton animation curve. */
		TArray<float> CurveValues;	

		/** The current sample time, in seconds. */
		float SampleTime = 0.0f;

		/** The number of imported vertices of the skeletal mesh and geometry cache. This will be 8 for a cube. */
		int32 NumImportedVertices = 0;

		/** The animation frame we sampled the deltas for. */
		int32 AnimFrameIndex = -1;

		/** The animation index for which this is a sampler for. This is in range of 0..GetNumTrainingInputAnims(). */
		int32 AnimIndex = -1;

		/**
		 * The number of floats per curve value. If this is larger than 1, the remaining floats (after the first one) will be set to 0. 
		 * So basically we will pad the curve values.
		 * The CurveValues float array will contain something like : "0.75, 0.0, 0.0, 0.0" when this is set to 4.
		 */
		int32 NumFloatsPerCurve = 1;

		/** The vertex delta space (pre or post skinning) used when calculating the deltas. */
		EVertexDeltaSpace VertexDeltaSpace = EVertexDeltaSpace::PreSkinning;

		/** The skinning mode, which only works in combination with pre-skinning space. */
		EMLDeformerSkinningMode SkinningMode = EMLDeformerSkinningMode::Linear;
	};
}	// namespace UE::MLDeformer

#undef UE_API
