// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Misc/AutomationTest.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class UMLDeformerAsset;
class UMLDeformerModel;

namespace UE::MLDeformer
{
	class FMLDeformerEditorToolkit;
	class FMLDeformerEditorModelRegistry;

	/** A scoped ML Deformer editor, which auto closes the window on destruction. */
	class FMLDeformerScopedEditor
	{
	public:
		FMLDeformerScopedEditor() = delete;
		FMLDeformerScopedEditor(const FMLDeformerScopedEditor&) = delete;
		FMLDeformerScopedEditor(FMLDeformerScopedEditor&&) = delete;
		UE_API FMLDeformerScopedEditor(FMLDeformerEditorToolkit* InEditor);
		UE_API FMLDeformerScopedEditor(UMLDeformerAsset* Asset);
		UE_API FMLDeformerScopedEditor(const FString& AssetName);
		UE_API ~FMLDeformerScopedEditor();	// Auto close the editor window.
		UE_API bool IsValid() const;
		UE_API void SetCloseEditor(bool bCloseOnDestroy);
		UE_API FMLDeformerEditorToolkit* Get() const;
		FMLDeformerEditorToolkit* operator = (const FMLDeformerScopedEditor&) = delete;
		FMLDeformerEditorToolkit* operator = (const FMLDeformerScopedEditor&&) = delete;
		UE_API FMLDeformerEditorToolkit* operator ->() const;

	private:
		FMLDeformerEditorToolkit* Editor = nullptr;
		bool bCloseEditor = true;
	};

	/** Some helper functions for testing ML Deformer related things. */
	class FMLDeformerTestHelpers
	{
	public:
		/**
		 * Open the asset editor for a given ML Deformer asset.
		 * This will log an error when something doesn't succeed.
		 * @param Asset A pointer to the asset to open an editor for.
		 * @return A pointer to the editor toolkit, or nullptr when it didn't succeed.
		 */
		static UE_API FMLDeformerEditorToolkit* OpenAssetEditor(UMLDeformerAsset* Asset);

		/**
		 * Open the asset editor for a given ML Deformer asset, by its package name.
		 * This will log an error when something doesn't succeed.
		 * Examples of failure cases are when the package cannot be loaded, or if it doesn't contain a valid ML Deformer asset.
		 * @param Asset The package name, for example something like: "/MLDeformerFramework/Tests/MLD_Rampage.MLD_Rampage".
		 * @return A pointer to the editor toolkit, or nullptr when it didn't succeed.
		 */
		static UE_API FMLDeformerEditorToolkit* OpenAssetEditor(const FString& AssetName);

		/**
		 * Post an edit change event for a given property in the Model class.
		 * @param Model The model that the property has been changed in.
		 * @param PropertyName The name of the property (the class member name).
		 * @return Returns true if successfully emitted the PostEditChangeProperty call, or false if the property cannot be found.
		 */
		static UE_API bool PostModelPropertyChanged(UMLDeformerModel* Model, const FName PropertyName);
	};
}	// namespace UE::MLDeformer

#undef UE_API
