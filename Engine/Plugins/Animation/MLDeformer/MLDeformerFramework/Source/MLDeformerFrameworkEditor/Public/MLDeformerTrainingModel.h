// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "MLDeformerTrainingModel.generated.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class UMLDeformerModel;

namespace UE::MLDeformer
{
	class FMLDeformerEditorModel;

	// A helper to create the last derived object of a given type.
	// This is useful when we create a class in Python, derived from a given base class.
	template<class T>
	static T* NewDerivedObject()
	{
		TArray<UClass*> Classes;
		GetDerivedClasses(T::StaticClass(), Classes);
		if (Classes.IsEmpty())
		{
			return nullptr;
		}
		return NewObject<T>(GetTransientPackage(), Classes.Last());
	}
}

/**
 * The training model base class.
 * This class is used to interface with Python by providing some methods you can call inside your python training code.
 * For example it allows you to get all the sampled data, such as the deltas, bones and curve values.
 * When you create a new model you need to create a class inherited from this base class, and define a Train method inside it as follows:
 *
 * @code{.cpp}
 * // Doesn't need an implementation inside cpp, just a declaration in the header file.
 * UFUNCTION(BlueprintImplementableEvent, Category = "Training Model")
 * int32 Train() const;
 * @endcode
 * 
 * Now inside your Python class you do something like:
 * 
 * @code{.py}
 * @unreal.uclass()
 * class YourModelPythonTrainingModel(unreal.YourTrainingModel):
 *     @unreal.ufunction(override=True)
 *     def train(self):
 *         # ...do training here...
 *         return 0   # A value of 0 is success, 1 means aborted, see ETrainingResult.
 * @endcode
 * 
 * The editor will execute the Train method, which will trigger the "train" method in your Python class to be executed.
 * Keep in mind that in Unreal Engine all python code is lower case. So a "Train" method inside c++ will need to be called "train" inside the python code.
 * Or if you have something called "PerformMyTraining" it will need to be called "perform_my_training" inside Python.
 */
UCLASS(MinimalAPI, Blueprintable)
class UMLDeformerTrainingModel
	: public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Initialize the training model.
	 * This is automatically called by the editor.
	 * @param InEditorModel The pointer to the editor model that this is a training model for.
	 */
	UE_API virtual void Init(UE::MLDeformer::FMLDeformerEditorModel* InEditorModel);

	/** Get the runtime ML Deformer model object. */
	UFUNCTION(BlueprintPure, Category = "Training Data")
	UE_API UMLDeformerModel* GetModel() const;

	/** Get the number of input transforms. This is the number of bones. */
	UFUNCTION(BlueprintPure, Category = "Training Data")
	UE_API int32 GetNumberSampleTransforms() const;

	/** Get number of input curves. */
	UFUNCTION(BlueprintPure, Category = "Training Data")
	UE_API int32 GetNumberSampleCurves() const;

	/** Get the number of vertex deltas. */
	UFUNCTION(BlueprintPure, Category = "Training Data")
	UE_API int32 GetNumberSampleDeltas() const;

	/** Get the number of samples in this data set. This is the number of sample frames we want to train on. */
	UFUNCTION(BlueprintPure, Category = "Training Data")
    UE_API int32 NumSamples() const;

	/** This will make the sampling start again from the beginning. This can be used if you have to iterate multiple times over the data set. */
	UFUNCTION(BlueprintCallable, Category = "Training Data")
    UE_API void ResetSampling();

	/** 
	 * Set the current sample frame. This will internally call the SampleFrame method, which will update the deltas, curve values and bone rotations. 
	 * You call this before getting the input bone/curve and vertex delta values.
	 * @param Index The training data frame/sample number.
	 * @return Returns true when successful, or false when the specified sample index is out of range.
	 */
	UE_DEPRECATED(5.4, "Please use NextSample instead.")
	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "Please use NextSample instead."))
	UE_API bool SetCurrentSampleIndex(int32 Index);

	/**
	 * Take the next sample. This will update the deltas, curve values and bone rotation arrays with values sampled at the next frame.
	 * This will return false when there is something wrong or we sampled more times than NumSamples() returns.
	 */
	UFUNCTION(BlueprintCallable, Category = "Training Data")
	UE_API bool NextSample();

	/** 
	 * Check whether we need to resample the inputs and outputs, or if we can use a cached version. 
	 * This will return true if any inputs changed, that indicate that we should regenerate any cached data.
	 * @return Returns true when we need to regenerate any cached data, otherwise false is returned.
	 */
	UFUNCTION(BlueprintCallable, Category = "Training Data")
	UE_API bool GetNeedsResampling() const;

	/** Specify whether we need to resample any cached data or not, because our input assets, or any other relevant settings changed that would invalidate the cached data. */
	UFUNCTION(BlueprintCallable, Category = "Training Data")
	UE_API void SetNeedsResampling(bool bNeedsResampling);

	/** Set the number of floats per curve. On default this is one. */
	UFUNCTION(BlueprintCallable, Category = "Training Data")
	UE_API void SetNumFloatsPerCurve(int32 NumFloatsPerCurve);

	/** Get the names of all valid training animation masks. They are valid when they are used, the anim is enabled, and the attribute exists on the skeletal mesh. */
	UFUNCTION(BlueprintPure, Category = "Training Data")
    UE_API TArray<FName> GetTrainingInputAnimMasks() const;

	/** Get the per vertex data for a given mask name. The mask name must be one that is present in the array returned by GetTrainingInputAnimMasks. */
	UFUNCTION(BlueprintPure, Category = "Training Data")
	UE_API TArray<float> GetTrainingInputAnimMaskData(FName MaskName) const;

	/** 
	 * For each sample we took, this contains the index inside the mask as returned by the GetTrainingInputAnimMasks() array.
	 */
	UFUNCTION(BlueprintPure, Category = "Training Data")
	TArray<int32> GetMaskIndexPerSampleArray() const	{ return MaskIndexPerSample; }

	/** Set the list of possible training devices. */
	UFUNCTION(BlueprintCallable, Category = "Training Model")
	UE_API void SetDeviceList(const TArray<FString>& DeviceNames, int32 PreferredDeviceIndex);

	UFUNCTION(BlueprintImplementableEvent, Category = "Training Model")
	UE_API void UpdateAvailableDevices() const;

protected:
	/** Sample the next frame in the sequence of samples. */
	UE_API virtual bool SampleNextFrame();

	/** Change the pointer to the editor model. */
	UE_API void SetEditorModel(UE::MLDeformer::FMLDeformerEditorModel* InModel);

	/** Get a pointer to the editor model. */ 
	UE_API UE::MLDeformer::FMLDeformerEditorModel* GetEditorModel() const;

	/**
	 * Find the next input animation to sample from.
	 * This is an index inside the training input animations list.
	 * This assumes the SampleAnimIndex member as starting point. The method does not modify this member directly, unless passed in as parameter.
	 * @param OutNextAnimIndex The next animation index to sample from when we take our next sample.
	 * @return Returns true when we found our next animation to sample. Returns false when we already sampled everything.
	 */
	virtual bool FindNextAnimToSample(int32& OutNextAnimIndex) const { return false; }

	/**
	 * Get the mask index for a given input training animation.
	 * @see GetTrainingInputAnimMasks.
	 */
	UE_API int32 GetMaskIndexForAnimIndex(int32 AnimIndex) const;

public:
	// The delta values per vertex for this sample. This is updated after NextSample is called. Contains an xyz (3 floats) for each vertex.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Training Data")
	TArray<float> SampleDeltas;

	// The curve weights. This is updated after NextSample is called.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Training Data")
	TArray<float> SampleCurveValues;

	// The bone rotations in bone (local) space for this sample. This is updated after NextSample is called and is 6 floats per bone (2 columns of 3x3 rotation matrix).
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Training Data")
	TArray<float> SampleBoneRotations;

	// The mask index for each sample.
	// See GetTrainingInputAnimMasks() and GetTrainingInputAnimMaskData().
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Training Data")
	TArray<int32> MaskIndexPerSample;

protected:
	/** A pointer to the editor model from which we use the sampler. */
	UE::MLDeformer::FMLDeformerEditorModel* EditorModel = nullptr;

	/** 
	 * The number of times a given input animation has been sampled. 
	 * The size of the array equals the number of training input animations.
	 * Note that this also contains counts of disabled training input anims. These disabled ones should be ignored.
	 */
	TArray<int32> NumTimesSampled;

	/** The training input animation to take the next sample from. */
	int32 SampleAnimIndex = 0;

	/** Did we finish sampling? This is set to true when every possible frame has been sampled. */
	bool bFinishedSampling = false;

	TArray<FName> MaskNames;

	/** The default mask name. */
	static UE_API FName DefaultMaskName;
};

#undef UE_API
