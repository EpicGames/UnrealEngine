// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IDetailCustomization.h"
#include "AssetRegistry/AssetData.h"
#include "PropertyHandle.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class IDetailCategoryBuilder;
class IDetailLayoutBuilder;
class IDetailGroup;
class USkeleton;
class UMLDeformerModel;
class UMLDeformerVizSettings;

namespace UE::MLDeformer
{
	class FMLDeformerEditorModel;

	/**
	 * The detail customization base class for any UMLDeformerVizSettings inherited object.
	 * This automatically adds the settings of the UMLDeformerVizSettings class to the UI and creates some groups, inserts error messages when needed, etc.
	 */
	class FMLDeformerVizSettingsDetails
		: public IDetailCustomization
	{
	public:
		// ILayoutDetails overrides.
		UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
		// ~END ILayoutDetails overrides.

		/** Get a pointer to the detail layout builder. */
		IDetailLayoutBuilder* GetDetailLayoutBuilder() const { return DetailLayoutBuilder; }

		// UE_DEPRECATED(5.4, Remove these constructors when StatsMainMemUsageGroup is removed and there is no longer warnings in the implicit functions)
		FMLDeformerVizSettingsDetails() = default;
		UE_API FMLDeformerVizSettingsDetails(const FMLDeformerVizSettingsDetails&);
		UE_API FMLDeformerVizSettingsDetails(FMLDeformerVizSettingsDetails&&);
		UE_API FMLDeformerVizSettingsDetails& operator=(const FMLDeformerVizSettingsDetails&);
		UE_API FMLDeformerVizSettingsDetails& operator=(FMLDeformerVizSettingsDetails&&);
		// END UE_DEPRECATED(5.4)

	protected:
		/** The filter that only shows anim sequences that are compatible with the given skeleton. */
		UE_API bool FilterAnimSequences(const FAssetData& AssetData, USkeleton* Skeleton);

		/** When we click the reset to default button next to the deformer graph property. */
		UE_API void OnResetToDefaultDeformerGraph(TSharedPtr<IPropertyHandle> PropertyHandle);

		/** Check if the reset to default button next to the deformer graph property should be visible or not. */
		UE_API bool IsResetToDefaultDeformerGraphVisible(TSharedPtr<IPropertyHandle> PropertyHandle);

		/**
		 * Update the class member pointers, which includes the pointer to the model, its editor model, and the viz settings.
		 * @param Objects The array of objects that the detail customization is showing.
		 */
		UE_API virtual bool UpdateMemberPointers(const TArray<TWeakObjectPtr<UObject>>& Objects);

		/** Create the categories to which we can add properties. Update the member category pointers to those. */
		UE_API virtual void CreateCategories();

		/** Add some error message related to the test animation sequence. */
		virtual void AddTestSequenceErrors() {}

		/** Add some error related to the deformer graph. */
		virtual void AddDeformerGraphErrors() {}

		/** Add the ground truth test sequence property. */
		virtual void AddGroundTruth() {}

		/** Add any additional settings. */
		virtual void AddAdditionalSettings() {}

		/** Add any statistics. */
		UE_API virtual void AddStatistics();

		UE_API void AddStatsPerfRow(
			IDetailGroup& Group,
			const FText& Label,
			const FMLDeformerEditorModel* InEditorModel,
			const FNumberFormattingOptions& Format,
			bool bHighlight,
			TFunctionRef<int32(const FMLDeformerEditorModel*)> GetCyclesFunction);

	protected:
		/** Associated detail layout builder. */
		IDetailLayoutBuilder* DetailLayoutBuilder = nullptr;

		/** The ML deformer model that these visualization settings belong to. */
		TObjectPtr<UMLDeformerModel> Model = nullptr;

		/** The visualization settings we are viewing. */
		TObjectPtr<UMLDeformerVizSettings> VizSettings = nullptr;

		/** The editor model of the runtime model. */
		FMLDeformerEditorModel* EditorModel = nullptr;

		/** A pointer to the shared settings category. */
		IDetailCategoryBuilder* SharedCategoryBuilder = nullptr;

		/** A pointer to the test assets category. */
		IDetailCategoryBuilder* TestAssetsCategory = nullptr;

		/** A pointer to the live settings category. */
		IDetailCategoryBuilder* LiveSettingsCategory = nullptr;

		/** A pointer to the training mesh category. This is only visible in training mode. */
		IDetailCategoryBuilder* TrainingMeshesCategoryBuilder = nullptr;

		/** A pointer to the statistics category. */
		IDetailCategoryBuilder* StatsCategoryBuilder = nullptr;

		/** The performance group inside the statistics category. */
		IDetailGroup* StatsPerformanceGroup = nullptr;

		/** The memory usage group inside the statistics category. */
		IDetailGroup* StatsMemUsageGroup = nullptr;

		/** The memory usage group inside the statistics category. */
		IDetailGroup* StatsAssetSizeGroup = nullptr;

		/** Main memory usage subgroup. */
		IDetailGroup* StatsMainMemUsageGroup_DEPRECATED = nullptr;

		/** GPU memory usage subgroup. */
		IDetailGroup* StatsGPUMemUsageGroup = nullptr;

		/** Number format used for performance numbers. */
		FNumberFormattingOptions PerformanceMetricFormat;

		/** Number format used for memory usage numbers. */
		FNumberFormattingOptions MemUsageMetricFormat;
	};
}	// namespace UE::MLDeformer

#undef UE_API
