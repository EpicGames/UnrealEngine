// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"
#include "Dialog/SCustomDialog.h"
#include "Framework/Commands/Commands.h"
#include "MLDeformerEditorStyle.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

struct FReferenceSkeleton;

namespace UE::MLDeformer
{
	class SMLDeformerBonePickerTreeWidget;
	class SMLDeformerBonePickerDialog;
	class FMLDeformerEditorModel;


	class FMLDeformerBonePickerTreeElement
		: public TSharedFromThis<FMLDeformerBonePickerTreeElement>
	{
	public:
		TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FMLDeformerBonePickerTreeElement> InTreeElement, TSharedPtr<SMLDeformerBonePickerTreeWidget> InTreeWidget);

	public:
		TArray<TSharedPtr<FMLDeformerBonePickerTreeElement>> Children;
		FName Name;
		FSlateColor TextColor;
	};


	class SMLDeformerBonePickerTreeRowWidget 
		: public STableRow<TSharedPtr<FMLDeformerBonePickerTreeElement>>
	{
	public:
		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FMLDeformerBonePickerTreeElement> InTreeElement, TSharedPtr<SMLDeformerBonePickerTreeWidget> InTreeView);

	private:
		TWeakPtr<FMLDeformerBonePickerTreeElement> WeakTreeElement;
		FText GetName() const;

		friend class SMLDeformerBonePickerTreeWidget; 
	};


	class SMLDeformerBonePickerTreeWidget
		: public STreeView<TSharedPtr<FMLDeformerBonePickerTreeElement>>
	{
		SLATE_BEGIN_ARGS(SMLDeformerBonePickerTreeWidget) {}
		SLATE_ARGUMENT(bool, AllowMultiSelect)
		SLATE_ARGUMENT(TSharedPtr<SMLDeformerBonePickerDialog>, PickerWidget)
		SLATE_END_ARGS()

	public:
		void Construct(const FArguments& InArgs);
		const TArray<TSharedPtr<FMLDeformerBonePickerTreeElement>>& GetRootElements() const	{ return RootElements; }
		void AddElement(TSharedPtr<FMLDeformerBonePickerTreeElement> Element, TSharedPtr<FMLDeformerBonePickerTreeElement> ParentElement);
		void Clear();

	private:
		TSharedRef<ITableRow> MakeTableRowWidget(TSharedPtr<FMLDeformerBonePickerTreeElement> InItem, const TSharedRef<STableViewBase>& OwnerTable);
		void HandleGetChildrenForTree(TSharedPtr<FMLDeformerBonePickerTreeElement> InItem, TArray<TSharedPtr<FMLDeformerBonePickerTreeElement>>& OutChildren);
		void OnMouseDoubleClicked(TSharedPtr<FMLDeformerBonePickerTreeElement> ClickedItem);

	private:
		TArray<TSharedPtr<FMLDeformerBonePickerTreeElement>> RootElements;
		TSharedPtr<SMLDeformerBonePickerDialog> PickerWidget;
	};


	/**
	 * A bone picker dialog window.
	 * This allows users to select one or more bones.
	 */
	class SMLDeformerBonePickerDialog
		: public SCustomDialog
	{
	public:
		friend class SMLDeformerBonePickerTreeWidget;

		SLATE_BEGIN_ARGS(SMLDeformerBonePickerDialog) {}
		SLATE_ARGUMENT(FReferenceSkeleton const*, RefSkeleton)		// The reference skeleton to grab the bone names and hierarchy from.
		SLATE_ARGUMENT(TArray<FName>, HighlightBoneNames)		// Highlight bones in this list.
		SLATE_ARGUMENT(TArray<FName>, InitialSelectedBoneNames)	// Select all bones in this list.
		SLATE_ARGUMENT(FSlateColor, HighlightBoneNamesColor)	// Bones inside the HighlightBoneNames list will use this color.
		SLATE_ARGUMENT(TArray<FName>, IncludeList)				// If empty, all bones can be picked. If not empty, only show bones with names inside this list.
		SLATE_ARGUMENT(FText, Title)							// The window title.
		SLATE_ARGUMENT(bool, AllowMultiSelect)					// Allow multi select, or only allow to pick one bone?
		SLATE_ARGUMENT(TSharedPtr<SWidget>, ExtraWidget)		// An optional extra widget shown on top, next to the search field.
		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs);

		/**
		 * Get the array of picked bone names.
		 * This will be an empty list when nothing got selected, which means we basically cancelled.
		 * @return The array of selected bone names. Empty if cancelled or if no bones were there to begin with.
		 */
		UE_API const TArray<FName>& GetPickedBoneNames() const;

	private:
		UE_API TSharedRef<SMLDeformerBonePickerTreeWidget> CreateBoneTree();
		UE_API FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
		UE_API void OnOkClicked();
		UE_API void OnCancelClicked();
		UE_API void OnFilterTextChanged(const FText& InFilterText);
		UE_API void RefreshBoneTreeElements() const;
		UE_API void SelectInitialItems();
		UE_API void SelectInitialItemsRecursive(const TSharedPtr<FMLDeformerBonePickerTreeElement>& Item);

	private:
		TSharedPtr<SMLDeformerBonePickerTreeWidget> BoneTreeWidget;
		TSharedPtr<SWidget> ExtraWidget;
		TArray<FName> HighlightBoneNames;
		TArray<FName> InitialSelectedBoneNames;
		TArray<FName> IncludeList;
		FReferenceSkeleton const* RefSkeleton = nullptr;
		TArray<FName> PickedBoneNames;
		FSlateColor HighlightColor;
		bool bAllowMultiSelect = true;
		FString FilterText;	
	};
}

#undef UE_API
