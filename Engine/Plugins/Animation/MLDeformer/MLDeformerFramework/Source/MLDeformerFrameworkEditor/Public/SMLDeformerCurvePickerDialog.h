// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Dialog/SCustomDialog.h"
#include "Framework/Commands/Commands.h"
#include "MLDeformerEditorStyle.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class USkeleton;
class USkeletalMesh;

namespace UE::MLDeformer
{
	class SMLDeformerCurvePickerDialog;
	class SMLDeformerCurvePickerListView;
	class FMLDeformerEditorModel;
	class FMLDeformerCurvePickerElement;


	class FMLDeformerCurvePickerElement
		: public TSharedFromThis<FMLDeformerCurvePickerElement>
	{
	public:
		FMLDeformerCurvePickerElement(const FName InName, const FSlateColor& InTextColor);
		TSharedRef<ITableRow> MakeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FMLDeformerCurvePickerElement> InElement, TSharedPtr<SMLDeformerCurvePickerListView> InListView);

	public:
		FName Name;
		FSlateColor TextColor;
	};


	class SMLDeformerCurvePickerRowWidget 
		: public STableRow<TSharedPtr<FMLDeformerCurvePickerElement>>
	{
	public:
		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FMLDeformerCurvePickerElement> InTreeElement, TSharedPtr<SMLDeformerCurvePickerListView> InListView);

	private:
		TWeakPtr<FMLDeformerCurvePickerElement> WeakElement;
		FText GetName() const;

		friend class SMLDeformerCurvePickerListView; 
	};


	class SMLDeformerCurvePickerListView
		: public SListView<TSharedPtr<FMLDeformerCurvePickerElement>>
	{
		SLATE_BEGIN_ARGS(SMLDeformerCurvePickerListView) {}
		SLATE_ARGUMENT(bool, AllowMultiSelect)
		SLATE_ARGUMENT(TSharedPtr<SMLDeformerCurvePickerDialog>, PickerDialog)
		SLATE_END_ARGS()

	public:
		void Construct(const FArguments& InArgs);
		void AddElement(TSharedPtr<FMLDeformerCurvePickerElement> Element);
		void Clear();

	private:
		TSharedRef<ITableRow> MakeTableRowWidget(TSharedPtr<FMLDeformerCurvePickerElement> InItem, const TSharedRef<STableViewBase>& OwnerTable);
		void OnMouseDoubleClicked(TSharedPtr<FMLDeformerCurvePickerElement> ClickedItem);

	private:
		TArray<TSharedPtr<FMLDeformerCurvePickerElement>> Elements;
		TSharedPtr<SMLDeformerCurvePickerDialog> PickerDialog;
	};


	class SMLDeformerCurvePickerDialog
		: public SCustomDialog
	{
	public:
		friend class SMLDeformerCurvePickerListView;

		SLATE_BEGIN_ARGS(SMLDeformerCurvePickerDialog) {}
		SLATE_ARGUMENT(USkeletalMesh*, SkeletalMesh)			// The skeleton to pick the curves from.
		SLATE_ARGUMENT(TArray<FName>, IncludeList)				// If empty, all curves can be picked. If not empty, only show curves with names inside this list.
		SLATE_ARGUMENT(TArray<FName>, HighlightCurveNames)		// The curve names to highlight, if any.
		SLATE_ARGUMENT(FSlateColor, HighlightCurveNamesColor)	// The text color to use for this highlighting.
		SLATE_ARGUMENT(FText, Title)							// The window title.
		SLATE_ARGUMENT(bool, AllowMultiSelect)					// Set to false to allow the user to only select one curve, or true to allow for multiple.
		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs);

		/**
		 * Get the array of picked curves names.
		 * This will be an empty list when nothing got selected, which means we basically cancelled.
		 * @return The array of selected curves names. Empty if cancelled or if no curves were there to begin with.
		 */
		UE_API const TArray<FName>& GetPickedCurveNames() const;

	private:
		UE_API TSharedRef<SMLDeformerCurvePickerListView> CreateListWidget();
		UE_API FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
		UE_API void OnOkClicked();
		UE_API void OnCancelClicked();
		UE_API void OnFilterTextChanged(const FText& InFilterText);
		UE_API void RefreshListElements() const;

	private:
		TSharedPtr<SMLDeformerCurvePickerListView> CurveListView;
		TArray<FName> PickedCurveNames;
		TArray<FName> IncludeList;
		FString FilterText;
		TWeakObjectPtr<USkeletalMesh> SkeletalMesh;
		TArray<FName> HighlightCurveNames;
		FSlateColor HighlightColor = FSlateColor(FLinearColor::White);
		bool bAllowMultiSelect = true;
	};
}

#undef UE_API
