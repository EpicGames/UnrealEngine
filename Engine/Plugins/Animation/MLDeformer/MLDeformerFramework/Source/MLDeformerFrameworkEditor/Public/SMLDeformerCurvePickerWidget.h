// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Object.h"
#include "SlateFwd.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/STableRow.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class SComboButton;
class SSearchBox;
class USkeletalMesh;

DECLARE_DELEGATE_OneParam(FOnCurveSelectionChanged, const FString&);
DECLARE_DELEGATE_OneParam(FOnCurveNamePicked, const FString&)

DECLARE_DELEGATE_RetVal(FString, FOnGetSelectedCurve);
DECLARE_DELEGATE_RetVal(USkeletalMesh*, FOnGetSkeletalMesh)


namespace UE::MLDeformer
{
	/**
	 * The curve picker widget for the ML Deformer editor.
	 */
	class SCurvePickerWidget
		: public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SCurvePickerWidget) {}
		SLATE_EVENT(FOnCurveNamePicked, OnCurveNamePicked)
		SLATE_EVENT(FOnGetSkeletalMesh, OnGetSkeletalMesh)
		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs);

		TSharedPtr<SSearchBox> GetFilterTextWidget() const { return SearchBox; }

	private:
		UE_API void RefreshListItems();
		UE_API void FilterAvailableCurves();
		UE_API void HandleSelectionChanged(TSharedPtr<FString> InItem, ESelectInfo::Type InSelectionType);
		UE_API void HandleFilterTextChanged(const FText& InFilterText);
		UE_API TSharedRef<ITableRow> HandleGenerateRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& InOwnerTable);

	private:
		/** Delegate fired when a curve name is picked. */
		FOnCurveNamePicked OnCurveNamePicked;

		/** Provide us with a skeletal mesh. */
		FOnGetSkeletalMesh OnGetSkeletalMesh;

		/** The search filter box. */
		TSharedPtr<SSearchBox> SearchBox;

		/** The skeletal mesh to get the curves from. */
		TWeakObjectPtr<USkeletalMesh> SkeletalMesh;

		/** The names of the curves we are displaying. */
		TArray<TSharedPtr<FString>> CurveNames;

		/** All the unique curve names we can find. */
		TSet<FString> UniqueCurveNames;

		/** The string we use to filter curve names. */
		FString FilterText;

		/** The list view used to display names. */
		TSharedPtr<SListView<TSharedPtr<FString>>> NameListView;
	};

	/**
	 * The curve selection widget for the ML Deformer.
	 */
	class SCurveSelectionWidget
		: public SCompoundWidget
	{
	public: 
		SLATE_BEGIN_ARGS(SCurveSelectionWidget) {}
			SLATE_EVENT(FOnCurveSelectionChanged, OnCurveSelectionChanged);
			SLATE_EVENT(FOnGetSelectedCurve, OnGetSelectedCurve);
			SLATE_EVENT(FOnGetSkeletalMesh, OnGetSkeletalMesh)
		SLATE_END_ARGS();

		void Construct(const FArguments& InArgs);

	private: 
		TSharedRef<SWidget> CreateSkeletonWidgetMenu();
		void OnSelectionChanged(const FString& CurveName);
		USkeletalMesh* GetSkeletalMesh() const;
		FText GetCurrentCurveName() const;
		FText GetFinalToolTip() const;

	private:
		TSharedPtr<SComboButton> CurvePickerButton;
		FOnCurveSelectionChanged OnCurveSelectionChanged;
		FOnGetSelectedCurve OnGetSelectedCurve;
		FOnGetSkeletalMesh OnGetSkeletalMesh;
		FText SuppliedToolTip;
		FText SelectedCurve;
	};
}	// namespace UE::MLDeformer

#undef UE_API
