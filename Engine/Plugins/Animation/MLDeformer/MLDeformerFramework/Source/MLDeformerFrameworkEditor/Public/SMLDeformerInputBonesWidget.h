// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"
#include "Framework/Commands/Commands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "UObject/NameTypes.h"
#include "Styling/SlateColor.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

struct FReferenceSkeleton;

namespace UE::MLDeformer
{
	class SMLDeformerInputBoneTreeWidget;
	class SMLDeformerInputBonesWidget;
	class SMLDeformerInputWidget;
	class FMLDeformerEditorModel;

	class MLDEFORMERFRAMEWORKEDITOR_API FMLDeformerInputBonesWidgetCommands
		: public TCommands<FMLDeformerInputBonesWidgetCommands>
	{
	public:
		FMLDeformerInputBonesWidgetCommands();
		virtual void RegisterCommands() override;

	public:
		TSharedPtr<FUICommandInfo> AddInputBones;
		TSharedPtr<FUICommandInfo> DeleteInputBones;
		TSharedPtr<FUICommandInfo> ClearInputBones;
		TSharedPtr<FUICommandInfo> AddAnimatedBones;
		TSharedPtr<FUICommandInfo> CopyInputBones;
		TSharedPtr<FUICommandInfo> PasteInputBones;
	};


	class FMLDeformerInputBoneTreeElement
		: public TSharedFromThis<FMLDeformerInputBoneTreeElement>
	{
	public:
		UE_API TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FMLDeformerInputBoneTreeElement> InTreeElement, TSharedPtr<SMLDeformerInputBoneTreeWidget> InTreeWidget);

	public:
		FName Name;
		TArray<TSharedPtr<FMLDeformerInputBoneTreeElement>> Children;
		FSlateColor TextColor;
	};


	class SMLDeformerInputBoneTreeRowWidget 
		: public STableRow<TSharedPtr<FMLDeformerInputBoneTreeElement>>
	{
	public:
		UE_API void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FMLDeformerInputBoneTreeElement> InTreeElement, TSharedPtr<SMLDeformerInputBoneTreeWidget> InTreeView);

	private:
		TWeakPtr<FMLDeformerInputBoneTreeElement> WeakTreeElement;
		UE_API FText GetName() const;

		friend class SMLDeformerInputBoneTreeWidget; 
	};


	class SMLDeformerInputBoneTreeWidget
		: public STreeView<TSharedPtr<FMLDeformerInputBoneTreeElement>>
	{
		SLATE_BEGIN_ARGS(SMLDeformerInputBoneTreeWidget) {}
		SLATE_ARGUMENT(TSharedPtr<SMLDeformerInputBonesWidget>, InputBonesWidget)
		SLATE_END_ARGS()

	public:
		UE_API void Construct(const FArguments& InArgs);
		const TArray<TSharedPtr<FMLDeformerInputBoneTreeElement>>& GetRootElements() const	{ return RootElements; }
		UE_API void AddElement(TSharedPtr<FMLDeformerInputBoneTreeElement> Element, TSharedPtr<FMLDeformerInputBoneTreeElement> ParentElement);
		UE_API void RefreshElements(const TArray<FName>& BoneNames, const FReferenceSkeleton* RefSkeleton);
		UE_API TArray<FName> ExtractAllElementNames() const;

	private:
		UE_API TSharedRef<ITableRow> MakeTableRowWidget(TSharedPtr<FMLDeformerInputBoneTreeElement> InItem, const TSharedRef<STableViewBase>& OwnerTable);
		UE_API void HandleGetChildrenForTree(TSharedPtr<FMLDeformerInputBoneTreeElement> InItem, TArray<TSharedPtr<FMLDeformerInputBoneTreeElement>>& OutChildren);
		UE_API FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
		UE_API TSharedPtr<SWidget> OnContextMenuOpening() const;
		UE_API void RecursiveSortElements(TSharedPtr<FMLDeformerInputBoneTreeElement> Element);
		UE_API void RecursiveAddNames(const FMLDeformerInputBoneTreeElement& Element, TArray<FName>& OutNames) const;
		UE_API void OnSelectionChanged(TSharedPtr<FMLDeformerInputBoneTreeElement> Selection, ESelectInfo::Type SelectInfo);
		UE_API TSharedPtr<FMLDeformerInputBoneTreeElement> FindParentElementForBone(FName BoneName, const FReferenceSkeleton& RefSkeleton, const TMap<FName, TSharedPtr<FMLDeformerInputBoneTreeElement>>& NameToElementMap) const;

	private:
		TArray<TSharedPtr<FMLDeformerInputBoneTreeElement>> RootElements;
		TSharedPtr<SMLDeformerInputBonesWidget> InputBonesWidget;
	};


	class SMLDeformerInputBonesWidget
		: public SCompoundWidget
	{
		friend class SMLDeformerInputBoneTreeWidget;

		SLATE_BEGIN_ARGS(SMLDeformerInputBonesWidget) {}
		SLATE_ARGUMENT(FMLDeformerEditorModel*, EditorModel)
		SLATE_ARGUMENT(TSharedPtr<SMLDeformerInputWidget>, InputWidget)
		SLATE_END_ARGS()

	public:
		UE_API void Construct(const FArguments& InArgs);
		UE_API void Refresh();
		UE_API void BindCommands(TSharedPtr<FUICommandList> CommandList);
		UE_API TSharedPtr<SMLDeformerInputBoneTreeWidget> GetTreeWidget() const;
		UE_API TSharedPtr<SMLDeformerInputWidget> GetInputWidget() const;
		UE_API FText GetSectionTitle() const;

	private:
		UE_API void OnFilterTextChanged(const FText& InFilterText);
		UE_API void RefreshTree(bool bBroadCastPropertyChanged=true);
		UE_API bool BroadcastModelPropertyChanged(const FName PropertyName);

		UE_API void OnAddInputBones();
		UE_API void OnDeleteInputBones();
		UE_API void OnClearInputBones();
		UE_API void OnAddAnimatedBones();
		UE_API void OnCopyInputBones() const;
		UE_API void OnPasteInputBones();

	private:
		TSharedPtr<SMLDeformerInputBoneTreeWidget> TreeWidget;
		TSharedPtr<SMLDeformerInputWidget> InputWidget;
		FMLDeformerEditorModel* EditorModel = nullptr;
		FString FilterText;
		FText SectionTitle;
	};

}	// namespace UE::MLDeformer

#undef UE_API
