// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Framework/Commands/Commands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "UObject/NameTypes.h"
#include "Styling/SlateColor.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class USkeletalMesh;

namespace UE::MLDeformer
{
	class SMLDeformerInputCurveListWidget;
	class SMLDeformerInputCurvesWidget;
	class SMLDeformerInputWidget;
	class FMLDeformerEditorModel;

	class MLDEFORMERFRAMEWORKEDITOR_API FMLDeformerInputCurvesWidgetCommands
		: public TCommands<FMLDeformerInputCurvesWidgetCommands>
	{
	public:
		FMLDeformerInputCurvesWidgetCommands();
		virtual void RegisterCommands() override;

	public:
		TSharedPtr<FUICommandInfo> AddInputCurves;
		TSharedPtr<FUICommandInfo> DeleteInputCurves;
		TSharedPtr<FUICommandInfo> ClearInputCurves;
		TSharedPtr<FUICommandInfo> AddAnimatedCurves;
		TSharedPtr<FUICommandInfo> CopyInputCurves;
		TSharedPtr<FUICommandInfo> PasteInputCurves;	
	};


	class FMLDeformerInputCurveListElement
		: public TSharedFromThis<FMLDeformerInputCurveListElement>
	{
	public:
		UE_API TSharedRef<ITableRow> MakeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FMLDeformerInputCurveListElement> InTreeElement, TSharedPtr<SMLDeformerInputCurveListWidget> InTreeWidget);

	public:
		FName Name;
		FSlateColor TextColor;
	};


	class SMLDeformerInputCurveListRowWidget 
		: public STableRow<TSharedPtr<FMLDeformerInputCurveListElement>>
	{
	public:
		UE_API void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, TSharedRef<FMLDeformerInputCurveListElement> InElement, TSharedPtr<SMLDeformerInputCurveListWidget> InListView);

	private:
		TWeakPtr<FMLDeformerInputCurveListElement> WeakElement;
		UE_API FText GetName() const;

		friend class SMLDeformerInputCurveListWidget; 
	};


	class SMLDeformerInputCurveListWidget
		: public SListView<TSharedPtr<FMLDeformerInputCurveListElement>>
	{
		SLATE_BEGIN_ARGS(SMLDeformerInputCurveListWidget) {}
		SLATE_ARGUMENT(TSharedPtr<SMLDeformerInputCurvesWidget>, InputCurvesWidget)
		SLATE_END_ARGS()

	public:
		UE_API void Construct(const FArguments& InArgs);
		const TArray<TSharedPtr<FMLDeformerInputCurveListElement>>& GetElements() const	{ return Elements; }
		UE_API void AddElement(TSharedPtr<FMLDeformerInputCurveListElement> Element);
		UE_API void RefreshElements(const TArray<FName>& CurveNames, USkeletalMesh* SkeletalMesh);

	private:
		UE_API TSharedRef<ITableRow> MakeTableRowWidget(TSharedPtr<FMLDeformerInputCurveListElement> InItem, const TSharedRef<STableViewBase>& OwnerTable);
		UE_API FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
		UE_API TSharedPtr<SWidget> OnContextMenuOpening() const;
		UE_API void SortElements();
		UE_API void OnSelectionChanged(TSharedPtr<FMLDeformerInputCurveListElement> Selection, ESelectInfo::Type SelectInfo);

	private:
		TArray<TSharedPtr<FMLDeformerInputCurveListElement>> Elements;
		TSharedPtr<SMLDeformerInputCurvesWidget> InputCurvesWidget;
	};


	class SMLDeformerInputCurvesWidget
		: public SCompoundWidget
	{
		friend class SMLDeformerInputCurveListWidget;

		SLATE_BEGIN_ARGS(SMLDeformerInputCurvesWidget) {}
		SLATE_ARGUMENT(FMLDeformerEditorModel*, EditorModel)
		SLATE_ARGUMENT(TSharedPtr<SMLDeformerInputWidget>, InputWidget)
		SLATE_END_ARGS()

	public:
		UE_API void Construct(const FArguments& InArgs);
		UE_API void Refresh();
		UE_API void BindCommands(TSharedPtr<FUICommandList> CommandList);
		UE_API FText GetSectionTitle() const;
		UE_API TSharedPtr<SMLDeformerInputCurveListWidget> GetListWidget() const;
		UE_API TSharedPtr<SMLDeformerInputWidget> GetInputWidget() const;

	private:
		UE_API void OnFilterTextChanged(const FText& InFilterText);
		UE_API void RefreshList(bool bBroadCastPropertyChanged=true);
		UE_API bool BroadcastModelPropertyChanged(const FName PropertyName);

		UE_API void OnCopyInputCurves() const;
		UE_API void OnPasteInputCurves();
	
		UE_API void OnAddInputCurves();
		UE_API void OnDeleteInputCurves();
		UE_API void OnClearInputCurves();
		UE_API void OnAddAnimatedCurves();

	private:
		TSharedPtr<SMLDeformerInputWidget> InputWidget;
		TSharedPtr<SMLDeformerInputCurveListWidget> ListWidget;
		FMLDeformerEditorModel* EditorModel = nullptr;
		FString FilterText;
		FText SectionTitle;
	};

}	// namespace UE::MLDeformer

#undef UE_API
