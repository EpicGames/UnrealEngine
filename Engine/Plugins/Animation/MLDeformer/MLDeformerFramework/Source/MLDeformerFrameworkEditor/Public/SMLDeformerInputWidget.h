// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Commands/UICommandList.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class SWidget;
class FMenuBuilder;
class SVerticalBox;
class FJsonObject;

namespace UE::MLDeformer
{
	class SMLDeformerInputBonesWidget;
	class SMLDeformerInputCurvesWidget;
	class FMLDeformerEditorModel;

	class SMLDeformerInputWidget
		: public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SMLDeformerInputWidget) {}
		SLATE_ARGUMENT(FMLDeformerEditorModel*, EditorModel)
		SLATE_END_ARGS()

		struct FSectionInfo
		{
			TAttribute<FText> PlusButtonTooltip;
			TAttribute<FText> SectionTitle;
			FOnClicked PlusButtonPressed;
		};

		UE_API void Construct(const FArguments& InArgs);
		UE_API void AddSection(TSharedPtr<SWidget> Widget, const FSectionInfo& SectionInfo);

		UE_API TSharedPtr<FUICommandList> GetBonesCommandList() const;
		UE_API TSharedPtr<FUICommandList> GetCurvesCommandList() const;

		/**
		 * Refresh all sub-widget contents.
		 */
		UE_API virtual void Refresh();

		virtual void AddInputBonesMenuItems(FMenuBuilder& MenuBuilder) {}
		virtual void AddInputBonesPlusIconMenuItems(FMenuBuilder& MenuBuilder) {}
		virtual void AddInputCurvesMenuItems(FMenuBuilder& MenuBuilder) {}
		virtual void AddInputCurvesPlusIconMenuItems(FMenuBuilder& MenuBuilder) {}

		virtual void OnClearInputBones() {}
		virtual void OnAddInputBones(const TArray<FName>& Names) {}
		virtual void OnDeleteInputBones(const TArray<FName>& Names) {}
		virtual void OnAddAnimatedBones() {}

		virtual void OnClearInputCurves() {}
		virtual void OnAddInputCurves(const TArray<FName>& Names) {}
		virtual void OnDeleteInputCurves(const TArray<FName>& Names) {}
		virtual void OnAddAnimatedCurves() {}

		UE_API virtual void ClearSelectionForAllWidgetsExceptThis(TSharedPtr<SWidget> ExceptThisWidget);
		virtual void OnSelectInputBone(FName BoneName) {}
		virtual void OnSelectInputCurve(FName BoneName) {}

		UE_API virtual TSharedPtr<SWidget> GetExtraBonePickerWidget();

		// Registers the commands that are being executed by the different widgets.
		static UE_API void RegisterCommands();
		static UE_API void UnregisterCommands();

		static UE_API TSharedPtr<FJsonObject> GetJsonObject(const FString& JsonString);
		static UE_API bool ExtractStringFromJson(const TSharedPtr<FJsonObject>& JsonObject, const FString& Key, FString& OutString);
		static UE_API bool HasValidClipBoardData(const FString& DataTypeId, TSharedPtr<FJsonObject>& OutJsonObject);

	protected:
		UE_API void AddSectionSeparator();
		UE_API void CreateBonesWidget();
		UE_API void CreateCurvesWidget();

		UE_API FReply ShowCurvesPlusIconContextMenu();
		UE_API FReply ShowBonesPlusIconContextMenu();

	protected:
		TSharedPtr<SMLDeformerInputBonesWidget> InputBonesWidget;
		TSharedPtr<SMLDeformerInputCurvesWidget> InputCurvesWidget;
		TSharedPtr<FUICommandList> BonesCommandList;
		TSharedPtr<FUICommandList> CurvesCommandList;
		TSharedPtr<SVerticalBox> SectionVerticalBox;
		FMLDeformerEditorModel* EditorModel = nullptr;
	};
}

#undef UE_API
