// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Dialog/SCustomDialog.h"
#include "MLDeformerModel.h"
#include "MLDeformerMasking.h"
#include "MeshDescription.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

namespace UE::MLDeformer
{
	class FMLDeformerEditorModel;

	DECLARE_DELEGATE_OneParam(FOnSetNewVertexAttributeValues, TVertexAttributesRef<float>);

	/**
	 * The dialog that allows the user to choose a masking mode (Generated or using Skeletal Mesh Vertex Attribute).
	 * When using a vertex attribute, you can pick which attribute to use, or create a new one.
	 * Creating a new attribute will pop up the SMLDeformerNewVertexAttributeDialog.
	 */
	class SMLDeformerMaskConfigDialog
		: public SCustomDialog
	{
	public:
		SLATE_BEGIN_ARGS(SMLDeformerMaskConfigDialog) {}
		SLATE_ARGUMENT(FMLDeformerMaskInfo, InitialMaskInfo)
		SLATE_EVENT(FOnSetNewVertexAttributeValues, OnSetNewVertexAttributeValues)
		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs, FMLDeformerEditorModel* InEditorModel);

		/**
		 * This returns the mask info object how it was setup in the dialog.
		 * You should only use this when the user didn't cancel the dialog.
		 */
		const FMLDeformerMaskInfo& GetMaskInfo() const { return MaskInfo; }

	private:
		UE_API void UpdateAttributeNames();
		UE_API TSharedPtr<SWidget> CreateMeshAttributeModeWidget();
		UE_API TSharedPtr<FString> MaskModeEnumToString(EMLDeformerMaskingMode MaskMode) const;
		UE_API FReply OnCreateVertexAttributeClicked();

	private:
		TArray<FName> AttributeNames;
		TArray<TSharedPtr<FString>> MaskingModeNames;
		TSharedPtr<SComboBox<FName>> VertexAttributeComboWidget;
		FMLDeformerEditorModel* EditorModel = nullptr;
		FMLDeformerMaskInfo MaskInfo;
		FOnSetNewVertexAttributeValues OnSetNewVertexAttributeValues;
	};

}	// namespace UE::MLDeformer

#undef UE_API
