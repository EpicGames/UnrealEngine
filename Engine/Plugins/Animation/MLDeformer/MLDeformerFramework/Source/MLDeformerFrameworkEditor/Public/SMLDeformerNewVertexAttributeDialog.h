// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"

class USkeletalMesh;

namespace UE::MLDeformer
{
	/**
	 * The dialog that asks the user to enter a vertex attribute name.
	 * On default the dialog creates the actual vertex attribute on the mesh as well, when the user pressed Create.
	 * You can disable this behavior using the bAutoCreateAttribute Slate argument.
	 * A default value for all vertices can be set as well using the DefaultValue attribute.
	 * 
	 * Example usage:
	 * 
	 * <code>
	 * TSharedPtr<SMLDeformerNewVertexAttributeDialog> Dialog = SNew(SMLDeformerNewVertexAttributeDialog, SkeletalMesh)
	 *		.bAutoCreateAttribute(true)
	 *		.DefaultValue(1.0f);
	 * 
	 * if (Dialog.ShowModal() == SMLDeformerNewVertexAttributeDialog::EReturnCode::CreatePressed)
	 * {
	 *		// Get the name of the attribute that was created.
	 *		const FString& AttributeName = Dialog.GetAttributeName();
	 * 
	 *		// ...
	 * }
	 * </code>
	 */
	class SMLDeformerNewVertexAttributeDialog
		: public SWindow
	{
	public:
		/** The result of the dialog, which determines whether the user pressed the Create button, or if the user would like to cancel the creation. */
		enum class EReturnCode : uint8
		{
			/** The user pressed the Create button. */
			CreatePressed,

			/** The user canceled the creation. */
			Canceled
		};

		SLATE_BEGIN_ARGS(SMLDeformerNewVertexAttributeDialog)
			: _bAutoCreateAttribute(true)
			, _DefaultAttributeValue(0.0f)
		{}

			/** Should we automatically create the vertex attribute? The default is true. */
			SLATE_ARGUMENT(bool, bAutoCreateAttribute)

			/** What value should the vertex attributes have for all vertices after creation? This only is used when bAutoCreateAttribute is set to true. The default value is 0. */
			SLATE_ARGUMENT(float, DefaultAttributeValue)

		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, USkeletalMesh* InSkeletalMesh);

		/** Shows the dialog. The return code tells you whether the user canceled or pressed create. */
		EReturnCode ShowModal();

		/** 
		 * When the user pressed Create, which you can check using the return code of ShowModal, then this is the attribute name they entered.
		 * It is guaranteed that this attribute is one with a unique name if no other has been created in some other thread or so after the dialog has been created.
		 */
		const FString& GetAttributeName() const { return AttributeName; }


	private:
		/** Override the OnKeyDown to detect when the Escape key has been pressed, to cancel and close the dialog. */
		FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;

		/** Perform actions when the create button has been clicked. */
		FReply OnCreateClicked();

		/**
		 * Check whether a given skeletal mesh has a given vertex attribute already.
		 * @return Returns True when the attribute already exists, or false when not.
		 */
		static bool HasVertexAttribute(const USkeletalMesh* SkeletalMesh, FName InAttributeName);

		/**
		 * Create a float based vertex attribute with a given name, using a specific default value.
		 * When there already is an attribute with the specified name, then false is returned and nothing is changed.
		 * When the attribute name is unique, the new attribute will be created and all vertices will have the default value.
		 * This will create the attribute on LOD 0.
		 * @result Returns true on success, or false when failed. A fail can occur when there is already an attribute with the given name or when no Mesh Description can be found.
		 */
		static bool CreateVertexAttribute(USkeletalMesh& SkeletalMesh, const FString& AttributeName, float DefaultValue = 0.0f);


	private:
		/** The Skeletal Mesh that we wish to create the attribute on. */
		TObjectPtr<USkeletalMesh> SkeletalMesh;

		/** The return code, specifying whether the user pressed create or cancel. */
		EReturnCode ReturnCode = EReturnCode::Canceled;

		/** The name of the attribute that the user entered. */
		FString AttributeName;

		/** The create button. */
		TSharedPtr<SButton> CreateButton;

		/** Should we automatically create the new attribute when the Create button has been pressed? */
		bool bAutoCreate = true;

		/** The default value of all vertices for this vertex attribute. Only used when bAutoCreate we end up creating the attribute. */
		float DefaultAttributeValue = 0.0f;
	};
}	// namespace UE::MLDeformer
