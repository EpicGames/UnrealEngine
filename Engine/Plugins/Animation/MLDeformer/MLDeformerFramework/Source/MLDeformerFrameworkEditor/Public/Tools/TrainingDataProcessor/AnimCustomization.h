// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"
#include "DetailWidgetRow.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class UMLDeformerTrainingDataProcessorSettings;

namespace UE::MLDeformer::TrainingDataProcessor
{
	/**
	 * The property detail customization for a single animation input that can be enabled or disabled.
	 * This is a customization for the type FMLDeformerTrainingDataProcessorAnim.
	 */
	class FAnimCustomization final : public IPropertyTypeCustomization
	{
	public:
		static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance();

		//~ Begin IPropertyTypeCustomization overrides.
		UE_API virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
		UE_API virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
		UE_API EVisibility GetAnimErrorVisibility(const TSharedRef<IPropertyHandle> StructPropertyHandle, int32 AnimIndex) const;
		// ~End IPropertyTypeCustomization overrides.

	private:
		UE_API void RefreshProperties() const;
		static UE_API UMLDeformerTrainingDataProcessorSettings* FindSettings(const TSharedRef<IPropertyHandle>& StructPropertyHandle);

	private:
		TSharedPtr<IPropertyUtilities> PropertyUtilities;
	};
}	// namespace UE::MLDeformer::TrainingDataProcessor

#undef UE_API
