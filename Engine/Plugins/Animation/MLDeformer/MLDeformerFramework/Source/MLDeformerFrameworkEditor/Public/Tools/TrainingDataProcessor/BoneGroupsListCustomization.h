// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"
#include "MLDeformerTrainingDataProcessorSettings.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class UMLDeformerTrainingDataProcessorSettings;
class USkeleton;

namespace UE::MLDeformer::TrainingDataProcessor
{
	/**
	 * The property detail customization for a list of bone groups.
	 * This is a customization for the type FMLDeformerTrainingDataProcessorBoneGroupsList.
	 */
	class FBoneGroupsListCustomization : public IPropertyTypeCustomization
	{
	public:
		static UE_API TSharedRef<IPropertyTypeCustomization> MakeInstance();

		//~ Begin IPropertyTypeCustomization overrides.
		UE_API virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow,
		                             IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
		virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder,
		                               IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}
		//~ End IPropertyTypeCustomization overrides.

	private:
		UE_API TArray<FMLDeformerTrainingDataProcessorBoneGroup>* GetBoneGroups() const;

	private:
		TSharedPtr<IPropertyUtilities> PropertyUtilities;
		TSharedPtr<IPropertyHandle> StructProperty;
	};
} // namespace UE::MLDeformer::TrainingDataProcessor

#undef UE_API
