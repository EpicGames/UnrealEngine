// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"
#include "Framework/Commands/Commands.h"
#include "MLDeformerTrainingDataProcessorSettings.h"
#include "Styling/SlateColor.h"
#include "EditorUndoClient.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class USkeleton;
class FUICommandList;
class FNotifyHook;
struct FReferenceSkeleton;

namespace UE::MLDeformer::TrainingDataProcessor
{
	class SBoneGroupsTreeWidget;
	class SBoneGroupsListWidget;
	
	class MLDEFORMERFRAMEWORKEDITOR_API FBoneGroupsListWidgetCommands final : public TCommands<FBoneGroupsListWidgetCommands>
	{
	public:
		FBoneGroupsListWidgetCommands();
		virtual void RegisterCommands() override;

	public:
		TSharedPtr<FUICommandInfo> CreateGroup;
		TSharedPtr<FUICommandInfo> DeleteSelectedItems;
		TSharedPtr<FUICommandInfo> ClearGroups;
		TSharedPtr<FUICommandInfo> AddBoneToGroup;
	};


	class FBoneGroupTreeElement final : public TSharedFromThis<FBoneGroupTreeElement>
	{
	public:
		static UE_API TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable,
		                                               const TSharedRef<FBoneGroupTreeElement>& InTreeElement,
		                                               const TSharedPtr<SBoneGroupsTreeWidget>& InTreeWidget);

		bool IsGroup() const { return GroupIndex != INDEX_NONE; }

	public:
		FString Name;
		TArray<TSharedPtr<FBoneGroupTreeElement>> Children;
		TWeakPtr<FBoneGroupTreeElement> ParentGroup;
		FSlateColor TextColor;
		int32 GroupIndex = INDEX_NONE;
		int32 GroupBoneIndex = INDEX_NONE;
	};


	class SBoneGroupTreeRowWidget final : public STableRow<TSharedPtr<FBoneGroupTreeElement>>
	{
	public:
		UE_API void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, const TSharedRef<FBoneGroupTreeElement>& InTreeElement,
		               const TSharedPtr<SBoneGroupsTreeWidget>& InTreeView);

	private:
		UE_API FText GetName() const;

	private:
		TWeakPtr<FBoneGroupTreeElement> WeakTreeElement;
		friend class SBoneGroupsTreeWidget;
	};


	class SBoneGroupsTreeWidget final : public STreeView<TSharedPtr<FBoneGroupTreeElement>>
	{
	public:
		friend class SBoneGroupsListWidget;

		SLATE_BEGIN_ARGS(SBoneGroupsTreeWidget)
			{
			}

			SLATE_ARGUMENT(TSharedPtr<SBoneGroupsListWidget>, BoneGroupsWidget)
		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs);
		UE_API void Refresh();
		UE_API int32 GetNumSelectedGroups() const;

	private:
		//~ Begin STreeView overrides.
		UE_API virtual FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
		//~ End STreeView overrides.

		UE_API void AddElement(const TSharedPtr<FBoneGroupTreeElement>& Element, const TSharedPtr<FBoneGroupTreeElement>& ParentElement);
		UE_API const TArray<TSharedPtr<FBoneGroupTreeElement>>& GetRootElements() const;
		UE_API TSharedRef<ITableRow> MakeTableRowWidget(TSharedPtr<FBoneGroupTreeElement> InItem, const TSharedRef<STableViewBase>& OwnerTable);
		UE_API TSharedPtr<SWidget> CreateContextMenuWidget() const;
		UE_API void UpdateTreeElements();
		UE_API void RefreshTree();

		static UE_API void HandleGetChildrenForTree(TSharedPtr<FBoneGroupTreeElement> InItem, TArray<TSharedPtr<FBoneGroupTreeElement>>& OutChildren);
		static UE_API TSharedPtr<SWidget> CreateContextWidget();

	private:
		TArray<TSharedPtr<FBoneGroupTreeElement>> RootElements;
		TWeakPtr<SBoneGroupsListWidget> BoneGroupsWidget;
	};


	DECLARE_DELEGATE_RetVal(TArray<FMLDeformerTrainingDataProcessorBoneGroup>*, FBoneGroupsListWidgetGetBoneGroups)

	/**
	 * A widget that shows a set of bone groups, and allows you to manage them by creating, removing and editing of groups.
	 * We see a bone group as a list of bone names. Multiple bone groups can exist. If you need only one list of bones
	 * then you can use the bone SBoneListWidget instead.
	 */
	class SBoneGroupsListWidget final : public SCompoundWidget, public FEditorUndoClient
	{
		friend class SBoneGroupsTreeWidget;

		SLATE_BEGIN_ARGS(SBoneGroupsListWidget)
			{
			}

			SLATE_ARGUMENT(TWeakObjectPtr<USkeleton>, Skeleton)
			SLATE_ARGUMENT(TWeakObjectPtr<UObject>, UndoObject)
			SLATE_EVENT(FBoneGroupsListWidgetGetBoneGroups, GetBoneGroups)
		SLATE_END_ARGS()

	public:
		UE_API virtual ~SBoneGroupsListWidget() override;

		UE_API void Construct(const FArguments& InArgs, FNotifyHook* InNotifyHook);

	private:
		//~ Begin FEditorUndoClient overrides.
		UE_API virtual void PostUndo(bool bSuccess) override;
		UE_API virtual void PostRedo(bool bSuccess) override;
		//~ End FEditorUndoClient overrides.

		UE_API void BindCommands(const TSharedPtr<FUICommandList>& InCommandList);
		UE_API void OnFilterTextChanged(const FText& InFilterText);
		UE_API void RefreshTree() const;
		UE_API FReply OnAddButtonClicked() const;
		UE_API FReply OnClearButtonClicked() const;

		UE_API TSharedPtr<SBoneGroupsTreeWidget> GetTreeWidget() const;
		UE_API TSharedPtr<FUICommandList> GetCommandList() const;
		UE_API TArray<FMLDeformerTrainingDataProcessorBoneGroup>* GetBoneGroupsValues() const;
		UE_API TWeakObjectPtr<USkeleton> GetSkeleton() const;
		UE_API const FString& GetFilterText() const;

		UE_API void OnCreateBoneGroup() const;
		UE_API void OnDeleteSelectedItems() const;
		UE_API void OnClearBoneGroups() const;
		UE_API void OnAddBoneToGroup() const;

	private:
		TSharedPtr<SBoneGroupsTreeWidget> TreeWidget;
		TWeakObjectPtr<USkeleton> Skeleton;
		TWeakObjectPtr<UObject> UndoObject;
		TSharedPtr<FUICommandList> CommandList;
		FBoneGroupsListWidgetGetBoneGroups GetBoneGroups;
		FString FilterText;
		FNotifyHook* NotifyHook = nullptr;
	};
} // namespace UE::MLDeformer::TrainingDataProcessor

#undef UE_API
