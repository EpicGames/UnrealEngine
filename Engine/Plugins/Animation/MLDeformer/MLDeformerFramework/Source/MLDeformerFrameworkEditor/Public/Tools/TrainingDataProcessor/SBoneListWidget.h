// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STreeView.h"
#include "Framework/Commands/Commands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "UObject/NameTypes.h"
#include "Styling/SlateColor.h"
#include "EditorUndoClient.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class USkeleton;
class FUICommandList;
class FNotifyHook;
struct FReferenceSkeleton;

namespace UE::MLDeformer::TrainingDataProcessor
{
	class SBoneTreeWidget;
	class SBoneListWidget;

	class MLDEFORMERFRAMEWORKEDITOR_API FBoneListWidgetCommands final : public TCommands<FBoneListWidgetCommands>
	{
	public:
		FBoneListWidgetCommands();
		virtual void RegisterCommands() override;

	public:
		TSharedPtr<FUICommandInfo> AddBones;
		TSharedPtr<FUICommandInfo> RemoveBones;
		TSharedPtr<FUICommandInfo> ClearBones;
	};


	class FBoneTreeWidgetElement final : public TSharedFromThis<FBoneTreeWidgetElement>
	{
	public:
		UE_API TSharedRef<ITableRow> MakeTreeRowWidget(const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FBoneTreeWidgetElement> InTreeElement,
		                                        TSharedPtr<SBoneTreeWidget> InTreeWidget);

	public:
		FName Name;
		TArray<TSharedPtr<FBoneTreeWidgetElement>> Children;
		FSlateColor TextColor;
	};


	class SBoneTreeRowWidget final : public STableRow<TSharedPtr<FBoneTreeWidgetElement>>
	{
	public:
		UE_API void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable, const TSharedRef<FBoneTreeWidgetElement>& InTreeElement,
		               const TSharedPtr<SBoneTreeWidget>& InTreeView);

	private:
		TWeakPtr<FBoneTreeWidgetElement> TreeElement;
		UE_API FText GetName() const;
	};


	class SBoneTreeWidget final : public STreeView<TSharedPtr<FBoneTreeWidgetElement>>
	{
		SLATE_BEGIN_ARGS(SBoneTreeWidget) { }
			SLATE_ARGUMENT(TSharedPtr<SBoneListWidget>, BoneListWidget)
		SLATE_END_ARGS()

	public:
		UE_API void Construct(const FArguments& InArgs);
		UE_API void RefreshElements(const TArray<FName>& BoneNames, const FReferenceSkeleton* RefSkeleton, const FString& FilterText);
		UE_API TArray<FName> ExtractAllElementNames() const;
		const TArray<TSharedPtr<FBoneTreeWidgetElement>>& GetRootElements() const { return RootElements; }

	private:
		UE_API virtual FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
		
		static UE_API void RecursiveAddNames(const FBoneTreeWidgetElement& Element, TArray<FName>& OutNames);
		UE_API TSharedRef<ITableRow> MakeTableRowWidget(TSharedPtr<FBoneTreeWidgetElement> InItem, const TSharedRef<STableViewBase>& OwnerTable);
		UE_API TSharedPtr<SWidget> OnContextMenuOpening() const;
		static UE_API TSharedPtr<FBoneTreeWidgetElement> FindParentElementForBone(FName BoneName, const FReferenceSkeleton& RefSkeleton, const TMap<FName, TSharedPtr<FBoneTreeWidgetElement>>& NameToElementMap);
		UE_API void RecursiveSortElements(const TSharedPtr<FBoneTreeWidgetElement>& Element);
		static UE_API void HandleGetChildrenForTree(TSharedPtr<FBoneTreeWidgetElement> InItem, TArray<TSharedPtr<FBoneTreeWidgetElement>>& OutChildren);

	private:
		TArray<TSharedPtr<FBoneTreeWidgetElement>> RootElements;
		TWeakPtr<SBoneListWidget> BoneListWidget;
	};


	DECLARE_DELEGATE_OneParam(FOnBoneListWidgetBonesAdded, const TArray<FName>& BoneNames)
	DECLARE_DELEGATE_OneParam(FOnBoneListWidgetBonesRemoved, const TArray<FName>& BoneNames)
	DECLARE_DELEGATE(FOnBoneListWidgetBonesCleared)
	DECLARE_DELEGATE_RetVal(TArray<FName>*, FBoneListWidgetGetBoneNames)

	/**
	 * The bone list widget, which displays a list of bones (in a hierarchy) and allows you to add and remove bones.
	 * It works directly on a TArray<FName> as source. You can use the GetBoneNames event to provide this array.
	 * You can use some of the delegates to listen for changes to the array.
	 */
	class SBoneListWidget final : public SCompoundWidget, public FEditorUndoClient
	{
		SLATE_BEGIN_ARGS(SBoneListWidget) {}
			SLATE_ARGUMENT(TWeakObjectPtr<USkeleton>, Skeleton)
			SLATE_ARGUMENT(TWeakObjectPtr<UObject>, UndoObject)
			SLATE_EVENT(FOnBoneListWidgetBonesAdded, OnBonesAdded)
			SLATE_EVENT(FOnBoneListWidgetBonesRemoved, OnBonesRemoved)
			SLATE_EVENT(FOnBoneListWidgetBonesCleared, OnBonesCleared)
			SLATE_EVENT(FBoneListWidgetGetBoneNames, GetBoneNames)
		SLATE_END_ARGS()

	public:
		UE_API virtual ~SBoneListWidget() override;

		UE_API void Construct(const FArguments& InArgs, FNotifyHook* InNotifyHook);
		UE_API void Refresh() const;
		UE_API TSharedPtr<SBoneTreeWidget> GetTreeWidget() const;
		UE_API TSharedPtr<FUICommandList> GetCommandList() const;

	private:
		// FEditorUndoClient overrides.
		UE_API virtual void PostUndo(bool bSuccess) override;
		UE_API virtual void PostRedo(bool bSuccess) override;
		// ~END FEditorUndoClient overrides.

		UE_API void BindCommands(const TSharedPtr<FUICommandList>& InCommandList);
		UE_API void OnFilterTextChanged(const FText& InFilterText);
		UE_API void RefreshTree() const;
		UE_API FReply OnAddBonesButtonClicked() const;
		UE_API FReply OnClearBonesButtonClicked() const;
		UE_API void NotifyPropertyChanged() const;

		UE_API void OnAddBones() const;
		UE_API void OnRemoveBones() const;
		UE_API void OnClearBones() const;

	private:
		TSharedPtr<SBoneTreeWidget> TreeWidget;
		TWeakObjectPtr<USkeleton> Skeleton;
		TWeakObjectPtr<UObject> UndoObject;
		TSharedPtr<FUICommandList> CommandList;
		FNotifyHook* NotifyHook = nullptr;
		FString FilterText;
		FOnBoneListWidgetBonesAdded OnBonesAdded;
		FOnBoneListWidgetBonesRemoved OnBonesRemoved;
		FOnBoneListWidgetBonesCleared OnBonesCleared;
		FBoneListWidgetGetBoneNames GetBoneNames;
	};
} // namespace UE::MLDeformer::TrainingDataProcessor

#undef UE_API
