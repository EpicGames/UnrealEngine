// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "MLDeformerModel.h"
#include "Misc/NotifyHook.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class IDetailsView;

namespace UE::MLDeformer::TrainingDataProcessor
{
	/**
	 * The main widget for the training data processor tool.
	 * This widget is basically what's inside the tab when this tool opens.
	 * It contains a detail view and generate button.
	 */
	class STrainingDataProcessorMainWidget final : public SCompoundWidget, public FEditorUndoClient, public FNotifyHook
	{
	public:
		SLATE_BEGIN_ARGS(STrainingDataProcessorMainWidget) { }
			SLATE_ARGUMENT(TObjectPtr<UMLDeformerModel>, Model)
		SLATE_END_ARGS()

		UE_API virtual ~STrainingDataProcessorMainWidget() override;
		UE_API void Construct(const FArguments& InArgs);

		//~ Begin FEditorUndoClient overrides.
		UE_API virtual void PostUndo(bool bSuccess) override;
		UE_API virtual void PostRedo(bool bSuccess) override;
		//~ End FEditorUndoClient overrides.

		//~ Begin FNotifyHook overrides.
		UE_API virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
		//~ End FNotifyHook overrides.

	private:
		UE_API FReply OnGenerateButtonClicked() const;
		UE_API bool IsValidConfiguration() const;
		UE_API bool IsObjectOfInterest(UObject* Object) const;
		UE_API void OnAssetModified(UObject* Object) const;
		UE_API void Refresh() const;

	private:
		/** The details view that shows the properties of our UMLDeformerTrainingDataProcessorSettings. */
		TSharedPtr<IDetailsView> DetailsView;

		/** A pointer to our model. */
		TObjectPtr<UMLDeformerModel> Model;

		/** The delegate that handles when an object got modified (any object). */
		FDelegateHandle ObjectModifiedHandle;
	};
} // namespace UE::MLDeformer::TrainingDataProcessor

#undef UE_API
