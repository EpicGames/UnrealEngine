// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MLDeformerTrainingDataProcessorSettings.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

struct FReferenceSkeleton;
class USkeleton;

namespace UE::MLDeformer::TrainingDataProcessor
{
	/**
	 * The training data processor algorithm, which is executed when you press the Generate button.
	 * This will take a set of animation sequences as input, then find the 'best' N number of frames.
	 * After that it can remix the poses based on a set of bone groups. Basically the keyframes of the
	 * bones inside the groups get shuffled. So all the bones in the bone group will change together.
	 */
	class FTrainingDataProcessor final
	{
	public:
		/**
		 * Run the algorithm using specific settings.
		 * The output of this process is a UAnimSequence that is specified inside the Settings.
		 * @param Settings The settings to use during execution. This could come from the UMLDeformerModel::GetTrainingDataProcessorSettings() for example.
		 * @param Skeleton The skeleton to use when sampling animations and apply the transforms on in the output animation.
		 * @return Returns true if successful, or false if not. Errors will be logged in case false is returned. 
		 */
		UE_API bool Execute(const UMLDeformerTrainingDataProcessorSettings& Settings, const USkeleton* Skeleton);

	private:
		UE_API void Clear();
		UE_API bool SampleFrames(const UMLDeformerTrainingDataProcessorSettings& Settings, const USkeleton& Skeleton);
		UE_API TArray<int32> FindBestFrameIndices(int32 MaxNumFrames, const FReferenceSkeleton& RefSkeleton) const;
		UE_API bool RemixPoses(const UMLDeformerTrainingDataProcessorSettings& Settings, const FReferenceSkeleton& RefSkeleton);
		UE_API bool SaveAnimationDataInAnimSequence(const UMLDeformerTrainingDataProcessorSettings& Settings, const FReferenceSkeleton& RefSkeleton, const TArray<int32>& FramesToInclude);
		UE_API int32 GetNumFrames() const;
		UE_API TArrayView<FTransform3f> GetFrameTransforms(int32 FrameIndex);
		UE_API TConstArrayView<FTransform3f> GetFrameTransforms(int32 FrameIndex) const;
		UE_API TConstArrayView<FTransform3f> GetFrameTransforms(const TArray<FTransform3f>& Transforms, int32 FrameIndex) const;
		UE_API TConstArrayView<FTransform3f> GetRefPoseTransforms() const;
		UE_API double CalculateMeanError(TConstArrayView<FTransform3f> PoseA, TConstArrayView<FTransform3f> PoseB) const;
		UE_API double CalculateMeanError(int32 PreviousPoseIndex, int32 CurrentPoseIndex) const;
		static UE_API FTransform3f GetRefPoseTransform(const FReferenceSkeleton& RefSkeleton, FName BoneName);
		
	private:
		/**
		 * The animation data represented as buffers of vectors and quaternions.
		 * The size of the arrays are the number of bones in the reference skeleton, multiplied by the number of frames.
		 * So the layout for a two bone skeleton would be like this:
		 * [(Bone1, Bone2), (Bone1, Bone2), (Bone1, Bone2), ...] where each (Bone1, Bone2) represents a single frame.
		 * In case of positions and scales, the value for Bone1 and Bone2 would be a FVector.
		 * For the rotations it is an FQuat.
		 */
		struct FAnimFrameData
		{
			// NumRefSkelBones * GetNumFrames().
			TArray<FTransform3f> Transforms;

			// NumRefSkelBones FTransforms.			
			TArray<FTransform3f> RefPoseTransforms;

			// Bone names in the bones list, that actually exist.
			TArray<FName> BoneNames;

			// For each entry in the BoneNames array, this contains an index into the FReferenceSkeleton.
			TArray<int32> BoneIndices;

			// Number of bones in the reference skeleton.
			int32 NumRefSkelBones = 0;
		};

		/** The animation data as flat float arrays. @see FAnimFrameData for more information. */
		FAnimFrameData AnimFrameData;
	};
} // namespace UE::MLDeformer::TrainingDataProcessor

#undef UE_API
