// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"

#define UE_API MLDEFORMERFRAMEWORKEDITOR_API

class UMLDeformerModel;
class UMLDeformerTrainingDataProcessorSettings;
class USkeleton;
struct FAssetData;

namespace UE::MLDeformer::TrainingDataProcessor
{
	/**
	 * The detail customization class for the UMLDeformerTrainingDataProcessorSettings class.
	 */
	class FTrainingDataProcessorSettingsDetailCustomization final : public IDetailCustomization
	{
	public:
		static UE_API TSharedRef<IDetailCustomization> MakeInstance();

		UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	private:
		UE_API EVisibility GetNoInputBonesWarningVisibility() const;
		UE_API EVisibility GetNoFramesWarningVisibility() const;
		UE_API EVisibility GetSkeletonMismatchErrorVisibility() const;
		
		UE_API FReply OnCreateNewButtonClicked() const;
		UE_API int32 GetTotalNumInputFrames() const;
		UE_API bool FilterAnimSequences(const FAssetData& AssetData) const;
		
		static UE_API void Refresh(IDetailLayoutBuilder* DetailBuilder);
		static UE_API FString FindDefaultAnimSequencePath(const UMLDeformerModel* Model);

	private:
		TWeakObjectPtr<UMLDeformerTrainingDataProcessorSettings> TrainingDataProcessorSettings;
	};
} // namespace UE::MLDeformer::TrainingDataProcessor

#undef UE_API
