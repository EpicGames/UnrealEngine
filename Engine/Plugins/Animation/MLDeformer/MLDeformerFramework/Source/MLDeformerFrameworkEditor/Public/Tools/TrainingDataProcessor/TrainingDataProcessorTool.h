// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class USkeleton;
class IPropertyHandle;

namespace UE::MLDeformer::TrainingDataProcessor
{
	/**
	 * Registers the Training Data Processor tool to the ML Deformer asset editor tools menu.
	 */
	MLDEFORMERFRAMEWORKEDITOR_API void RegisterTool();

	/**
	 * Find a skeleton for a given property.
	 * It does this by iterating over the outer objects of the property that's passed as parameter. It then checks the type of the object
	 * against a set of known types for which we know how to get the skeleton for.
	 * @param PropertyHandle The property handle to try to get a skeleton for.
	 * @return A pointer to the skeleton, or nullptr if not found.
	 */
	MLDEFORMERFRAMEWORKEDITOR_API USkeleton* FindSkeletonForProperty(const TSharedPtr<IPropertyHandle>& PropertyHandle);
}
